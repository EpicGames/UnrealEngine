{
    "FileVersion" : 3,
    "Version" : 1,
    "VersionName" : "0.1",
    "FriendlyName" : "Geometry",
    "Description" : "Adds Geometry Collection Container.",
    "Category": "Geometry",
    "CreatedBy" : "Epic Games, Inc.",
    "CreatedByURL" : "https://epicgames.com",
    "DocsURL" : "",
    "MarketplaceURL" : "",
    "SupportURL" : "",
    "EnabledByDefault" : false,
    "CanContainContent" : true,
    "IsBetaVersion" : true,
    "Installed" : false,
	"Modules": [
		{
			"Name": "GeometryCollectionEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "GeometryCollectionTracks",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "GeometryCollectionSequencer",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "GeometryCollectionNodes",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "GeometryCollectionDepNodes",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ProceduralMeshComponent",
			"Enabled": true
		},
		{
			"Name": "ChaosSolverPlugin",
			"Enabled": true
		},
		{
			"Name": "Dataflow",
			"Enabled": true
		},
		{
			"Name": "PlanarCut",
			"Enabled": true
		},
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "GeometryDataflow",
			"Enabled": true
		},
		{
			"Name": "Fracture",
			"Enabled": true
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		}
	]
}
