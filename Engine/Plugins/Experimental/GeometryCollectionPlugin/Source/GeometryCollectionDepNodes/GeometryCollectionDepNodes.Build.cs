// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GeometryCollectionDepNodes : ModuleRules
	{
        public GeometryCollectionDepNodes(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"ChaosCore",
					"Chaos",
					"DataflowCore",
					"DataflowEngine",
					"DataflowEnginePlugin",
					"GeometryCollectionEngine",
					"GeometryCollectionNodes",
					"Engine"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] 
				{
				}
			);
		}
	}
}
