// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionDepNodesPlugin.h"

#include "CoreMinimal.h"

#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/GeometryCollectionTransferVertexScalarAttributeDepNode.h"
#include "Dataflow/SetVertexColorFromFloatArrayDepNode.h"
#include "Dataflow/SetVertexColorFromVertexSelectionDepNode.h"

#define LOCTEXT_NAMESPACE "DataflowNodes"


void IGeometryCollectionDepNodesPlugin::StartupModule()
{
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGeometryCollectionTransferVertexScalarAttributeNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetVertexColorInCollectionFromVertexSelectionDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetVertexColorInCollectionFromFloatArrayDataflowNode);
}

void IGeometryCollectionDepNodesPlugin::ShutdownModule()
{
}


IMPLEMENT_MODULE(IGeometryCollectionDepNodesPlugin, GeometryCollectionDepNodes)


#undef LOCTEXT_NAMESPACE
