// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionTransferVertexScalarAttributeDepNode.h"

#include "Chaos/Triangle.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/HierarchicalSpatialHash.h"
#include "Chaos/TriangleCollisionPoint.h"
#include "Dataflow/DataflowInputOutput.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionTransferVertexScalarAttributeDepNode)
#define LOCTEXT_NAMESPACE "FGeometryCollectionTransferVertexScalarAttributeDepNode"

namespace UE::Private {

	class FTransferFacade
	{
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;
	public:
		FTransferFacade(FManagedArrayCollection& InCollection)
			: ConstCollection(InCollection)
			, Collection(&InCollection)
			, BoneMap(InCollection, FName("BoneMap"), FName("Vertices"))
			, Vertex(InCollection, FName("Vertex"), FName("Vertices"))
			, Indices(InCollection, FName("Indices"), FName("Faces"))
			, Transform(InCollection, FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup)
			, Parent(InCollection, FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup)
			, VertexStart(InCollection, "VertexStart", FGeometryCollection::GeometryGroup)
			, VertexCount(InCollection, "VertexCount", FGeometryCollection::GeometryGroup)
			, FaceStart(InCollection, "FaceStart", FGeometryCollection::GeometryGroup)
			, FaceCount(InCollection, "FaceCount", FGeometryCollection::GeometryGroup)
		{}

		FTransferFacade(const FManagedArrayCollection& InCollection)
			: ConstCollection(InCollection)
			, Collection(nullptr)
			, BoneMap(InCollection, FName("BoneMap"), FName("Vertices"))
			, Vertex(InCollection, FName("Vertex"), FName("Vertices"))
			, Indices(InCollection, FName("Indices"), FName("Faces"))
			, Transform(InCollection, FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup)
			, Parent(InCollection, FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup)
			, VertexStart(InCollection, "VertexStart", FGeometryCollection::GeometryGroup)
			, VertexCount(InCollection, "VertexCount", FGeometryCollection::GeometryGroup)
			, FaceStart(InCollection, "FaceStart", FGeometryCollection::GeometryGroup)
			, FaceCount(InCollection, "FaceCount", FGeometryCollection::GeometryGroup)
		{}


		bool IsValid() const {
			return BoneMap.IsValid() && Vertex.IsValid() && Indices.IsValid() && Transform.IsValid() && Parent.IsValid() && VertexStart.IsValid() &&
				VertexCount.IsValid() && FaceStart.IsValid() && FaceCount.IsValid();
		}

		const TManagedArray<float>* GetFloatArray(FString AttributeName, FString Group) const
		{
			return ConstCollection.FindAttribute<float>(FName(AttributeName), FName(Group));
		}

		TManagedArray<float>* GetFloatArray(FString AttributeName, FString Group)
		{
			TManagedArray<float>* TargetFloatArray = nullptr;
			if (!Collection->HasAttribute(FName(AttributeName), FName(Group)))
			{
				Collection->AddAttribute<float>(FName(AttributeName), FName(Group));
			}
			return Collection->FindAttribute<float>(FName(AttributeName), FName(Group));
		}


		TManagedArrayAccessor<int32> BoneMap;
		TManagedArrayAccessor<FVector3f> Vertex;
		TManagedArrayAccessor<FIntVector3> Indices;
		TManagedArrayAccessor<FTransform3f> Transform;
		TManagedArrayAccessor<int32> Parent;
		TManagedArrayAccessor<int32> VertexStart;
		TManagedArrayAccessor<int32> VertexCount;
		TManagedArrayAccessor<int32> FaceStart;
		TManagedArrayAccessor<int32> FaceCount;
	};

}

void FGeometryCollectionTransferVertexScalarAttributeNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	typedef TPair<FName, FName> NamePair;
	FCollectionAttributeKey Key = GetValue(Context, &AttributeKey, AttributeKey);

	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection TargetCollection = GetValue(Context, &Collection);
		const FManagedArrayCollection& SampleCollection = GetValue(Context, &FromCollection);

		UE::Private::FTransferFacade Target(TargetCollection);
		const UE::Private::FTransferFacade Sample(SampleCollection);

		if (Target.IsValid() && Sample.IsValid())
		{
			if (TManagedArray<float>* TargetFloatArray = Target.GetFloatArray(Key.Attribute, Key.Group))
			{
				TargetFloatArray->Fill(0.f);

				TArray<FIntVector2> AlignedGeometry = FindSourceToTargetGeometryMap(SampleCollection, TargetCollection);
				if (AlignedGeometry.Num() == TargetCollection.NumElements(FGeometryCollection::GeometryGroup))
				{
					PairedGeometryTransfer(Key, AlignedGeometry, Sample, Target, TargetFloatArray);
				}
				else
				{
					NearestVertexTransfer(Key, Sample, Target, TargetFloatArray);
				}
			}
		}

		SetValue(Context, MoveTemp(TargetCollection), &Collection);
	}
	else if (Out->IsA(&AttributeKey))
	{
		SetValue(Context, MoveTemp(Key), &AttributeKey);
	}
}


TArray<FIntVector2> FGeometryCollectionTransferVertexScalarAttributeNode::FindSourceToTargetGeometryMap(const FManagedArrayCollection& AttributeCollectionVal, const FManagedArrayCollection& CollectionVal) const
{
	TArray<FIntVector2> Mapping;
	const TManagedArray<FString>* SourceName = AttributeCollectionVal.FindAttribute<FString>(FName("BoneName"), FTransformCollection::TransformGroup);
	const TManagedArray<int32>* SourceGeometryGroup = AttributeCollectionVal.FindAttribute<int32>(FName("TransformToGeometryIndex"), FTransformCollection::TransformGroup);
	const TManagedArray<FString>* TargetName = CollectionVal.FindAttribute<FString>(FName("BoneName"), FTransformCollection::TransformGroup);
	const TManagedArray<int32>* TargetGeometryGroup = CollectionVal.FindAttribute<int32>(FName("TransformToGeometryIndex"), FTransformCollection::TransformGroup);
	if (SourceName && SourceGeometryGroup && TargetName && TargetGeometryGroup)
	{
		for (int i = 0; i < SourceName->Num(); i++)
		{
			for (int j = 0; j < TargetName->Num(); j++)
			{
				FString TestName = FString::Printf(TEXT("%s_Tet"), *(*SourceName)[i]);
				if ((*TargetName)[j].StartsWith(TestName))
				{
					Mapping.Add(FIntVector2((*SourceGeometryGroup)[i], (*TargetGeometryGroup)[j]));
					break;
				}
			}
		}
	}

	return Mapping;
}

void FGeometryCollectionTransferVertexScalarAttributeNode::PairedGeometryTransfer(FCollectionAttributeKey Key, const TArray<FIntVector2>& PairedGeometry,
	const UE::Private::FTransferFacade& Sample, UE::Private::FTransferFacade&  Target, TManagedArray<float>* TargetFloatArray) const
{
	if (const TManagedArray<float>* FloatArray = Sample.GetFloatArray(Key.Attribute, Key.Group))
	{
		Chaos::FReal SphereFullRadius;

		if (SampleScale == EDataflowTransferNodeSampleScale::Dataflow_Transfer_Asset_Edge || SampleScale == EDataflowTransferNodeSampleScale::Dataflow_Transfer_Asset_Bound)
		{
			// Build component space vertices for TargetCollection
			TArray<FVector> ComponentSpaceFullTargetVertices; // size of num vertices of the geometry entry. 
			BuildComponentSpaceVertices(&Target.Transform.Get(), &Target.Parent.Get(), &Target.BoneMap.Get(), &Target.Vertex.Get(),
				0, Target.Vertex.Num(), ComponentSpaceFullTargetVertices);

			// Build component space vertices for TargetCollection
			TArray<FVector> ComponentSpaceFullVertices; // size of num vertices of the geometry entry
			BuildComponentSpaceVertices(&Sample.Transform.Get(), &Sample.Parent.Get(), &Sample.BoneMap.Get(), &Sample.Vertex.Get(),
				0, Sample.Vertex.Num(), ComponentSpaceFullVertices);
			if (SampleScale == EDataflowTransferNodeSampleScale::Dataflow_Transfer_Asset_Edge)
			{
				SphereFullRadius = Chaos::FReal(EdgeMultiplier * FMath::Max(
					MaxEdgeLength(ComponentSpaceFullTargetVertices, Target.Indices.Get(), 0, 0, Target.Indices.Num()),
					MaxEdgeLength(ComponentSpaceFullVertices, Sample.Indices.Get(), 0, 0, Sample.Indices.Num())));
			}
			else if (SampleScale == EDataflowTransferNodeSampleScale::Dataflow_Transfer_Asset_Bound)
			{
				Chaos::TVec3<float> CoordMaxs(-FLT_MAX);
				Chaos::TVec3<float> CoordMins(FLT_MAX);
				for (int32 i = 0; i < ComponentSpaceFullVertices.Num(); i++)
				{
					for (int32 j = 0; j < 3; j++)
					{
						if (ComponentSpaceFullVertices[i][j] > CoordMaxs[j])
						{
							CoordMaxs[j] = ComponentSpaceFullVertices[i][j];
						}
						if (ComponentSpaceFullVertices[i][j] < CoordMins[j])
						{
							CoordMins[j] = ComponentSpaceFullVertices[i][j];
						}
					}
				}
				Chaos::TVec3<float> CoordDiff = (CoordMaxs - CoordMins) * BoundMultiplier;
				SphereFullRadius = Chaos::FReal(FGenericPlatformMath::Min(CoordDiff[0], FGenericPlatformMath::Min(CoordDiff[1], CoordDiff[2])));
			}
		}
		ParallelFor(PairedGeometry.Num(), [&](int32 Pdx)
		{
			int32 AttributeGeometryIndex = PairedGeometry[Pdx][0];
			int32 TargetGeometryIndex = PairedGeometry[Pdx][1];
			if (ensure(0 <= AttributeGeometryIndex && AttributeGeometryIndex < Sample.VertexStart.Num()))
			{
				if (ensure(0 <= TargetGeometryIndex && TargetGeometryIndex < Target.VertexStart.Num()))
				{
					// Build component space vertices for TargetCollection
					TArray<FVector> ComponentSpaceTargetVertices; // size of num vertices of the geometry entry. 
					BuildComponentSpaceVertices(&Target.Transform.Get(), &Target.Parent.Get(), &Target.BoneMap.Get(), &Target.Vertex.Get(),
						Target.VertexStart[TargetGeometryIndex], Target.VertexCount[TargetGeometryIndex], ComponentSpaceTargetVertices);

					// Build component space vertices for SampleCollection
					TArray<FVector> ComponentSpaceVertices; // size of num vertices of the geometry entry
					BuildComponentSpaceVertices(&Sample.Transform.Get(), &Sample.Parent.Get(), &Sample.BoneMap.Get(), &Sample.Vertex.Get(),
						Sample.VertexStart[AttributeGeometryIndex], Sample.VertexCount[AttributeGeometryIndex], ComponentSpaceVertices);

					// build Sphere based BVH
					Chaos::FReal SphereRadius = SphereFullRadius;
					if (SampleScale == EDataflowTransferNodeSampleScale::Dataflow_Transfer_Component_Edge)
					{
						SphereRadius = Chaos::FReal(EdgeMultiplier * FMath::Max(
							MaxEdgeLength(ComponentSpaceTargetVertices, Target.Indices.Get(), Target.VertexStart[TargetGeometryIndex], Target.FaceStart[TargetGeometryIndex], Target.FaceCount[TargetGeometryIndex]),
							MaxEdgeLength(ComponentSpaceVertices, Sample.Indices.Get(), Sample.VertexStart[AttributeGeometryIndex], Sample.FaceStart[AttributeGeometryIndex], Sample.FaceCount[AttributeGeometryIndex])));
					}

					int32 TargetVertexStartVal = Target.VertexStart[TargetGeometryIndex];
					int32 TargetVertexCountVal = Target.VertexCount[TargetGeometryIndex];
					int32 VertexStartVal = Sample.VertexStart[AttributeGeometryIndex];
					int32 FaceStartVal = Sample.FaceStart[AttributeGeometryIndex];
					int32 FaceCountVal = Sample.FaceCount[AttributeGeometryIndex];

					if (BoundingVolumeType == EDataflowTransferNodeBoundingVolume::Dataflow_Transfer_Triangle)
					{
						TArray<Chaos::TVec3<Chaos::FReal>> ComponentSpaceVerticesTVec3;
						ComponentSpaceVerticesTVec3.SetNum(ComponentSpaceVertices.Num());
						for (int32 SourceIndex = 0; SourceIndex < ComponentSpaceVerticesTVec3.Num(); SourceIndex++)
						{
							ComponentSpaceVerticesTVec3[SourceIndex] = Chaos::TVec3<Chaos::FReal>(ComponentSpaceVertices[SourceIndex]);
						}
						TConstArrayView<Chaos::TVec3<Chaos::FReal>> ConstComponentSpaceVertices(ComponentSpaceVerticesTVec3);
						Chaos::FTriangleMesh TriangleMesh;
						TArray<Chaos::TVec3<int32>> SourceElements;
						SourceElements.SetNum(FaceCountVal);
						for (int32 ElementIndex = 0; ElementIndex < FaceCountVal; ++ElementIndex)
						{
							FIntVector3 Element = Sample.Indices[FaceStartVal + ElementIndex];
							SourceElements[ElementIndex] = Chaos::TVec3<int32>(Element[0]-VertexStartVal, Element[1]-VertexStartVal, Element[2]-VertexStartVal);
						}
						TriangleMesh.Init(SourceElements);
						Chaos::FTriangleMesh::TSpatialHashType<Chaos::FReal> SpatialHash;
						TriangleMesh.BuildSpatialHash(ConstComponentSpaceVertices, SpatialHash, SphereRadius);
						for (int32 TargetIndex = 0; TargetIndex < TargetVertexCountVal; TargetIndex++)
						{
							TArray<Chaos::TTriangleCollisionPoint<Chaos::FReal>> Result;
							if (TriangleMesh.PointClosestTriangleQuery(SpatialHash, ConstComponentSpaceVertices,
								TargetIndex, Chaos::TVec3<Chaos::FReal>(ComponentSpaceTargetVertices[TargetIndex]), SphereRadius / 2.f, SphereRadius / 2.f,
								[](const int32 PointIndex, const int32 TriangleIndex)->bool {return true; }, Result))
							{
								for (const Chaos::TTriangleCollisionPoint<Chaos::FReal>& CollisionPoint : Result)
								{
									Chaos::FRealSingle CurrentDistance = abs(CollisionPoint.Phi);
									float TriRadius = FalloffThreshold * MaxEdgeLength(ComponentSpaceVertices, Sample.Indices.Get(), VertexStartVal, FaceStartVal+CollisionPoint.Indices[1], 1);
									float FalloffScale = CalculateFalloffScale(Falloff, TriRadius, CurrentDistance);
									if (!FMath::IsNearlyZero(FalloffScale))
									{
										int32 TargetCandidateIndex = CollisionPoint.Indices[0]+ TargetVertexStartVal;
										if (ensure(0 <= TargetCandidateIndex && TargetCandidateIndex < TargetFloatArray->Num()))
										{
											float Value = 0.f;
											for (int32 k = 0; k < 3; k++)
											{
												Value += FalloffScale * (CollisionPoint.Bary[k] * (*FloatArray)[Sample.Indices[FaceStartVal+CollisionPoint.Indices[1]][k]]);
											}
											(*TargetFloatArray)[TargetCandidateIndex] = FMath::Max((*TargetFloatArray)[TargetCandidateIndex], Value);
										}
									}
								}
							}
						}
					}
					else if (BoundingVolumeType == EDataflowTransferNodeBoundingVolume::Dataflow_Transfer_Vertex)
					{
						TUniquePtr<UE::Private::BVH> VertexBVH(BuildParticleSphereBVH(ComponentSpaceTargetVertices, SphereRadius));
						for (int32 i = 0; i < FaceCountVal; i++)
						{
							FIntVector3 ComponentTriangle = Sample.Indices[FaceStartVal + i] - FIntVector3(VertexStartVal);
							if (TriangleHasWeightsToTransfer(Sample.Indices[FaceStartVal + i], *FloatArray))
							{
								TArray<int32> TargetVertexIntersection({});
								TriangleToVertexIntersections(*VertexBVH, ComponentSpaceVertices, ComponentTriangle, TargetVertexIntersection);

								for (int32 j = 0; j < TargetVertexIntersection.Num(); j++)
								{
									Chaos::TVector<float, 3> Bary,
										TriPos0(ComponentSpaceVertices[ComponentTriangle[0]]),
										TriPos1(ComponentSpaceVertices[ComponentTriangle[1]]),
										TriPos2(ComponentSpaceVertices[ComponentTriangle[2]]),
										ParticlePos(ComponentSpaceTargetVertices[TargetVertexIntersection[j]]);

									Chaos::TVector<Chaos::FRealSingle, 3> ClosestPoint = Chaos::FindClosestPointAndBaryOnTriangle(TriPos0, TriPos1, TriPos2, ParticlePos, Bary);
									Chaos::FRealSingle CurrentDistance = (ParticlePos - ClosestPoint).Size();
									float TriRadius = FalloffThreshold * MaxEdgeLength(ComponentSpaceVertices, Sample.Indices.Get(), VertexStartVal, FaceStartVal+i, 1);
									float FalloffScale = CalculateFalloffScale(Falloff, TriRadius, CurrentDistance);
									if (!FMath::IsNearlyZero(FalloffScale))
									{
										int32 TargetIndex = TargetVertexIntersection[j] + TargetVertexStartVal;
										if (ensure(0 <= TargetIndex && TargetIndex < TargetFloatArray->Num()))
										{
											float Value = 0.f;
											for (int32 k = 0; k < 3; k++)
											{
												Value += FalloffScale * (Bary[k] * (*FloatArray)[ComponentTriangle[k] + VertexStartVal]);
											}
											(*TargetFloatArray)[TargetIndex] = FMath::Max((*TargetFloatArray)[TargetIndex], Value);
										}
									}
								}
							}
						}
					}
				}
			}
		});
	}
}

void FGeometryCollectionTransferVertexScalarAttributeNode::NearestVertexTransfer(FCollectionAttributeKey Key, const UE::Private::FTransferFacade& Sample, UE::Private::FTransferFacade& Target, TManagedArray<float>* TargetFloatArray) const
{
	if (const TManagedArray<float>* FloatArray = Sample.GetFloatArray(Key.Attribute, Key.Group))
	{
		// Build component space vertices for TargetCollection
		TArray<FVector> ComponentSpaceTargetVertices;
		BuildComponentSpaceVertices(&Target.Transform.Get(), &Target.Parent.Get(), &Target.BoneMap.Get(), &Target.Vertex.Get(), 0, Target.Vertex.Num(), ComponentSpaceTargetVertices);

		// Build component space vertices for SourceCollection
		TArray<FVector> ComponentSpaceVertices;
		BuildComponentSpaceVertices(&Sample.Transform.Get(), &Sample.Parent.Get(), &Sample.BoneMap.Get(), &Sample.Vertex.Get(), 0, Sample.Vertex.Num(), ComponentSpaceVertices);

		// build Sphere based BVH
		Chaos::FReal SphereRadius = Chaos::FReal(EdgeMultiplier * FMath::Max(
			MaxEdgeLength(ComponentSpaceTargetVertices, Target.Indices.Get(), 0, 0, Target.Indices.Num()),
			MaxEdgeLength(ComponentSpaceVertices, Sample.Indices.Get(), 0, 0, Sample.Indices.Num())));
		TUniquePtr<UE::Private::BVH> VertexBVH(BuildParticleSphereBVH(ComponentSpaceTargetVertices, SphereRadius));

		for (int32 i = 0; i < Sample.Indices.Num(); i++)
		{
			FIntVector3 Triangle = Sample.Indices[i];
			if (TriangleHasWeightsToTransfer(Triangle, *FloatArray))
			{
				TArray<int32> TargetVertexIntersection({});
				TriangleToVertexIntersections(*VertexBVH, ComponentSpaceVertices, Triangle, TargetVertexIntersection);

				for (int32 j = 0; j < TargetVertexIntersection.Num(); j++)
				{
					Chaos::TVector<float, 3> Bary,
						TriPos0(ComponentSpaceVertices[Sample.Indices[i][0]]),
						TriPos1(ComponentSpaceVertices[Sample.Indices[i][1]]),
						TriPos2(ComponentSpaceVertices[Sample.Indices[i][2]]),
						ParticlePos(ComponentSpaceTargetVertices[TargetVertexIntersection[j]]);

					Chaos::TVector<Chaos::FRealSingle, 3> ClosestPoint = Chaos::FindClosestPointAndBaryOnTriangle(TriPos0, TriPos1, TriPos2, ParticlePos, Bary);
					Chaos::FRealSingle CurrentDistance = (ParticlePos - ClosestPoint).Size();
					float TriRadius = FalloffThreshold * MaxEdgeLength(ComponentSpaceVertices, Sample.Indices.Get(), 0, i, 1);
					float FalloffScale = CalculateFalloffScale(Falloff, TriRadius, CurrentDistance);
					if (!FMath::IsNearlyZero(FalloffScale))
					{
						int32 TargetIndex = TargetVertexIntersection[j];
						if (ensure(0 <= TargetIndex && TargetIndex < TargetFloatArray->Num()))
						{
							float Value = 0.f;
							for (int32 k = 0; k < 3; k++)
							{
								Value += FalloffScale * (Bary[k] * (*FloatArray)[Sample.Indices[i][k]]);
							}
							(*TargetFloatArray)[TargetIndex] = FMath::Max((*TargetFloatArray)[TargetIndex], Value);
						}
					}
				}
			}
		}
	}
}


float FGeometryCollectionTransferVertexScalarAttributeNode::MaxEdgeLength(TArray<FVector>& Vert, const TManagedArray<FIntVector3>& Tri, int VertexOffset, int TriStart, int TriCount)
{
	auto TriInRange = [](const FIntVector3& T, int Max) {
		for (int k = 0; k < 3; k++)
		{
			if (ensure(0 <= T[k] && T[k] < Max))
			{
				return true;
			}
		}
		return false;
	};

	float Max = 0;
	int TriStop = TriStart + TriCount;
	for (int i = TriStart; i < TriStop; i++)
	{
		if (TriInRange(Tri[i] - FIntVector3(VertexOffset), Vert.Num()))
		{
			Max = FMath::Max(Max, FVector3f(Vert[Tri[i][0] - VertexOffset] - Vert[Tri[i][1] - VertexOffset]).SquaredLength());
			Max = FMath::Max(Max, FVector3f(Vert[Tri[i][0] - VertexOffset] - Vert[Tri[i][2] - VertexOffset]).SquaredLength());
			Max = FMath::Max(Max, FVector3f(Vert[Tri[i][1] - VertexOffset] - Vert[Tri[i][2] - VertexOffset]).SquaredLength());
		}
	}
	return FMath::Sqrt(Max);
}

void FGeometryCollectionTransferVertexScalarAttributeNode::BuildComponentSpaceVertices(const TManagedArray<FTransform3f>* LocalSpaceTransform, const TManagedArray<int32>* Parent, const TManagedArray<int32>* BoneMapArray,
	const TManagedArray<FVector3f>* VertexArray, int32 Start, int32 Count, TArray<FVector>& ComponentSpaceVertices)
{
	TArray<FTransform> ComponentTransform;
	GeometryCollectionAlgo::GlobalMatrices(*LocalSpaceTransform, *Parent, ComponentTransform);

	ComponentSpaceVertices.SetNumUninitialized(Count);
	for (int i = 0; i < Count; i++)
	{
		int j = i + Start;
		if (0 < (*BoneMapArray)[i] && (*BoneMapArray)[i] < ComponentTransform.Num())
		{
			ComponentSpaceVertices[i] = ComponentTransform[(*BoneMapArray)[j]].TransformPosition(FVector((*VertexArray)[j]));
		}
		else
		{
			ComponentSpaceVertices[i] = FVector((*VertexArray)[j]);
		}
	}
}

UE::Private::BVH* FGeometryCollectionTransferVertexScalarAttributeNode::BuildParticleSphereBVH(const TArray<FVector>& Vertices, float Radius)
{
	TArray<Chaos::FSphere*> VertexSpherePtrs;
	TArray<Chaos::FSphere> VertexSpheres;

	VertexSpheres.Init(Chaos::FSphere(Chaos::TVec3<Chaos::FReal>(0), Radius), Vertices.Num());
	VertexSpherePtrs.SetNum(Vertices.Num());

	for (int32 i = 0; i < Vertices.Num(); i++)
	{
		Chaos::TVec3<Chaos::FReal> SphereCenter(Vertices[i]);
		Chaos::FSphere VertexSphere(SphereCenter, Radius);
		VertexSpheres[i] = Chaos::FSphere(SphereCenter, Radius);
		VertexSpherePtrs[i] = &VertexSpheres[i];
	}
	return new UE::Private::BVH(VertexSpherePtrs);
}

bool FGeometryCollectionTransferVertexScalarAttributeNode::TriangleHasWeightsToTransfer(const FIntVector3& T, const TManagedArray<float>& F)
{
	return !FMath::IsNearlyZero((F[T[0]] + F[T[1]] + F[T[2]]));
}

void FGeometryCollectionTransferVertexScalarAttributeNode::TriangleToVertexIntersections(
	UE::Private::BVH& VertexBVH, const TArray<FVector>& ComponentSpaceVertices, const FIntVector3& Triangle, TArray<int32>& OutTargetVertexIntersection)
{
	OutTargetVertexIntersection.Empty();

	TArray<int32> TargetVertexIntersection0 = VertexBVH.FindAllIntersections(ComponentSpaceVertices[Triangle[0]]);
	TArray<int32> TargetVertexIntersection1 = VertexBVH.FindAllIntersections(ComponentSpaceVertices[Triangle[1]]);
	TArray<int32> TargetVertexIntersection2 = VertexBVH.FindAllIntersections(ComponentSpaceVertices[Triangle[2]]);
	TargetVertexIntersection0.Sort();
	TargetVertexIntersection1.Sort();
	TargetVertexIntersection2.Sort();

	for (int32 k = 0; k < TargetVertexIntersection0.Num(); k++)
	{
		if (TargetVertexIntersection1.Contains(TargetVertexIntersection0[k])
			&& TargetVertexIntersection2.Contains(TargetVertexIntersection0[k]))
		{
			OutTargetVertexIntersection.Emplace(TargetVertexIntersection0[k]);
		}
	}
}


float FGeometryCollectionTransferVertexScalarAttributeNode::CalculateFalloffScale(EDataflowTransferNodeFalloff FalloffSetting, float Threshold, float Distance)
{
	float Denominator = 1.0;
	if (Distance > Threshold && !FMath::IsNearlyZero(Threshold))
	{
		Denominator = Distance / Threshold;
	}
	switch (FalloffSetting)
	{
	case EDataflowTransferNodeFalloff::Dataflow_Transfer_Linear:
		return 1. / Denominator;
	case EDataflowTransferNodeFalloff::Dataflow_Transfer_Squared:
		return 1. / FMath::Square(Denominator);
	}
	return 1.0;
}


#undef LOCTEXT_NAMESPACE
