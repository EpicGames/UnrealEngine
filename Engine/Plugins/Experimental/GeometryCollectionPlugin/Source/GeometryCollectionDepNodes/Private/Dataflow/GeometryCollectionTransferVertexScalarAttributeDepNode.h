// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Chaos/BoundingVolumeHierarchy.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowConnectionTypes.h"
#include "GeometryCollection/GeometryCollection.h"

#include "GeometryCollectionTransferVertexScalarAttributeDepNode.generated.h"


namespace UE::Private 
{
	class FTransferFacade;
	typedef Chaos::TBoundingVolumeHierarchy<TArray<Chaos::FSphere*>, TArray<int32>, Chaos::FReal, 3> BVH;
};

UENUM(BlueprintType)
enum class EDataflowTransferNodeFalloff : uint8
{
	/** Squared falloff based on distance from triangle*/
	Dataflow_Transfer_Squared  UMETA(DisplayName = "Squared"),

	/** Linear falloff based on distance from triangle*/
	Dataflow_Transfer_Linear UMETA(DisplayName = "Linear"),

	/** No distance falloff */
	Dataflow_Transfer_None UMETA(DisplayName = "None"),
	//~~~
	//256th entry
	Dataflow_Transfer_Dataflow_Max UMETA(Hidden)
};

UENUM(BlueprintType)
enum class EDataflowTransferNodeSampleScale : uint8
{
	/** Bounding volume hierarchy cell size based on max edge length of each geometry group*/
	Dataflow_Transfer_Component_Edge  UMETA(DisplayName = "Component Max Edge"),

	/** Bounding volume hierarchy cell size based on max edge length of the whole asset*/
	Dataflow_Transfer_Asset_Edge UMETA(DisplayName = "Asset Max Edge"),

	/** Bounding volume hierarchy cell size based on max length of the bounding box of the whole asset*/
	Dataflow_Transfer_Asset_Bound UMETA(DisplayName = "Asset Max Bound"),

	//~~~
	//256th entry
	Dataflow_Transfer_Dataflow_Max UMETA(Hidden)
};

UENUM(BlueprintType)
enum class EDataflowTransferNodeBoundingVolume : uint8
{
	/** Bounding volume on vertices of the source triangle mesh*/
	Dataflow_Transfer_Vertex  UMETA(DisplayName = "Vertex"),

	/** Bounding volume on triangles of the source triangle mesh*/
	Dataflow_Transfer_Triangle UMETA(DisplayName = "Triangle"),

	//~~~
	//256th entry
	Dataflow_Transfer_Dataflow_Max UMETA(Hidden)
};

/**
 * Transfer float properties from a sample collection to a target collection. 
 */
USTRUCT(meta = (Deprecated = "5.5"))
struct FGeometryCollectionTransferVertexScalarAttributeNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGeometryCollectionTransferVertexScalarAttributeNode, "TransferVertexScalarAttribute", "GeometryCollection", "Transfer a named scalar attribute from the Source Collection to the Target Collection")
	DATAFLOW_NODE_RENDER_TYPE("SurfaceWeightsRender", FGeometryCollection::StaticType(), "Collection", "AttributeKey")

public:

	/* Target collection to transfer vertex float attribute to. */
	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/* Sample collection to transfer vertex float attribute from. */
	UPROPERTY(Meta = (DataflowInput, DisplayName = "FromCollection"))
	FManagedArrayCollection FromCollection;

	/* The name of the vertex attribute to generate indices from. */
	UPROPERTY(EditAnywhere, Category = "Dataflow", Meta = (DataflowInput, DataflowOutput, DisplayName = "AttributeKey", DataflowPassthrough = "AttributeKey"))
	FCollectionAttributeKey AttributeKey;

	/* Bounding volume type for source assets[default: Triangle] */
	UPROPERTY(EditAnywhere, Category = "Thresholds")
	EDataflowTransferNodeBoundingVolume BoundingVolumeType = EDataflowTransferNodeBoundingVolume::Dataflow_Transfer_Triangle;

	/* Bounding volume hierarchy cell size for neighboring vertices to transfer into[default: Asset] */
	UPROPERTY(EditAnywhere, Category = "Thresholds")
	EDataflowTransferNodeSampleScale SampleScale = EDataflowTransferNodeSampleScale::Dataflow_Transfer_Asset_Bound;

	/* Falloff of sample value based on distance from sample triangle[default: Squared] */
	UPROPERTY(EditAnywhere, Category = "Thresholds")
	EDataflowTransferNodeFalloff Falloff = EDataflowTransferNodeFalloff::Dataflow_Transfer_Squared;

	/* Threshold based on distance from sample triangle.Values sampled past the threshold will falloff.[Defaults to 1 percent of triangle size(0.01)] */
	UPROPERTY(EditAnywhere, Category = "Thresholds")
	float FalloffThreshold = 0.01f;

	/* Edge multiplier for the Bounding Volume Hierarchy(BVH) target's particle search radius. */
	UPROPERTY(EditAnywhere, Category = "Thresholds", meta = (EditCondition = "SampleScale == EDataflowTransferNodeSampleScale::Dataflow_Transfer_Asset_Edge || SampleScale == EDataflowTransferNodeSampleScale::Dataflow_Transfer_Component_Edge", EditConditionHides))
	float EdgeMultiplier = 0.5f;

	/* Max bound multiplier for the Bounding Volume Hierarchy(BVH) target's particle search radius. */
	UPROPERTY(EditAnywhere, Category = "Thresholds", meta = (
		EditCondition = "SampleScale == EDataflowTransferNodeSampleScale::Dataflow_Transfer_Asset_Bound", 
		EditConditionHides))
	float BoundMultiplier = 0.01f;

	FGeometryCollectionTransferVertexScalarAttributeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&FromCollection);
		RegisterInputConnection(&AttributeKey);

		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&AttributeKey, &AttributeKey);
	}

private:

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;


	//
	// private helpers
	//
	TArray<FIntVector2> FindSourceToTargetGeometryMap(const FManagedArrayCollection& SourceCollection, const FManagedArrayCollection& TargetCollection) const;
	void PairedGeometryTransfer(FCollectionAttributeKey Key, const TArray<FIntVector2>& PairedGeometry,const UE::Private::FTransferFacade& Sample, UE::Private::FTransferFacade& Target, TManagedArray<float>* TargetFloatArray) const;
	void NearestVertexTransfer(FCollectionAttributeKey Key, const UE::Private::FTransferFacade& Sample, UE::Private::FTransferFacade& Target, TManagedArray<float>* TargetFloatArray) const;
	static float MaxEdgeLength(TArray<FVector>& Vert, const TManagedArray<FIntVector3>& Tri, int VertexOFfset, int TriStart, int TriCount);
	static void BuildComponentSpaceVertices(const TManagedArray<FTransform3f>* LocalSpaceTransform, const TManagedArray<int32>* Parent, const TManagedArray<int32>* BoneMapArray, const TManagedArray<FVector3f>* VertexArray, int32 Start, int32 Count, TArray<FVector>& ComponentSpaceVertices);
	static UE::Private::BVH* BuildParticleSphereBVH(const TArray<FVector>& Vertices, float Radius);
	static bool TriangleHasWeightsToTransfer(const FIntVector3& T, const TManagedArray<float>& F);
	static void TriangleToVertexIntersections(UE::Private::BVH& VertexBVH, const TArray<FVector>& ComponentSpaceVertices, const FIntVector3& Triangle, TArray<int32>& OutTargetVertexIntersection);
	static float CalculateFalloffScale(EDataflowTransferNodeFalloff FalloffSetting, float Threshold, float Distance);

};


