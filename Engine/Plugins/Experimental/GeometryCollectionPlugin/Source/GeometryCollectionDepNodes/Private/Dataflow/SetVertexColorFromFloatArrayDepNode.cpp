// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SetVertexColorFromFloatArrayDepNode.h"
#include "Dataflow/DataflowCore.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(SetVertexColorFromFloatArrayDepNode)

void FSetVertexColorInCollectionFromFloatArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TArray<float>& InFloatArray = GetValue<TArray<float>>(Context, &FloatArray);

		const int32 NumVertices = InCollection.NumElements(FGeometryCollection::VerticesGroup);

		if (InFloatArray.Num() == NumVertices)
		{
			if (TManagedArray<FLinearColor>* VertexColors = InCollection.FindAttribute<FLinearColor>("Color", FGeometryCollection::VerticesGroup))
			{
				for (int32 Idx = 0; Idx < NumVertices; ++Idx)
				{
					(*VertexColors)[Idx] = FLinearColor(Scale * InFloatArray[Idx], Scale * InFloatArray[Idx], Scale * InFloatArray[Idx]);
				}
			}
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}