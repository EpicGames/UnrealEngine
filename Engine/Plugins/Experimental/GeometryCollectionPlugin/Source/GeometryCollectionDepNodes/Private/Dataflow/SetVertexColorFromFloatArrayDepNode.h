// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"


#include "SetVertexColorFromFloatArrayDepNode.generated.h"

/** Set collection color based on input float array */
USTRUCT(meta = (Deprecated = "5.5"))
struct FSetVertexColorInCollectionFromFloatArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
		DATAFLOW_NODE_DEFINE_INTERNAL(FSetVertexColorInCollectionFromFloatArrayDataflowNode, "SetVertexColorInCollectionFromFloatArray", "Collection|Utilities", "")

public:
	/** Collection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
		FManagedArrayCollection Collection;

	/**  */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
		TArray<float> FloatArray;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Color")
		float Scale = 1.f;

	FSetVertexColorInCollectionFromFloatArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&FloatArray);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};