// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SetVertexColorFromVertexSelectionDepNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SetVertexColorFromVertexSelectionDepNode)

void FSetVertexColorInCollectionFromVertexSelectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const FDataflowVertexSelection& InVertexSelection = GetValue<FDataflowVertexSelection>(Context, &VertexSelection);

		if (InCollection.NumElements(FGeometryCollection::VerticesGroup) == InVertexSelection.Num())
		{
			const int32 NumVertices = InCollection.NumElements(FGeometryCollection::VerticesGroup);

			//			TManagedArray<FLinearColor>& VertexColors = InCollection.ModifyAttribute<FLinearColor>("Color", FGeometryCollection::VerticesGroup);
			if (TManagedArray<FLinearColor>* VertexColors = InCollection.FindAttribute<FLinearColor>("Color", FGeometryCollection::VerticesGroup))
			{
				for (int32 Idx = 0; Idx < NumVertices; ++Idx)
				{
					if (InVertexSelection.IsSelected(Idx))
					{
						(*VertexColors)[Idx] = SelectedColor;
					}
					else
					{
						(*VertexColors)[Idx] = NonSelectedColor;
					}
				}
			}
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}
