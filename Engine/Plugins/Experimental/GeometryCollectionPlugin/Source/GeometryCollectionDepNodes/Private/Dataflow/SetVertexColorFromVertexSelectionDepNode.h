// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowSelection.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "SetVertexColorFromVertexSelectionDepNode.generated.h"

/** Set the collections vertex color from the selection set. */
USTRUCT(meta = (Deprecated = "5.5"))
struct FSetVertexColorInCollectionFromVertexSelectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetVertexColorInCollectionFromVertexSelectionDataflowNode, "SetVertexColorInCollectionFromVertexSelection", "Collection|Utilities", "")

public:
	/** Collection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
		FManagedArrayCollection Collection;

	/**  */
	UPROPERTY(meta = (DataflowInput, DisplayName = "VertexSelection", DataflowIntrinsic))
		FDataflowVertexSelection VertexSelection;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Color")
		FLinearColor SelectedColor = FLinearColor(FColor::Yellow);

	/**  */
	UPROPERTY(EditAnywhere, Category = "Color", meta = (DisplayName = "NonSelected Color"))
		FLinearColor NonSelectedColor = FLinearColor(FColor::Blue);

	FSetVertexColorInCollectionFromVertexSelectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&VertexSelection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};
