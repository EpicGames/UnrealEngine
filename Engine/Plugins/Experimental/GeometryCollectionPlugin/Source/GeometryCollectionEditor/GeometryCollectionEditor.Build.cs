// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GeometryCollectionEditor : ModuleRules
	{
        public GeometryCollectionEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition",
					"Core",
					"CoreUObject",
				    "Slate",
				    "SlateCore",
				    "Engine",
					"EditorFramework",
					"UnrealEd",
				    "PropertyEditor",
				    "RenderCore",
				    "RHI",
                    "GeometryCollectionEngine",
                    "RawMesh",
				    "AssetTools",
				    "AssetRegistry",
				    "SceneOutliner",
					"EditorStyle",
					"ToolMenus",
					"Chaos",
					"MeshDescription",
					"StaticMeshDescription",
					"LevelEditor",
					"InputCore",
					"GraphEditor",
					"DataflowCore",
					"DataflowEngine",
					"DataflowEditor",
				}
			);
			
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"ContentBrowser",
					"EditorScriptingUtilities",
					"DynamicMesh",
					"FractureEngine",
					"GeometryCore",
					"GeometryFramework",
					"MeshModelingTools",
					"ModelingComponentsEditorOnly",
					"ModelingComponents",
					"ModelingToolsEditorMode",
					"GeometryCollectionNodes",
					"DataflowMedialSkeleton"
				}
			);

			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");
		}
	}
}
