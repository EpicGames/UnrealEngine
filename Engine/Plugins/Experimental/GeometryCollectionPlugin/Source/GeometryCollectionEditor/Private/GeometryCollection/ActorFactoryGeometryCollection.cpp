// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/ActorFactoryGeometryCollection.h"
#include "GeometryCollection/GeometryCollectionActor.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionFactory.h"
#include "Misc/FileHelper.h"
#include "ActorFactories/ActorFactory.h"
#include "AssetRegistry/AssetData.h"
#include "Editor.h"
#include "EditorFramework/AssetImportData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorFactoryGeometryCollection)

#define LOCTEXT_NAMESPACE "ActorFactoryGeometryCollection"

DEFINE_LOG_CATEGORY_STATIC(LogGeometryCollectionFactories, Log, All);

/*-----------------------------------------------------------------------------
UActorFactoryGeometryCollection
-----------------------------------------------------------------------------*/
UActorFactoryGeometryCollection::UActorFactoryGeometryCollection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("GeometryCollectionDisplayName", "GeometryCollection");
	NewActorClass = AGeometryCollectionActor::StaticClass();
}

bool UActorFactoryGeometryCollection::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (!AssetData.IsValid() || !AssetData.IsInstanceOf(UGeometryCollection::StaticClass()))
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoGeometryCollectionSpecified", "No GeometryCollection mesh was specified.");
		return false;
	}

	return true;
}

void UActorFactoryGeometryCollection::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	UGeometryCollection* GeometryCollection = CastChecked<UGeometryCollection>(Asset);
	AGeometryCollectionActor* NewGeometryCollectionActor = CastChecked<AGeometryCollectionActor>(NewActor);

	// Term Component
	 NewGeometryCollectionActor->GetGeometryCollectionComponent()->UnregisterComponent();

	// Change properties
	NewGeometryCollectionActor->GetGeometryCollectionComponent()->SetRestCollection(GeometryCollection);

	// Set configured clustering properties.
	NewGeometryCollectionActor->GetGeometryCollectionComponent()->EnableClustering = GeometryCollection->EnableClustering;
	NewGeometryCollectionActor->GetGeometryCollectionComponent()->ClusterGroupIndex = GeometryCollection->ClusterGroupIndex;
	NewGeometryCollectionActor->GetGeometryCollectionComponent()->MaxClusterLevel = GeometryCollection->MaxClusterLevel;
	if (GeometryCollection->PhysicsMaterial == nullptr)
	{
		NewGeometryCollectionActor->GetGeometryCollectionComponent()->SetPhysMaterialOverride(GEngine->DefaultDestructiblePhysMaterial);
	}

	// Init Component
	NewGeometryCollectionActor->GetGeometryCollectionComponent()->RegisterComponent();
}

#undef LOCTEXT_NAMESPACE
