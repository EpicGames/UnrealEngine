// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/AssetDefinition_GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "Dataflow/DataflowEditor.h"
#include "Dataflow/DataflowEditorModule.h"
#include "Dataflow/DataflowEditorToolkit.h"
#include "Dataflow/DataflowSimulationScene.h"
#include "Math/Color.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "Toolkits/SimpleAssetEditor.h"
#include "ContentBrowserMenuContexts.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "AssetToolsModule.h"
#include "EditorAssetLibrary.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "Engine/SkeletalMesh.h"
#include "Animation/Skeleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_GeometryCollection)

#define LOCTEXT_NAMESPACE "AssetActions_GeometryCollection"

bool bCanEditGeometryCollection = true;
FAutoConsoleVariableRef CVarGeometryCollectionIsEditable(TEXT("p.Chaos.GC.IsEditable"), bCanEditGeometryCollection, TEXT("Whether to allow edits of the geometry collection"));

namespace UE::GeometryCollection
{
	struct FColorScheme
	{
		static inline const FLinearColor Asset = FColor(180, 120, 110);
		static inline const FLinearColor NodeHeader = FColor(180, 120, 110);
		static inline const FLinearColor NodeBody = FColor(18, 12, 11, 127);
	};
}

FText UAssetDefinition_GeometryCollection::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_GeometryCollection", "Geometry Collection");
}

TSoftClassPtr<UObject> UAssetDefinition_GeometryCollection::GetAssetClass() const
{
	return UGeometryCollection::StaticClass();
}

FLinearColor UAssetDefinition_GeometryCollection::GetAssetColor() const
{
	return UE::GeometryCollection::FColorScheme::Asset;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_GeometryCollection::GetAssetCategories() const
{
	static const TArray<FAssetCategoryPath> Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Physics, LOCTEXT("GeometryCollection_SubMenu", "Geometry Collection"), ECategoryMenuType::Section)
		};
	return Categories;
}

UThumbnailInfo* UAssetDefinition_GeometryCollection::LoadThumbnailInfo(const FAssetData& InAsset) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAsset.GetAsset(), USceneThumbnailInfo::StaticClass());
}

FAssetOpenSupport UAssetDefinition_GeometryCollection::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	if (bCanEditGeometryCollection)
	{
		return Super::GetAssetOpenSupport(OpenSupportArgs);
	}
	return FAssetOpenSupport(EAssetOpenMethod::View, false);
}


EAssetCommandResult UAssetDefinition_GeometryCollection::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<UGeometryCollection*> GeometryCollectionObjects = OpenArgs.LoadObjects<UGeometryCollection>();

	// For now the geometry collection editor only works on one asset at a time
	ensure(GeometryCollectionObjects.Num() == 0 || GeometryCollectionObjects.Num() == 1);
	if (GeometryCollectionObjects.Num() == 1)
	{
		// Validate the asset
		if (UGeometryCollection* const GeometryCollection = Cast<UGeometryCollection>(GeometryCollectionObjects[0]))
		{
			if (FDataflowEditorToolkit::HasDataflowAsset(GeometryCollection))
			{
				UAssetEditorSubsystem* const AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
				
				UDataflowEditor* const AssetEditor = NewObject<UDataflowEditor>(AssetEditorSubsystem, NAME_None, RF_Transient);

				if (UDataflowSimulationSettings* SimulationSettings = NewObject<UDataflowSimulationSettings>())
				{
					SimulationSettings->bIsSimulationPlayingByDefault = false;
					SimulationSettings->bIsAsyncCachingSupported = false;
					SimulationSettings->bIsAsyncCachingEnabledByDefault = false;
					AssetEditor->AddEditorSettings(SimulationSettings);
				}

				AssetEditor->RegisterToolCategories({"General"});

				const TSubclassOf<AActor> ActorClass = StaticLoadClass(AActor::StaticClass(), nullptr,
					TEXT("/GeometryCollectionPlugin/BP_GeometryCollectionPreview.BP_GeometryCollectionPreview_C"), nullptr, LOAD_None, nullptr);
				AssetEditor->Initialize({ GeometryCollection }, ActorClass);
				return EAssetCommandResult::Handled;
			}

			FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, GeometryCollection);
			return EAssetCommandResult::Handled;

		}
	}
	return EAssetCommandResult::Unhandled;
}

// Menu Extensions
//--------------------------------------------------------------------

namespace MenuExtension_GeometryCollection
{
	template <typename T>
	static T* GetOrCreateAsset(const FString Path, FString Name)
	{
		const FString AssetPath = FPaths::Combine(Path, Name);
		if (UEditorAssetLibrary::DoesAssetExist(AssetPath))
		{
			if (T* ExistingObject = Cast<T>(UEditorAssetLibrary::LoadAsset(AssetPath)))
			{
				return ExistingObject;
			}
		}
		// let's create new one 
		FString UniqueAssetName;
		FString UniquePackageName;
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(AssetPath, TEXT(""), UniquePackageName, UniqueAssetName);

		// Create a package for the asset
		UObject* OuterForAsset = CreatePackage(*UniquePackageName);

		// Create a frame in the package
		T* NewAsset = NewObject<T>(OuterForAsset, T::StaticClass(), *UniqueAssetName, RF_Public | RF_Standalone | RF_Transactional);
		if (NewAsset)
		{
			FAssetRegistryModule::AssetCreated(NewAsset);
			NewAsset->Modify();
		}
		return NewAsset;
	}

	static void ExecuteExportToSkeletalMesh(const FToolMenuContext& InContext)
	{
		const FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");

		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext))
		{
			TArray<UGeometryCollection*> GeometryCollections = Context->LoadSelectedObjects<UGeometryCollection>();
			if (!GeometryCollections.IsEmpty())
			{
				for (const UGeometryCollection* GeometryCollection : GeometryCollections)
				{
					if (GeometryCollection)
					{
						// find path to reference asset
						if (UPackage* AssetOuterPackage = Cast<UPackage>(GeometryCollection->GetOuter()))
						{
							FString AssetPackageName = AssetOuterPackage->GetName();
							FString AssetFolderPath = FPackageName::GetLongPackagePath(AssetPackageName);
							FString AssetBaseName = FPaths::GetBaseFilename(AssetPackageName);
							AssetBaseName.RemoveFromStart(TEXT("GC_"));

							FString SkeletalMeshAssetName = FString::Printf(TEXT("SK_%s"), *AssetBaseName);
							FString SkeletonAssetName = FString::Printf(TEXT("SKEL_%s"), *AssetBaseName);

							USkeletalMesh* SkeletalMesh = GetOrCreateAsset<USkeletalMesh>(AssetFolderPath, SkeletalMeshAssetName);
							USkeleton* Skeleton = GetOrCreateAsset<USkeleton>(AssetFolderPath, SkeletonAssetName);

							if (SkeletalMesh && Skeleton)
							{

								if (GeometryCollection->GetGeometryCollection())
								{
									FGeometryCollectionEngineConversion::ConvertCollectionToSkeletalMesh(*GeometryCollection->GetGeometryCollection(), GeometryCollection->Materials, *SkeletalMesh, *Skeleton);
								}

								SkeletalMesh->PostEditChange();
								Skeleton->PostEditChange();

								SkeletalMesh->MarkPackageDirty();
								Skeleton->MarkPackageDirty();

								FAssetRegistryModule::AssetCreated(SkeletalMesh);
								FAssetRegistryModule::AssetCreated(Skeleton);
							}
						}
					}
				}
			}
		}

	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []()
		{
			UToolMenus::RegisterStartupCallback(
				FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
				{
					FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
					UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UGeometryCollection::StaticClass());
					FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
					{
						const TAttribute<FText> Label = LOCTEXT("ExportToSkeletalMesh_Label", "Export to SkeletalMesh");
						const TAttribute<FText> ToolTip = LOCTEXT("ExportToSkeletalMesh_Tooltip", "Export the geometry collection to a skeletal mesh asset with the same name");
						const TAttribute<FSlateIcon> Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.SkeletalMesh");

						FToolUIAction UIAction;
						UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteExportToSkeletalMesh);
						Section.AddMenuEntry("GeometryCollection_ConvertToSkeletalMesh", Label, ToolTip, Icon, UIAction);
					}
				}));
		});
}

#undef LOCTEXT_NAMESPACE
