// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/DetailCustomizations/GeomComponentCacheCustomization.h"
#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "IPropertyTypeCustomization.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "IDetailChildrenBuilder.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "GeometryCollection/GeometryCollectionCache.h"
#include "PropertyCustomizationHelpers.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionEditorPlugin.h"
#include "EditorFontGlyphs.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "GeomCollectionCacheParamsCustomization"

TSharedRef<IPropertyTypeCustomization> FGeomComponentCacheParametersCustomization::MakeInstance()
{
	return MakeShared<FGeomComponentCacheParametersCustomization>();
}

void FGeomComponentCacheParametersCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	];
}

void FGeomComponentCacheParametersCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TargetCacheHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGeomComponentCacheParameters, TargetCache));

	if(IPropertyHandle* HandlePtr = TargetCacheHandle.Get())
	{
		HandlePtr->MarkHiddenByCustomization();

		UObject* CachePtr = nullptr;
		HandlePtr->GetValue(CachePtr);

		// First add the header notification informing the user of the cache state
		const ISlateStyle* Style = IGeometryCollectionEditorPlugin::GetEditorStyle();
		check(Style);

		auto MessageVisibility = [this]()
		{
			UObject* CacheObject = nullptr;
			TargetCacheHandle->GetValue(CacheObject);

			return CacheObject ? EVisibility::Visible : EVisibility::Collapsed;
		};

		auto GetBorderImage = [this, Style]()
		{
			UObject* CacheObject = nullptr;
			TargetCacheHandle->GetValue(CacheObject);

			const UGeometryCollection* Collection = GetCollection();
			UGeometryCollectionCache* TypedCacheObject = Cast<UGeometryCollectionCache>(CacheObject);

			if(CacheObject && Collection)
			{
				bool bIdsMatch = false;
				bool bStatesMatch = false;

				CheckTagsMatch(Collection, TypedCacheObject, bIdsMatch, bStatesMatch);

				if(!bIdsMatch)
				{
					return Style->GetBrush("GeomCacheCompat.Error");
				}

				if(!bStatesMatch)
				{
					return Style->GetBrush("GeomCacheCompat.Warning");
				}
			}

			return Style->GetBrush("GeomCacheCompat.OK");
		};

		auto GetIcon = [this]()
		{
			UObject* CacheObject = nullptr;
			TargetCacheHandle->GetValue(CacheObject);

			const UGeometryCollection* Collection = GetCollection();
			UGeometryCollectionCache* TypedCacheObject = Cast<UGeometryCollectionCache>(CacheObject);

			if(CacheObject && Collection)
			{
				bool bIdsMatch = false;
				bool bStatesMatch = false;

				CheckTagsMatch(Collection, TypedCacheObject, bIdsMatch, bStatesMatch);

				if(!bIdsMatch || !bStatesMatch)
				{
					return FEditorFontGlyphs::Exclamation_Triangle;
				}
			}

			return FEditorFontGlyphs::Check;
		};

		auto GetMessageText = [this]()
		{
			UObject* CacheObject = nullptr;
			TargetCacheHandle->GetValue(CacheObject);

			const UGeometryCollection* Collection = GetCollection();
			UGeometryCollectionCache* TypedCacheObject = Cast<UGeometryCollectionCache>(CacheObject);

			if(CacheObject && Collection)
			{
				bool bIdsMatch = false;
				bool bStatesMatch = false;

				CheckTagsMatch(Collection, TypedCacheObject, bIdsMatch, bStatesMatch);

				if(!bIdsMatch)
				{
					return LOCTEXT("Message_IdMismatch", "Cache incompatible, not valid for record or playback");
				}

				if(!bStatesMatch)
				{
					return LOCTEXT("Message_StateMismatch", "Cache is stale, valid for record but not playback.");
				}

				// Everything matches
				return LOCTEXT("Message_Ok", "Cache valid for playback and record");
			}
			
			return FText::GetEmpty();
		};

		auto IsEnabled = [this]() -> bool
		{
			return (GetCollection() != nullptr);
		};

		ChildBuilder.AddCustomRow(LOCTEXT("TargetCache_Info", "Target Cache"))
		.WholeRowContent()
		[
			SNew(SBorder)
			.Padding(6.0f)
			.BorderImage_Lambda(GetBorderImage)
			.Visibility_Lambda(MessageVisibility)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(STextBlock)
					.TextStyle(*Style, "GeomCacheCompat.Font")
					.Font(FAppStyle::GetFontStyle("FontAwesome.10"))
					.Text_Lambda(GetIcon)
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.FillWidth(1.0f)
				[
					SNew(STextBlock)
					.Text_Lambda(GetMessageText)
					.TextStyle(*Style, "GeomCacheCompat.Font")
				]
			]
		];

		// Now add the cache selector
		ChildBuilder.AddCustomRow(LOCTEXT("TargetCache_RowName", "Target Cache"))
		.NameContent()
		[
			HandlePtr->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SObjectPropertyEntryBox)
			.PropertyHandle(TargetCacheHandle)
			.AllowedClass(UGeometryCollectionCache::StaticClass())
			.ThumbnailPool(CustomizationUtils.GetThumbnailPool())
			.OnShouldFilterAsset(FOnShouldFilterAsset::CreateRaw(this, &FGeomComponentCacheParametersCustomization::ShouldFilterAsset))
			.DisplayUseSelected(true)
			.DisplayBrowse(true)
			.IsEnabled_Lambda(IsEnabled)
		];
	}

	uint32 NumChildren = 0;
	FPropertyAccess::Result AccessResult = PropertyHandle->GetNumChildren(NumChildren);

	if(AccessResult != FPropertyAccess::Fail && NumChildren > 0)
	{
		for(uint32 Index = 0; Index < NumChildren; ++Index)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(Index);

			if(!ChildHandle->IsCustomized())
			{
				ChildBuilder.AddProperty(ChildHandle.ToSharedRef());
			}
		}
	}
}

void FGeomComponentCacheParametersCustomization::AddReferencedObjects(FReferenceCollector& Collector)
{
	auto* Collection = GetCollectionRef();
	if(Collection && *Collection)
	{
		Collector.AddReferencedObject(*Collection);
	}
}

const UGeometryCollection* FGeomComponentCacheParametersCustomization::GetCollection() const
{
	auto Ref = GetCollectionRef();
	return Ref ? *Ref : nullptr;
}

TObjectPtr<const UGeometryCollection>* FGeomComponentCacheParametersCustomization::GetCollectionRef() const
{
	if(!TargetCacheHandle.IsValid())
	{
		return nullptr;
	}

	if(IPropertyHandle* HandlePtr = TargetCacheHandle.Get())
	{
		// Find the component in our outer chain and return whichever we find
		TArray<UObject*> Outers;
		HandlePtr->GetOuterObjects(Outers);

		for(UObject* PotentialComponent : Outers)
		{
			if(UGeometryCollectionComponent* AsTyped = Cast<UGeometryCollectionComponent>(PotentialComponent))
			{
				return &AsTyped->RestCollection;
			}
		}
	}

	return nullptr;
}

bool FGeomComponentCacheParametersCustomization::ShouldFilterAsset(const FAssetData& InData) const
{
	FString IdTagValue;
	FString StateTagValue;
	InData.GetTagValue(UGeometryCollectionCache::TagName_IdGuid, IdTagValue);
	InData.GetTagValue(UGeometryCollectionCache::TagName_StateGuid, StateTagValue);

	const UGeometryCollection* Collection = GetCollection();

	if(Collection && !IdTagValue.IsEmpty() && !StateTagValue.IsEmpty())
	{
		FGuid IdGuid;
		FGuid StateGuid;
		FGuid::Parse(IdTagValue, IdGuid);
		FGuid::Parse(StateTagValue, StateGuid);

		if(Collection && IdGuid == Collection->GetIdGuid())
		{
			// We have a collection and our IDs match
			return false;
		}
	}

	// Not supported by this component
	return true;
}

void FGeomComponentCacheParametersCustomization::CheckTagsMatch(const UGeometryCollection* InCollection, const UGeometryCollectionCache* InCache, bool& bOutIdsMatch, bool& bOutStatesMatch)
{
	check(InCollection && InCache);

	FAssetData CacheAssetData(InCache);
	FString IdGuidString;

	ensure(CacheAssetData.GetTagValue(UGeometryCollectionCache::TagName_IdGuid, IdGuidString));

	FGuid EmbeddedIdGuid;

	const bool bIdGuidValid = FGuid::Parse(IdGuidString, EmbeddedIdGuid);
	const bool bStateGuidValid = InCache->GetCompatibleStateGuid().IsValid();

	bOutIdsMatch = bIdGuidValid && EmbeddedIdGuid == InCollection->GetIdGuid();
	bOutStatesMatch = bStateGuidValid && InCache->GetCompatibleStateGuid() == InCollection->GetStateGuid();
}

#undef LOCTEXT_NAMESPACE
