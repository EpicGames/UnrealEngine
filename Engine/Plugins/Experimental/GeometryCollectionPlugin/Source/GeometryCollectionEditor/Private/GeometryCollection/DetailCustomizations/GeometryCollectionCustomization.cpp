// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollectionCustomization.h"

#include "GeometryCollection/GeometryCollectionObject.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailGroup.h"
#include "DetailWidgetRow.h"
#include "Async/ParallelFor.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "GeometryCollectionCustomization"

TSharedRef<IDetailCustomization> FGeometryCollectionCustomization::MakeInstance()
{
	return MakeShareable(new FGeometryCollectionCustomization());
}

void FGeometryCollectionCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{

	// Store off the objects that we are editing for analysis in later function calls.
	DetailBuilder.GetObjectsBeingCustomized(ObjectsCustomized);

	IDetailCategoryBuilder& CollisionsCategory = DetailBuilder.EditCategory(TEXT("Collisions"));
	
	IDetailGroup& ManualUpdateGroup = CollisionsCategory.AddGroup("ManualUpdate", LOCTEXT("ManualUpdateLabel", "ManualUpdate"));
	ManualUpdateGroup.HeaderRow()
		[
			SNew(SBorder)
			.Padding(FMargin(3.0f))
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.Padding(FMargin(6.f, 3.f))
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(0.0f, 0.0f, 10.0f, 0.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("UpdateChangesManually", "Update Changes Manually"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Justification(ETextJustify::Right)
					]
					+SHorizontalBox::Slot()
					[
						SNew(SCheckBox)
						.ToolTipText(LOCTEXT("UpdateChangesManually_Tooltip", "Defer collision regeneration until manually triggered."))
						.OnCheckStateChanged(this, &FGeometryCollectionCustomization::OnCheckStateChanged)
					]
				]
				+SVerticalBox::Slot()
				.Padding(FMargin(6.f, 3.f))
				.AutoHeight()
				[
					SNew(SBox)
					.Visibility(this, &FGeometryCollectionCustomization::ApplyButtonVisibility)
					.WidthOverride(100.f)
					.HeightOverride(25.f)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.OnClicked(this, &FGeometryCollectionCustomization::OnApplyChanges)
						.IsEnabled(this, &FGeometryCollectionCustomization::IsApplyNeeded)
						.ToolTipText(LOCTEXT("ApplyChanges_Tooltip", "Regenerate collisions using new parameters."))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ApplyChanges", "Apply Changes"))
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Justification(ETextJustify::Center)
						]
					]
				]
			]		
		];
		
}

bool FGeometryCollectionCustomization::IsApplyNeeded() const
{
	for (int32 i = 0; i < ObjectsCustomized.Num(); i++)
	{
		if (UGeometryCollection* GeometryCollection = Cast<UGeometryCollection>(ObjectsCustomized[i].Get()))
		{
			if (GeometryCollection->IsSimulationDataDirty())
			{
				return true;
			}
		}		
	}
	return false;
}

FReply FGeometryCollectionCustomization::OnApplyChanges()
{
	ParallelFor(ObjectsCustomized.Num(), [&ObjectsCustomized = ObjectsCustomized](int32 Idx)
	{
		if (UGeometryCollection* GeometryCollection = Cast<UGeometryCollection>(ObjectsCustomized[Idx].Get()))
		{
			if (GeometryCollection->IsSimulationDataDirty())
			{
				GeometryCollection->CreateSimulationData();
			}
		}
	});

	return FReply::Handled();
}

void FGeometryCollectionCustomization::OnCheckStateChanged(ECheckBoxState InNewState)
{
	bManualApplyActivated = (InNewState == ECheckBoxState::Checked);

	for (TWeakObjectPtr<UObject> ObjectCustomized : ObjectsCustomized)
	{
		if (UGeometryCollection* GeometryCollection = Cast<UGeometryCollection>(ObjectCustomized.Get()))
		{
			GeometryCollection->bManualDataCreate = bManualApplyActivated;
		}
	}
}

EVisibility FGeometryCollectionCustomization::ApplyButtonVisibility() const
{
	if (bManualApplyActivated)
	{
		return EVisibility::Visible;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}



#undef LOCTEXT_NAMESPACE