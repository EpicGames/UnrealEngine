// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"

enum class ECheckBoxState : uint8;

class FGeometryCollectionCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder);

private:
	bool IsApplyNeeded() const;
	FReply OnApplyChanges();
	void OnCheckStateChanged(ECheckBoxState InNewState);
	EVisibility ApplyButtonVisibility() const;

	bool bManualApplyActivated = false;

	TArray<TWeakObjectPtr<UObject>> ObjectsCustomized;
};
