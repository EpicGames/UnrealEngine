// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/DetailCustomizations/WarningMessageCustomization.h"

#include "PropertyHandle.h"
#include "IPropertyTypeCustomization.h"
#include "IPropertyUtilities.h"
#include "DetailWidgetRow.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "EditorFontGlyphs.h"
#include "GeometryCollection/GeometryCollectionActor.h"
#include "GeometryCollection/GeometryCollectionDebugDrawActor.h"
#include "GeometryCollection/GeometryCollectionDebugDrawComponent.h"

TSharedRef<IPropertyTypeCustomization> FWarningMessageCustomization::MakeInstance()
{
	return MakeShareable(new FWarningMessageCustomization);
}

void FWarningMessageCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Hide reset-to-default
	StructPropertyHandle->MarkResetToDefaultCustomized();

	// Check whether the selected actor has begun play
	bool bHasBegunPlay = false;
	for (const TWeakObjectPtr<UObject>& SelectedObject: StructCustomizationUtils.GetPropertyUtilities()->GetSelectedObjects())
	{
		if (AGeometryCollectionActor* const Actor = Cast<AGeometryCollectionActor>(SelectedObject.Get()))
		{
			bHasBegunPlay = Actor->HasActorBegunPlay();
			break;
		}
	}

	if (!bHasBegunPlay)
	{
		static const FText ToolTipMessage = NSLOCTEXT("GeometryCollectionDebugDrawWarningMessage", "WarningMessage_ToolTip", "The actor must be playing for these debug draw properties to have any effect.");
		static const FText Message = NSLOCTEXT("GeometryCollectionDebugDrawWarningMessage", "WarningMessage_MessageWaiting", "Waiting for Play");
		static const FSlateColor Color = FAppStyle::GetSlateColor("SelectionColor");

		// Add buttons
		HeaderRow.ValueContent()
		.MinDesiredWidth(140.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.ToolTipText(ToolTipMessage)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
					.Text(FEditorFontGlyphs::Exclamation_Triangle)
					.ColorAndOpacity(Color)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.f, 0.f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.ToolTipText(ToolTipMessage)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Font(FCoreStyle::GetDefaultFontStyle("Italic", 9) )
					.Text(Message)
					.ColorAndOpacity(Color)
				]
			]
		];
	}
}

void FWarningMessageCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> /*StructPropertyHandle*/, IDetailChildrenBuilder& /*ChildBuilder*/, IPropertyTypeCustomizationUtils& /*StructCustomizationUtils*/)
{
}
