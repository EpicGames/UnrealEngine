// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionAssetBroker.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionObject.h"


UClass* FGeometryCollectionAssetBroker::GetSupportedAssetClass()
{
	return UGeometryCollection::StaticClass();
}

bool FGeometryCollectionAssetBroker::AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset)
{
	if (UGeometryCollectionComponent* GeometryCollectionComponent = Cast<UGeometryCollectionComponent>(InComponent))
	{
		UGeometryCollection* GeomCollection = Cast<UGeometryCollection>(InAsset);

		if ((GeomCollection != nullptr) || (InAsset == nullptr))
		{
			GeometryCollectionComponent->SetRestCollection(GeomCollection);

			return true;
		}
	}

	return false;
}

UObject* FGeometryCollectionAssetBroker::GetAssetFromComponent(UActorComponent* InComponent)
{
	if (UGeometryCollectionComponent* GeometryCollectionComponent = Cast<UGeometryCollectionComponent>(InComponent))
	{
		return (UObject*)GeometryCollectionComponent->GetRestCollection();
	}
	return nullptr;
}
