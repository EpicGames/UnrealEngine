// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionCacheFactory.h"
#include "GeometryCollection/GeometryCollectionCache.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include "Widgets/SWindow.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "Editor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionCacheFactory)

#define LOCTEXT_NAMESPACE "GeomCollectionCacheFactory"

UGeometryCollectionCacheFactory::UGeometryCollectionCacheFactory()
{
	SupportedClass = UGeometryCollectionCache::StaticClass();
}

bool UGeometryCollectionCacheFactory::CanCreateNew() const
{
	return true;
}

bool UGeometryCollectionCacheFactory::FactoryCanImport(const FString& Filename)
{
	return false;
}

bool UGeometryCollectionCacheFactory::ShouldShowInNewMenu() const
{
	return true;
}

UObject* UGeometryCollectionCacheFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if(TargetCollection)
	{
		UGeometryCollectionCache* NewCache = NewObject<UGeometryCollectionCache>(InParent, InClass, InName, Flags);

		return NewCache;
	}

	return nullptr;
}

bool UGeometryCollectionCacheFactory::ConfigureProperties()
{
	TargetCollection = nullptr;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	
	// Config for picker list of geometry collections
	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(UGeometryCollection::StaticClass()->GetClassPathName());
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateUObject(this, &UGeometryCollectionCacheFactory::OnConfigSelection);
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

	// Create and show the window
	PickerWindow = SNew(SWindow)
		.Title(LOCTEXT("CreateCacheOptions", "Pick Target Geometry Collection"))
		.ClientSize(FVector2D(500, 600))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			]
		];

	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
	PickerWindow.Reset();

	return !!TargetCollection;
}

void UGeometryCollectionCacheFactory::OnConfigSelection(const FAssetData& InSelectedAssetData)
{
	TargetCollection = Cast<UGeometryCollection>(InSelectedAssetData.GetAsset());
	PickerWindow->RequestDestroyWindow();
}

#undef LOCTEXT_NAMESPACE

