// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionCommands.h"

#include "Engine/Selection.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionActor.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/GeometryCollectionEngineUtility.h"
#include "Internationalization/Regex.h"

#include "Logging/LogMacros.h"
#include "Editor.h"
#include "EditorSupportDelegates.h"
#include "SceneOutlinerDelegates.h"


DEFINE_LOG_CATEGORY_STATIC(UGeometryCollectionCommandsLogging, NoLogging, All);

void FGeometryCollectionCommands::ToString(UWorld * World)
{
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
			{
				const UGeometryCollection* RestCollection = Actor->GetGeometryCollectionComponent()->GetRestCollection();
				GeometryCollectionAlgo::PrintParentHierarchy(RestCollection->GetGeometryCollection().Get());
			}
		}
	}
}

void FGeometryCollectionCommands::WriteToHeaderFile(const TArray<FString>& Args, UWorld * World)
{
	if (Args.Num() > 1)
	{
		if (USelection* SelectedActors = GEditor->GetSelectedActors())
		{
			for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
			{
				if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
				{
					FString Name = *Args[0];
					ensure(!Name.IsEmpty());
					FString Path = *Args[1];
					if (Path.IsEmpty())
					{
						Path = "";
					}
					UE_LOGF(UGeometryCollectionCommandsLogging, Log, "... %ls %ls", *Name, *Path);

					const UGeometryCollection* RestCollection = Actor->GetGeometryCollectionComponent()->GetRestCollection();
					RestCollection->GetGeometryCollection().Get()->WriteDataToHeaderFile(Name, Path);

					return;
				}
			}
		}
	}
}

void FGeometryCollectionCommands::WriteToOBJFile(const TArray<FString>& Args, UWorld * World)
{
	if (Args.Num() > 1)
	{
		if (USelection* SelectedActors = GEditor->GetSelectedActors())
		{
			for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
			{
				if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
				{
					FString Name = *Args[0];
					ensure(!Name.IsEmpty());
					FString Path = *Args[1];
					if (Path.IsEmpty())
					{
						Path = "";
					}
					UE_LOGF(UGeometryCollectionCommandsLogging, Log, "... %ls %ls", *Name, *Path);

					const UGeometryCollection* RestCollection = Actor->GetGeometryCollectionComponent()->GetRestCollection();
					RestCollection->GetGeometryCollection().Get()->WriteDataToOBJFile(Name, Path);

					return;
				}
			}
		}
	}
}

void FGeometryCollectionCommands::PrintStatistics(UWorld * World)
{
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
			{
				const UGeometryCollection* RestCollection = Actor->GetGeometryCollectionComponent()->GetRestCollection();
				const FGeometryCollection* GeometryCollection = RestCollection->GetGeometryCollection().Get();

				GeometryCollectionAlgo::PrintStatistics(GeometryCollection);
				return;
			}
		}
	}
}

void FGeometryCollectionCommands::PrintDetailedStatistics(UWorld * World)
{
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
			{
				const UGeometryCollection* RestCollection = Actor->GetGeometryCollectionComponent()->GetRestCollection();
				const FGeometryCollection* GeometryCollection = RestCollection->GetGeometryCollection().Get();
				const UGeometryCollectionCache* Cache = Actor->GetGeometryCollectionComponent()->CacheParameters.TargetCache;

				GeometryCollectionEngineUtility::PrintDetailedStatistics(GeometryCollection, Cache);
				return;
			}
		}
	}
}

void FGeometryCollectionCommands::PrintDetailedStatisticsSummary(UWorld * World)
{
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		TArray<const FGeometryCollection*> GeometryCollectionArray;

		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
			{
				const UGeometryCollection* RestCollection = Actor->GetGeometryCollectionComponent()->GetRestCollection();
				const FGeometryCollection* GeometryCollection = RestCollection->GetGeometryCollection().Get();

				if (GeometryCollection != nullptr)
				{
					GeometryCollectionArray.Add(GeometryCollection);
				}
			}
		}

		GeometryCollectionEngineUtility::PrintDetailedStatisticsSummary(GeometryCollectionArray);
	}
}

void FGeometryCollectionCommands::DeleteCoincidentVertices(const TArray<FString>& Args, UWorld * World)
{
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
			{
				float Tol = 1e-2f;
				if (Args.Num() > 0)
					Tol = FCString::Atof(*Args[0]);
				ensure(Tol > 0.f);
				UE_LOGF(UGeometryCollectionCommandsLogging, Log, "... %f", Tol);

				const UGeometryCollection* RestCollection = Actor->GetGeometryCollectionComponent()->GetRestCollection();
				GeometryCollectionAlgo::DeleteCoincidentVertices(RestCollection->GetGeometryCollection().Get(), Tol);
			}
		}
	}
}



enum SupportedAttributeTypes
{
	UNKNOWN_ATTR_TYPE = 0,
	BOOL_ATTR_TYPE = 1,
	INT_ATTR_TYPE = 2,
	FLOAT_ATTR_TYPE = 3
};

SupportedAttributeTypes ParseSupportedAttributeTypes(FString TypeStr)
{
	if (TypeStr.Equals("bool"))
	{
		return SupportedAttributeTypes::BOOL_ATTR_TYPE;
	}
	else if (TypeStr.Equals("int"))
	{
		return SupportedAttributeTypes::INT_ATTR_TYPE;
	}
	else if (TypeStr.Equals("float"))
	{
		return SupportedAttributeTypes::FLOAT_ATTR_TYPE;
	}
	return SupportedAttributeTypes::UNKNOWN_ATTR_TYPE;
}

void FGeometryCollectionCommands::SetNamedAttributeValues(const TArray<FString>& Args, UWorld* World)
{
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
			{
				FString InType = "";
				if (Args.Num() > 0)
					InType = *Args[0];

				FName InAttributeName = "";
				if (Args.Num() > 1)
					InAttributeName = *Args[1];

				FName InGroupName = "";
				if (Args.Num() > 2)
					InGroupName = *Args[2];

				if (Args.Num() > 3)
				{

					FName InPatternName = "";
					if (Args.Num() > 4)
						InPatternName = *Args[4];

					FString InPattern = "*";
					if (Args.Num() > 5)
						InPattern = *Args[5];

					FGeometryCollectionEdit RestCollectionEdit = Actor->GetGeometryCollectionComponent()->EditRestCollection();

					if (UGeometryCollection* RestCollectionObject = RestCollectionEdit.GetRestCollection())
					{
						if (TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe>  RestCollection = RestCollectionObject->GetGeometryCollection())
						{
							SupportedAttributeTypes ResolvedType = ParseSupportedAttributeTypes(InType);
							if (ResolvedType == SupportedAttributeTypes::BOOL_ATTR_TYPE)
							{
								if (RestCollection->FindAttribute<bool>(InAttributeName, InGroupName))
								{

									bool InValue = false;
									InValue = bool(FCString::Atoi(*Args[3]));


									const FRegexPattern RegexPattern(InPattern);
									TManagedArray<bool>& AttributeArray = RestCollection->ModifyAttribute<bool>(InAttributeName, InGroupName);

									if (InPatternName.GetStringLength())
									{
										if (RestCollection->HasAttribute(InPatternName, InGroupName))
										{
											if (RestCollection->FindAttribute<FString>(InPatternName, InGroupName))
											{
												const TManagedArray<FString>& NameArray = RestCollection->GetAttribute<FString>(InPatternName, InGroupName);

												for (int i = 0; i < NameArray.Num(); i++)
												{
													FRegexMatcher RegexMatcher(RegexPattern, NameArray[i]);
													if (RegexMatcher.FindNext())
													{
														AttributeArray[i] = InValue;
													}
												}
											}
										}
									}
									else
									{
										for (int i = 0; i < AttributeArray.Num(); i++)
											AttributeArray[i] = InValue;
									}
								}
							}
							else if (ResolvedType == SupportedAttributeTypes::INT_ATTR_TYPE)
							{
								if (RestCollection->FindAttribute<int32>(InAttributeName, InGroupName))
								{

									int32 InValue = 0;
									InValue = FCString::Atoi(*Args[3]);


									const FRegexPattern RegexPattern(InPattern);
									TManagedArray<int32>& AttributeArray = RestCollection->ModifyAttribute<int32>(InAttributeName, InGroupName);

									if (InPatternName.GetStringLength())
									{
										if (RestCollection->HasAttribute(InPatternName, InGroupName))
										{
											if (RestCollection->FindAttribute<FString>(InPatternName, InGroupName))
											{
												const TManagedArray<FString>& NameArray = RestCollection->GetAttribute<FString>(InPatternName, InGroupName);

												for (int i = 0; i < NameArray.Num(); i++)
												{
													FRegexMatcher RegexMatcher(RegexPattern, NameArray[i]);
													if (RegexMatcher.FindNext())
													{
														AttributeArray[i] = InValue;
													}
												}
											}
										}
									}
									else
									{
										for (int i = 0; i < AttributeArray.Num(); i++)
											AttributeArray[i] = InValue;
									}
								}
							}
							else if (ResolvedType == SupportedAttributeTypes::FLOAT_ATTR_TYPE)
							{
								if (RestCollection->FindAttribute<float>(InAttributeName, InGroupName))
								{

									float InValue = 0;
									InValue = FCString::Atof(*Args[3]);


									const FRegexPattern RegexPattern(InPattern);
									TManagedArray<float>& AttributeArray = RestCollection->ModifyAttribute<float>(InAttributeName, InGroupName);

									if (InPatternName.GetStringLength())
									{
										if (RestCollection->HasAttribute(InPatternName, InGroupName))
										{
											if (RestCollection->FindAttribute<FString>(InPatternName, InGroupName))
											{
												const TManagedArray<FString>& NameArray = RestCollection->GetAttribute<FString>(InPatternName, InGroupName);

												for (int i = 0; i < NameArray.Num(); i++)
												{
													FRegexMatcher RegexMatcher(RegexPattern, NameArray[i]);
													if (RegexMatcher.FindNext())
													{
														AttributeArray[i] = InValue;
													}
												}
											}
										}
									}
									else
									{
										for (int i = 0; i < AttributeArray.Num(); i++)
											AttributeArray[i] = InValue;
									}
								}
							}
							else
							{
								UE_LOGF(UGeometryCollectionCommandsLogging, Log, "Error Unknown Array Type %ls", *InType);
							}
						}
					}
				}
			}
		}
	}
}



void FGeometryCollectionCommands::DeleteZeroAreaFaces(const TArray<FString>& Args, UWorld * World)
{
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
			{
				float Tol = 1e-4f;
				if (Args.Num() > 0)
					Tol = FCString::Atof(*Args[0]);
				ensure(Tol > 0.f);
				UE_LOGF(UGeometryCollectionCommandsLogging, Log, "... %f", Tol);

				const UGeometryCollection* RestCollection = Actor->GetGeometryCollectionComponent()->GetRestCollection();
				GeometryCollectionAlgo::DeleteZeroAreaFaces(RestCollection->GetGeometryCollection().Get(), Tol);
			}
		}
	}
}

void FGeometryCollectionCommands::DeleteHiddenFaces(UWorld * World)
{
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
			{
				const UGeometryCollection* RestCollection = Actor->GetGeometryCollectionComponent()->GetRestCollection();
				GeometryCollectionAlgo::DeleteHiddenFaces(RestCollection->GetGeometryCollection().Get());
			}
		}
	}
}

void FGeometryCollectionCommands::DeleteStaleVertices(UWorld * World)
{
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
			{
				const UGeometryCollection* RestCollection = Actor->GetGeometryCollectionComponent()->GetRestCollection();
				GeometryCollectionAlgo::DeleteStaleVertices(RestCollection->GetGeometryCollection().Get());
			}
		}
	}
}

int32 FGeometryCollectionCommands::EnsureSingleRoot(UGeometryCollection* RestCollection)
{
	if (RestCollection)
	{
		TManagedArray<FTransform3f>& Transform = RestCollection->GetGeometryCollection()->Transform;
		TManagedArray<int32>& Parent = RestCollection->GetGeometryCollection()->Parent;
		int32 NumElements = RestCollection->GetGeometryCollection()->NumElements(FGeometryCollection::TransformGroup);
		if (GeometryCollectionAlgo::HasMultipleRoots(RestCollection->GetGeometryCollection().Get()))
		{
			TArray<int32> RootIndices;
			for (int32 Index = 0; Index < NumElements; Index++)
			{
				if (Parent[Index] == FGeometryCollection::Invalid)
				{
					RootIndices.Add(Index);
				}
			}
			int32 RootIndex = RestCollection->GetGeometryCollection()->AddElements(1, FGeometryCollection::TransformGroup);
			Transform[RootIndex].SetTranslation(FVector3f(GeometryCollectionAlgo::AveragePosition(RestCollection->GetGeometryCollection().Get(), RootIndices)));
			GeometryCollectionAlgo::ParentTransforms(RestCollection->GetGeometryCollection().Get(), RootIndex, RootIndices);
			return RootIndex;
		}
		else
		{
			for (int32 Index = 0; Index < NumElements; Index++)
			{
				if (Parent[Index] == FGeometryCollection::Invalid)
				{
					return Index;
				}
			}
		}
		check(false);
	}
	return -1;
}



void SplitAcrossYZPlaneRecursive(uint32 RootIndex, const FTransform & ParentTransform, UGeometryCollection* Collection)
{
	TSet<uint32> RootIndices;
	TManagedArray<TSet<int32>>& Children = Collection->GetGeometryCollection()->Children;
	TManagedArray<FTransform3f>& Transform = Collection->GetGeometryCollection()->Transform;

	TArray<int32> SelectedBonesA, SelectedBonesB;
	for (auto& ChildIndex : Children[RootIndex])
	{
		if (Children[ChildIndex].Num())
		{
			SplitAcrossYZPlaneRecursive(ChildIndex, ParentTransform, Collection);
		}

		FVector Translation = (FTransform(Transform[ChildIndex])*ParentTransform).GetTranslation();
		UE_LOGF(UGeometryCollectionCommandsLogging, Log, "... [%d] global:(%3.5f,%3.5f,%3.5f) local:(%3.5f,%3.5f,%3.5f)",
			ChildIndex, Translation.X, Translation.Y, Translation.Z, Transform[ChildIndex].GetTranslation().X,
			Transform[ChildIndex].GetTranslation().Y, Transform[ChildIndex].GetTranslation().Z );

		if (Translation.X > 0.f)
		{
			SelectedBonesA.Add(ChildIndex);
		}
		else
		{
			SelectedBonesB.Add(ChildIndex);
		}
	}

	if (SelectedBonesB.Num() && SelectedBonesA.Num())
	{
		int32 BoneAIndex = Collection->GetGeometryCollection()->AddElements(1, FGeometryCollection::TransformGroup);
		GeometryCollectionAlgo::ParentTransform(Collection->GetGeometryCollection().Get(), RootIndex, BoneAIndex);
		Transform[BoneAIndex].SetTranslation(FVector3f(GeometryCollectionAlgo::AveragePosition(Collection->GetGeometryCollection().Get(), SelectedBonesA)));
		GeometryCollectionAlgo::ParentTransforms(Collection->GetGeometryCollection().Get(), BoneAIndex, SelectedBonesA);

		int32 BoneBIndex = Collection->GetGeometryCollection()->AddElements(1, FGeometryCollection::TransformGroup);
		GeometryCollectionAlgo::ParentTransform(Collection->GetGeometryCollection().Get(), RootIndex, BoneBIndex);
		Transform[BoneBIndex].SetTranslation(FVector3f(GeometryCollectionAlgo::AveragePosition(Collection->GetGeometryCollection().Get(), SelectedBonesB)));
		GeometryCollectionAlgo::ParentTransforms(Collection->GetGeometryCollection().Get(), BoneBIndex, SelectedBonesB);
	}
}

void FGeometryCollectionCommands::SplitAcrossYZPlane(UWorld * World)
{
	UE_LOGF(UGeometryCollectionCommandsLogging, Log, "FGeometryCollectionCommands::SplitAcrossXZPlane");
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
			{
				FGeometryCollectionEdit RestCollectionEdit = Actor->GetGeometryCollectionComponent()->EditRestCollection();
				UGeometryCollection* RestCollection = RestCollectionEdit.GetRestCollection();
				ensure(RestCollection);

				FGeometryCollectionCommands::EnsureSingleRoot(RestCollection);

				TManagedArray<int32>& Parent = RestCollection->GetGeometryCollection()->Parent;
				for (int32 Index = 0; Index < Parent.Num(); Index++)
				{
					if (Parent[Index] == FGeometryCollection::Invalid)
					{
						SplitAcrossYZPlaneRecursive(Index, Actor->GetTransform(), RestCollection);
					}
				}

				// post update all dependent actors.
				for (TActorIterator<AGeometryCollectionActor> ActorItr(Actor->GetWorld(), AGeometryCollectionActor::StaticClass()); ActorItr; ++ActorItr)
				{
					if (AGeometryCollectionActor* LocalActor = Cast<AGeometryCollectionActor>(*ActorItr))
					{
						if (LocalActor->GetGeometryCollectionComponent()->GetRestCollection() == RestCollection)
						{
							UE_LOGF(UGeometryCollectionCommandsLogging, Log, "...%ls", *LocalActor->GetActorLabel());
							//LocalActor->GetGeometryCollectionComponent()->ResetDynamicCollection();
						}
					}
				}
			}
		}

	}
}


void FGeometryCollectionCommands::DeleteGeometry(const TArray<FString>& Args, UWorld* World)
{
	UE_LOGF(UGeometryCollectionCommandsLogging, Log, "FGeometryCollectionCommands::SplitAcrossXZPlane");
	if (Args.Num() > 0)
	{
		TArray<FAssetData> SelectedAssets;
		GEditor->GetContentBrowserSelections(SelectedAssets);
		for (const FAssetData& AssetData : SelectedAssets)
		{
			if (AssetData.GetAsset()->IsA<UGeometryCollection>())
			{
				if (UGeometryCollection* Collection = static_cast<UGeometryCollection *>(AssetData.GetAsset()))
				{
					int32 Arg = FCString::Atoi(*Args[0]);
					for (int32 i = 0; i < Args.Num(); i++)
					{
						FString EntryName(Args[i]);
						UE_LOGF(UGeometryCollectionCommandsLogging, Log, "... %ls", *EntryName);

						int32 IndexToRemove = Collection->GetGeometryCollection()->BoneName.Find(EntryName);
						if (0 <= IndexToRemove)
						{
							Collection->RemoveElements(FGeometryCollection::TransformGroup, { IndexToRemove });

							// @todo(MaterialReindexing) Deleteing the materials for now, until we support reindexing. 
							int32 NumMaterials = Collection->NumElements(FGeometryCollection::MaterialGroup);
							TArray<int32> MaterialIndices;
							GeometryCollectionAlgo::ContiguousArray(MaterialIndices, NumMaterials);
							Collection->RemoveElements(FGeometryCollection::MaterialGroup, MaterialIndices);
						}
					}
				}
			}
		}
	}
}

void FGeometryCollectionCommands::SelectAllGeometry(const TArray<FString>& Args, UWorld* World)
{
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
			{
				FScopedColorEdit ColorEdit = Actor->GetGeometryCollectionComponent()->EditBoneSelection();
				ColorEdit.SelectBones(GeometryCollection::ESelectionMode::AllGeometry);
			}
		}
	}
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();

}

void FGeometryCollectionCommands::SelectNone(const TArray<FString>& Args, UWorld* World)
{
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
			{
				FScopedColorEdit ColorEdit = Actor->GetGeometryCollectionComponent()->EditBoneSelection();
				ColorEdit.SelectBones(GeometryCollection::ESelectionMode::None);
			}
		}
	}
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();

}

void FGeometryCollectionCommands::SelectLessThenVolume(const TArray<FString>& Args, UWorld* World)
{
	float Volume = 2000.0f;
	if (Args.Num() > 0)
	{
		Volume = FCString::Atof(*Args[0]);
	}

	
	int32 SelectedBoneCount = 0;
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
			{
				const UGeometryCollection* RestCollection = Actor->GetGeometryCollectionComponent()->GetRestCollection();
				FGeometryCollection* GeometryCollection = RestCollection->GetGeometryCollection().Get();
				FScopedColorEdit EditBoneColor = Actor->GetGeometryCollectionComponent()->EditBoneSelection();
				TArray<int32> SelectedBones( EditBoneColor.GetSelectedBones() );

				for( int32 BoundingBoxIndex = 0, ni = GeometryCollection->BoundingBox.Num() ; BoundingBoxIndex < ni ; ++BoundingBoxIndex )
				{
					const FBox& BoundingBox = GeometryCollection->BoundingBox[BoundingBoxIndex];

					if (BoundingBox.GetVolume() < Volume)
					{
						int32 BoneIndex = GeometryCollection->TransformIndex[BoundingBoxIndex];
						SelectedBones.AddUnique(BoneIndex);
						++SelectedBoneCount;
					}
				}

				if(SelectedBoneCount > 0)
				{
					EditBoneColor.SetSelectedBones(SelectedBones);
					EditBoneColor.SetHighlightedBones(SelectedBones);

 					FSceneOutlinerDelegates::Get().OnComponentSelectionChanged.Broadcast(Actor->GetGeometryCollectionComponent());
				}
			}
		}
	}
	UE_LOGF(UGeometryCollectionCommandsLogging, Log, "Selected %d Bones", SelectedBoneCount);

	FEditorSupportDelegates::RedrawAllViewports.Broadcast();
}

void FGeometryCollectionCommands::SelectInverseGeometry(const TArray<FString>& Args, UWorld* World)
{
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
			{
				FScopedColorEdit ColorEdit = Actor->GetGeometryCollectionComponent()->EditBoneSelection();
				ColorEdit.SelectBones(GeometryCollection::ESelectionMode::InverseGeometry);
			}
		}
	}
	FEditorSupportDelegates::RedrawAllViewports.Broadcast();

}

void FGeometryCollectionCommands::BuildProximityDatabase(const TArray<FString>& Args, UWorld * World)
{
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
			{
				const UGeometryCollection* RestCollection = Actor->GetGeometryCollectionComponent()->GetRestCollection();

				if (2 <= RestCollection->GetGeometryCollection().Get()->NumElements(FGeometryCollection::GeometryGroup))
				{
					FGeometryCollectionProximityUtility ProximityUtility(RestCollection->GetGeometryCollection().Get());
					ProximityUtility.UpdateProximity();
				}
			}
		}
	}
}

void FGeometryCollectionCommands::SetupNestedBoneAsset(UWorld * World)
{
	TArray<FAssetData> SelectedAssets;
	GEditor->GetContentBrowserSelections(SelectedAssets);
	for (const FAssetData& AssetData : SelectedAssets)
	{
		if (AssetData.GetAsset()->IsA<UGeometryCollection>())
		{
			if (UGeometryCollection* Collection = static_cast<UGeometryCollection *>(AssetData.GetAsset()))
			{
				GeometryCollection::SetupNestedBoneCollection(Collection->GetGeometryCollection().Get());
			}
		}
	}
}

void FGeometryCollectionCommands::SetupTwoClusteredCubesAsset(UWorld * World)
{
	TArray<FAssetData> SelectedAssets;
	GEditor->GetContentBrowserSelections(SelectedAssets);
	for (const FAssetData& AssetData : SelectedAssets)
	{
		if (AssetData.GetAsset()->IsA<UGeometryCollection>())
		{
			if (UGeometryCollection* Collection = static_cast<UGeometryCollection *>(AssetData.GetAsset()))
			{
				GeometryCollection::SetupTwoClusteredCubesCollection(Collection->GetGeometryCollection().Get());
			}
		}
	}
}

void FGeometryCollectionCommands::HealGeometry(UWorld * World)
{
	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AGeometryCollectionActor* Actor = Cast<AGeometryCollectionActor>(*Iter))
			{
				const UGeometryCollection* RestCollection = Actor->GetGeometryCollectionComponent()->GetRestCollection();

				TArray<TArray<TArray<int32>>> BoundaryVertexIndices;
				GeometryCollectionAlgo::FindOpenBoundaries(RestCollection->GetGeometryCollection().Get(), 1e-2f, BoundaryVertexIndices);
				if (BoundaryVertexIndices.Num() > 0)
				{
					GeometryCollectionAlgo::TriangulateBoundaries(RestCollection->GetGeometryCollection().Get(), BoundaryVertexIndices);
				}
			}
		}
	}
}

