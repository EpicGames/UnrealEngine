// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionConversion.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AnimationRuntime.h"
#include "Async/ParallelFor.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Selection.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "GameFramework/Actor.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionActor.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionFactory.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "GeometryCollectionProxyData.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "Logging/LogMacros.h"
#include "Materials/Material.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "MeshDescription.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "Physics/Experimental/ChaosInterfaceUtils.h"


DEFINE_LOG_CATEGORY_STATIC(UGeometryCollectionConversionLogging, Log, All);




void FGeometryCollectionConversion::AppendStaticMesh(const UStaticMesh* StaticMesh, const TArray<UMaterialInterface*>& Materials, const FTransform& StaticMeshTransform, UGeometryCollection* GeometryCollectionObject, bool ReindexMaterials)
{
	FGeometryCollectionEngineConversion::AppendStaticMesh(StaticMesh, Materials, StaticMeshTransform, GeometryCollectionObject, ReindexMaterials);
}

void FGeometryCollectionConversion::AppendStaticMesh(const UStaticMesh* StaticMesh, const UStaticMeshComponent* StaticMeshComponent, const FTransform& StaticMeshTransform, UGeometryCollection* GeometryCollection, bool ReindexMaterials)
{
	FGeometryCollectionEngineConversion::AppendStaticMesh(StaticMesh, StaticMeshComponent, StaticMeshTransform, GeometryCollection, ReindexMaterials);
}

void FGeometryCollectionConversion::AppendSkeletalMesh(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent, const FTransform& SkeletalMeshTransform, UGeometryCollection* GeometryCollection, bool ReindexMaterials)
{
	FGeometryCollectionEngineConversion::AppendSkeletalMesh(SkeletalMesh, SkeletalMeshComponent, SkeletalMeshTransform, GeometryCollection, ReindexMaterials);
}

void FGeometryCollectionConversion::AppendGeometryCollection(const UGeometryCollection* SourceGeometryCollection, const TArray<UMaterialInterface*>& Materials, const FTransform& GeometryCollectionTransform, UGeometryCollection* TargetGeometryCollectionObject, bool ReindexMaterials)
{
	FGeometryCollectionEngineConversion::AppendGeometryCollection(SourceGeometryCollection, Materials, GeometryCollectionTransform, TargetGeometryCollectionObject, ReindexMaterials);
}

void FGeometryCollectionConversion::AppendGeometryCollection(const UGeometryCollection* SourceGeometryCollection, const UGeometryCollectionComponent* GeometryCollectionComponent, const FTransform& GeometryCollectionTransform, UGeometryCollection* TargetGeometryCollectionObject, bool ReindexMaterials)
{
	FGeometryCollectionEngineConversion::AppendGeometryCollection(SourceGeometryCollection, GeometryCollectionComponent, GeometryCollectionTransform, TargetGeometryCollectionObject, ReindexMaterials);
}

struct FGeometryCollectionAsset
{
	UPackage* Package = nullptr;
	UGeometryCollection* GeometryCollection = nullptr;

	bool IsCreated() const
	{
		return (Package != nullptr && GeometryCollection != nullptr);
	}
	
	void Create()
	{
		if (ensure(!IsCreated()))
		{
			FGeometryCollectionAsset NewAsset;
			Package = CreatePackage(TEXT("/Game/GeometryCollectionAsset"));
			const auto GeometryCollectionFactory = NewObject<UGeometryCollectionFactory>();
			GeometryCollection = static_cast<UGeometryCollection*>(
				GeometryCollectionFactory->FactoryCreateNew(UGeometryCollection::StaticClass(),
					NewAsset.Package,
					FName("GeometryCollectionAsset"),
					RF_Standalone | RF_Public,
					nullptr,
					GWarn));
		}
	}

	void Finalize() const 
	{
		if (ensure(IsCreated()))
		{
			FAssetRegistryModule::AssetCreated(GeometryCollection);
			GeometryCollection->MarkPackageDirty();
			Package->SetDirtyFlag(true);
		}
	}
};

void FGeometryCollectionConversion::CreateGeometryCollectionCommand(UWorld * World)
{
	FGeometryCollectionAsset GeometryCollectionAsset;
	GeometryCollectionAsset.Create();
	GeometryCollectionAsset.Finalize();
}

void FGeometryCollectionConversion::CreateFromSelectedActorsCommand(UWorld * World)
{
	//UE_LOGF(UGeometryCollectionConversionLogging, Log, "FGeometryCollectionConversion::CreateCommand()");
	USelection* SelectedActors = GEditor->GetSelectedActors();
	if (!SelectedActors)
	{
		return;
	}

	FGeometryCollectionAsset GeometryCollectionAsset; 
	for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
	{
		if (AActor* Actor = Cast<AActor>(*Iter))
		{
			TArray<UStaticMeshComponent*> StaticMeshComponents;
			Actor->GetComponents(StaticMeshComponents);
			if (StaticMeshComponents.Num() > 0)
			{
				for (int Index = 0; Index < StaticMeshComponents.Num(); Index++)
				{
					if (StaticMeshComponents[Index]->GetStaticMesh())
					{
						if (!GeometryCollectionAsset.IsCreated())
						{
							GeometryCollectionAsset.Create();
						}
						FGeometryCollectionEngineConversion::AppendStaticMesh(
							StaticMeshComponents[Index]->GetStaticMesh(),
							StaticMeshComponents[Index],
							Actor->GetTransform(),
							GeometryCollectionAsset.GeometryCollection);
					}
				}
			}

			TArray<USkeletalMeshComponent*> SkeletalMeshComponents;
			Actor->GetComponents(SkeletalMeshComponents);
			if (SkeletalMeshComponents.Num() > 0)
			{
				for (int Index = 0; Index < SkeletalMeshComponents.Num(); Index++)
				{
					if (SkeletalMeshComponents[Index]->GetSkeletalMeshAsset())
					{
						if (!GeometryCollectionAsset.IsCreated())
						{
							GeometryCollectionAsset.Create();
						}
						FGeometryCollectionEngineConversion::AppendSkeletalMesh(
							SkeletalMeshComponents[Index]->GetSkeletalMeshAsset(),
							SkeletalMeshComponents[Index],
							Actor->GetTransform(),
							GeometryCollectionAsset.GeometryCollection);
					}
				}
			}
		}
	}

	if(GeometryCollectionAsset.IsCreated())
	{
		GeometryCollectionAlgo::PrepareForSimulation(GeometryCollectionAsset.GeometryCollection->GetGeometryCollection().Get());
		GeometryCollectionAsset.Finalize();
	}
}

void FGeometryCollectionConversion::CreateFromSelectedAssetsCommand(UWorld * World)
{
	//UE_LOGF(UGeometryCollectionConversionLogging, Log, "FGeometryCollectionConversion::CreateCommand()");
	FGeometryCollectionAsset GeometryCollectionAsset;
	
	TArray<FAssetData> SelectedAssets;
	GEditor->GetContentBrowserSelections(SelectedAssets);
	for (const FAssetData& AssetData : SelectedAssets)
	{
		if (AssetData.GetAsset()->IsA<UStaticMesh>())
		{
			UE_LOGF(UGeometryCollectionConversionLogging, Log, "Static Mesh Content Browser : %ls", *AssetData.GetClass()->GetName());
			if (!GeometryCollectionAsset.IsCreated())
			{
				GeometryCollectionAsset.Create();
			}
			FGeometryCollectionEngineConversion::AppendStaticMesh(
				static_cast<const UStaticMesh*>(AssetData.GetAsset()),
				nullptr,
				FTransform(), 
				GeometryCollectionAsset.GeometryCollection);
		}
		else if (AssetData.GetAsset()->IsA<USkeletalMesh>())
		{
			UE_LOGF(UGeometryCollectionConversionLogging, Log, "Skeletal Mesh Content Browser : %ls", *AssetData.GetClass()->GetName());
			if (!GeometryCollectionAsset.IsCreated())
			{
				GeometryCollectionAsset.Create();
			}
			FGeometryCollectionEngineConversion::AppendSkeletalMesh(
				static_cast<const USkeletalMesh *>(AssetData.GetAsset()), 
				nullptr,
				FTransform(),
				GeometryCollectionAsset.GeometryCollection);
		}
	}

	if(GeometryCollectionAsset.IsCreated())
	{
		GeometryCollectionAlgo::PrepareForSimulation(GeometryCollectionAsset.GeometryCollection->GetGeometryCollection().Get());
		GeometryCollectionAsset.Finalize();
	}
}
