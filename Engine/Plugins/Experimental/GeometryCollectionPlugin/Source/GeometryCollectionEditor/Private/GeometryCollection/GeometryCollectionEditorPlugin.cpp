// Copyright Epic Games, Inc. All Rights Reserved.


#include "GeometryCollection/GeometryCollectionEditorPlugin.h"

#include "AssetToolsModule.h"
#include "CoreMinimal.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Dataflow/AssetDefinition_DataflowAsset.h"
#include "Dataflow/DataflowTemplateRegistry.h"
#include "GeometryCollectionRendering/DataflowMedialSkeletonRenderableType.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionConversion.h"
#include "GeometryCollection/GeometryCollectionCommands.h"
#include "GeometryCollection/AssetDefinition_GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAssetBroker.h"
#include "GeometryCollection/GeometryCollectionEditorStyle.h"
#include "GeometryCollection/GeometryCollectionThumbnailRenderer.h"
#include "GeometryCollection/GeometryCollectionSelectRigidBodyEdMode.h"
#include "GeometryCollection/GeometryCollectionSelectionCommands.h"
#include "GeometryCollection/GeometryCollectionFactory.h"
#include "HAL/ConsoleManager.h"
#include "Features/IModularFeatures.h"
#include "GeometryCollection/DetailCustomizations/GeomComponentCacheCustomization.h"
#include "GeometryCollection/DetailCustomizations/WarningMessageCustomization.h"
#include "GeometryCollection/DetailCustomizations/GeometryCollectionCustomization.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/CoreStyle.h"
#include "ToolMenus.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/IToolkitHost.h"

TOptional<FGeometryCollectionEditorStyle> FGeometryCollectionEditorStyle::Singleton;

IMPLEMENT_MODULE( IGeometryCollectionEditorPlugin, GeometryCollectionEditor )

#define BOX_BRUSH(StyleSet, RelativePath, ...) FSlateBoxBrush(StyleSet->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)
#define DEFAULT_FONT(...) FCoreStyle::GetDefaultFontStyle(__VA_ARGS__)

void IGeometryCollectionEditorPlugin::StartupModule()
{
	FGeometryCollectionEditorStyle::Get();

	FGeometryCollectionSelectionCommands::Register();

	AssetBroker = new FGeometryCollectionAssetBroker();
	FComponentAssetBrokerage::RegisterBroker(MakeShareable(AssetBroker), UGeometryCollectionComponent::StaticClass(), true, true);

	if (GIsEditor && !IsRunningCommandlet())
	{
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.CreateFromSelectedActors"),
			TEXT("Creates a GeometryCollection from the selected Actors that contain Skeletal and Statict Mesh Components"),
			FConsoleCommandWithWorldDelegate::CreateStatic(&FGeometryCollectionConversion::CreateFromSelectedActorsCommand),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.CreateFromSelectedAssets"),
			TEXT("Creates a GeometryCollection from the selected Skeletal Mesh and Static Mesh Assets"),
			FConsoleCommandWithWorldDelegate::CreateStatic(&FGeometryCollectionConversion::CreateFromSelectedAssetsCommand),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.ToString"),
			TEXT("Dump the contents of the collection to the log file. WARNING: The collection can be very large."),
			FConsoleCommandWithWorldDelegate::CreateStatic(&FGeometryCollectionCommands::ToString),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.ClusterAlongYZPlane"),
			TEXT("Debuigging command to split the unclustered geometry collection along the YZPlane."),
			FConsoleCommandWithWorldDelegate::CreateStatic(&FGeometryCollectionCommands::SplitAcrossYZPlane),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.DeleteGeometry"),
			TEXT("Delete geometry by transform name."),
			FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FGeometryCollectionCommands::DeleteGeometry),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.SelectAllGeometry"),
			TEXT("Select all geometry in hierarchy."),
			FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FGeometryCollectionCommands::SelectAllGeometry),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.SelectNone"),
			TEXT("Deselect all geometry in hierarchy."),
			FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FGeometryCollectionCommands::SelectNone),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.SelectLessThenVolume"),
			TEXT("Select all geometry with a volume less than specified."),
			FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FGeometryCollectionCommands::SelectLessThenVolume),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.SelectInverseGeometry"),
			TEXT("Deselect inverse of currently selected geometry in hierarchy."),
			FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FGeometryCollectionCommands::SelectInverseGeometry),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.WriteToHeaderFile"),
			TEXT("Dump the contents of the collection to a header file. WARNING: The collection can be very large."),
			FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FGeometryCollectionCommands::WriteToHeaderFile),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.WriteToOBJFile"),
			TEXT("Dump the contents of the collection to an OBJ file. WARNING: The collection can be very large."),
			FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FGeometryCollectionCommands::WriteToOBJFile),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.BuildProximityDatabase"),
			TEXT("Build the Proximity information in the GeometryGroup for the selected collection."),
			FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FGeometryCollectionCommands::BuildProximityDatabase),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.DeleteCoincidentVertices"),
			TEXT("Delete coincident vertices on a GeometryCollection. WARNING: The collection can be very large."),
			FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FGeometryCollectionCommands::DeleteCoincidentVertices),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.DeleteZeroAreaFaces"),
			TEXT("Delete zero area faces on a GeometryCollection. WARNING: The collection can be very large."),
			FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FGeometryCollectionCommands::DeleteZeroAreaFaces),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.DeleteHiddenFaces"),
			TEXT("Delete hidden faces on a GeometryCollection. WARNING: The collection can be very large."),
			FConsoleCommandWithWorldDelegate::CreateStatic(&FGeometryCollectionCommands::DeleteHiddenFaces),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.DeleteStaleVertices"),
			TEXT("Delete stale vertices on a GeometryCollection. WARNING: The collection can be very large."),
			FConsoleCommandWithWorldDelegate::CreateStatic(&FGeometryCollectionCommands::DeleteStaleVertices),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.PrintStatistics"),
			TEXT("Prints statistics of the contents of the collection."),
			FConsoleCommandWithWorldDelegate::CreateStatic(&FGeometryCollectionCommands::PrintStatistics),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.PrintDetailedStatistics"),
			TEXT("Prints detailed statistics of the contents of the collection."),
			FConsoleCommandWithWorldDelegate::CreateStatic(&FGeometryCollectionCommands::PrintDetailedStatistics),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.PrintDetailedStatisticsSummary"),
			TEXT("Prints detailed statistics of the contents of the selected collection(s)."),
			FConsoleCommandWithWorldDelegate::CreateStatic(&FGeometryCollectionCommands::PrintDetailedStatisticsSummary),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.SetupNestedBoneAsset"),
			TEXT("Converts the selected GeometryCollectionAsset into a test asset."),
			FConsoleCommandWithWorldDelegate::CreateStatic(&FGeometryCollectionCommands::SetupNestedBoneAsset),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.SetupTwoClusteredCubesAsset"),
			TEXT("Addes two clustered cubes to the selected actor."),
			FConsoleCommandWithWorldDelegate::CreateStatic(&FGeometryCollectionCommands::SetupTwoClusteredCubesAsset),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.Heal"),
			TEXT("Tries to fill holes in go."),
			FConsoleCommandWithWorldDelegate::CreateStatic(&FGeometryCollectionCommands::HealGeometry),
			ECVF_Default
		));
		EditorCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("GeometryCollection.SetNamedAttributeValues"),
			TEXT("Command to set attributes within a named group."),
			FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(&FGeometryCollectionCommands::SetNamedAttributeValues),
			ECVF_Default
		));
	}

	// Map global level editor commands
	FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedRef<FUICommandList> CommandList = LevelEditorModule.GetGlobalLevelEditorActions();
	CommandList->MapAction(
		FGeometryCollectionSelectionCommands::Get().SelectAllGeometry,
		FExecuteAction::CreateLambda([]()
		{
			FGeometryCollectionCommands::SelectAllGeometry(TArray<FString>(), nullptr);
		}));
	CommandList->MapAction(
		FGeometryCollectionSelectionCommands::Get().SelectNone,
		FExecuteAction::CreateLambda([]()
		{
			FGeometryCollectionCommands::SelectNone(TArray<FString>(), nullptr);
		}));
	CommandList->MapAction(
		FGeometryCollectionSelectionCommands::Get().SelectInverseGeometry,
		FExecuteAction::CreateLambda([]()
		{
			FGeometryCollectionCommands::SelectInverseGeometry(TArray<FString>(), nullptr);
		}));

	// Bind our scene outliner provider to the editor
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	ModularFeatures.RegisterModularFeature(ITargetCacheProvider::GetFeatureName(), &TargetCacheProvider);

	// Register type customizations
	FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if(PropertyModule)
	{
		PropertyModule->RegisterCustomPropertyTypeLayout("GeomComponentCacheParameters", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FGeomComponentCacheParametersCustomization::MakeInstance));
		PropertyModule->RegisterCustomPropertyTypeLayout("GeometryCollectionDebugDrawWarningMessage", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWarningMessageCustomization::MakeInstance));

		const FName GeometryCollectionName = UGeometryCollection::StaticClass()->GetFName();
		PropertyModule->RegisterCustomClassLayout(GeometryCollectionName, FOnGetDetailCustomizationInstance::CreateStatic(&FGeometryCollectionCustomization::MakeInstance));
	}

	// Register rigid body selection editor mode
	FEditorModeRegistry::Get().RegisterMode<FGeometryCollectionSelectRigidBodyEdMode>(
		FGeometryCollectionSelectRigidBodyEdMode::EditorModeID,
		NSLOCTEXT("GeometryCollectionSelectRigidBody", "GeometryCollectionSelectRigidBodyEdMode", "Geometry Collection Select Editor Mode"));

	// Style sets
	StyleSet = MakeShared<FSlateStyleSet>(GetEditorStyleName());

	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	StyleSet->Set("GeomCacheCompat.Error", new BOX_BRUSH(StyleSet, "Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(0.728f, 0.0f, 0.0f)));
	StyleSet->Set("GeomCacheCompat.Warning", new BOX_BRUSH(StyleSet, "Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(0.728f, 0.364f, 0.003f)));
	StyleSet->Set("GeomCacheCompat.OK", new BOX_BRUSH(StyleSet, "Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(0.10616f, 0.48777f, 0.10616f)));

	StyleSet->Set("GeomCacheCompat.Font", FTextBlockStyle()
		.SetFont(DEFAULT_FONT("Bold", 10))
		.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f))
		.SetHighlightColor(FLinearColor(1.0f, 1.0f, 1.0f))
		.SetShadowOffset(FVector2D(1, 1))
		.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f)));

	UE::Dataflow::Private::RegisterMedialSkeletonRenderableType();

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());

	UThumbnailManager::Get().RegisterCustomRenderer(UGeometryCollection::StaticClass(), UGeometryCollectionThumbnailRenderer::StaticClass());

	// Register tool menus - delayed until ToolMenus module is loaded
	UToolMenus::RegisterStartupCallback(
		FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &IGeometryCollectionEditorPlugin::RegisterMenus));

	FDataflowTemplateRegistry::Get().RegisterTemplateFolder(UGeometryCollection::StaticClass(),
		{ TEXT("/GeometryCollectionPlugin/DataflowTemplates"), FSlateIcon("GeometryCollectionEditorStyle", "ClassThumbnail.GeometryCollection") });

	DataflowMenusHandle = UE::DataflowAssetDefinitionHelpers::RegisterDataflowAssetMenus(UGeometryCollection::StaticClass());
}


void IGeometryCollectionEditorPlugin::ShutdownModule()
{
	if (UObjectInitialized())
	{
		UE::DataflowAssetDefinitionHelpers::UnregisterDataflowAssetMenus(DataflowMenusHandle);

		UThumbnailManager::Get().UnregisterCustomRenderer(UGeometryCollection::StaticClass());

		FComponentAssetBrokerage::UnregisterBroker(MakeShareable(AssetBroker));

		// Unbind provider from editor
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		ModularFeatures.UnregisterModularFeature(ITargetCacheProvider::GetFeatureName(), &TargetCacheProvider);

		// Unregister type customizations
		FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
		if(PropertyModule)
		{
			PropertyModule->UnregisterCustomPropertyTypeLayout("GeomCollectionCacheParameters");
			PropertyModule->UnregisterCustomPropertyTypeLayout("GeometryCollectionDebugDrawActorSelectedRigidBody");
			PropertyModule->UnregisterCustomPropertyTypeLayout("GeometryCollectionDebugDrawWarningMessage");

			const FName GeometryCollectionName = UGeometryCollection::StaticClass()->GetFName();
			PropertyModule->UnregisterCustomClassLayout(GeometryCollectionName);
		}

		// Unregister rigid body selection editor mode
		FEditorModeRegistry::Get().UnregisterMode(FGeometryCollectionSelectRigidBodyEdMode::EditorModeID);
		
		// Unregister tool menu callback and remove any entries we added
		UToolMenus::UnRegisterStartupCallback(this);
		UToolMenus::UnregisterOwner(this);

		// Unmap global level editor commands
		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		TSharedRef<FUICommandList> CommandList = LevelEditorModule.GetGlobalLevelEditorActions();
		CommandList->UnmapAction(FGeometryCollectionSelectionCommands::Get().SelectAllGeometry);
		CommandList->UnmapAction(FGeometryCollectionSelectionCommands::Get().SelectNone);
		CommandList->UnmapAction(FGeometryCollectionSelectionCommands::Get().SelectInverseGeometry);

		// Unregister commands
		FGeometryCollectionSelectionCommands::Unregister();
	}
}

FName IGeometryCollectionEditorPlugin::GetEditorStyleName()
{
	return TEXT("GeometryCollectionStyle");
}

const ISlateStyle* IGeometryCollectionEditorPlugin::GetEditorStyle()
{
	return FSlateStyleRegistry::FindSlateStyle(GetEditorStyleName());
}

void IGeometryCollectionEditorPlugin::CreateGeometrySelectionMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.AddMenuEntry(FGeometryCollectionSelectionCommands::Get().SelectAllGeometry);
	InMenuBuilder.AddMenuEntry(FGeometryCollectionSelectionCommands::Get().SelectNone);
	InMenuBuilder.AddMenuEntry(FGeometryCollectionSelectionCommands::Get().SelectInverseGeometry);
}

TSharedRef<FExtender> IGeometryCollectionEditorPlugin::OnExtendLevelEditorViewMenu(const TSharedRef<FUICommandList> InCommandList)
{
	TSharedRef<FExtender> Extender(new FExtender());

	Extender->AddMenuExtension(
		"SelectGeometryCollections",
		EExtensionHook::After,
		nullptr,
		FMenuExtensionDelegate::CreateRaw(this, &IGeometryCollectionEditorPlugin::CreateGeometrySelectionMenu)
	);

	return Extender;
}

void IGeometryCollectionEditorPlugin::RegisterMenus()
{
	FToolMenuOwnerScoped OwnerScoped(this);

	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Select.GeometryCollections");

		FToolMenuSection& Section = Menu->AddSection(
			"GeometryCollections",
			NSLOCTEXT("LevelViewportContextMenu", "GeometryCollectionHeading", "Geometry Collection")
		);

		Section.AddMenuEntry(FGeometryCollectionSelectionCommands::Get().SelectAllGeometry);
		Section.AddMenuEntry(FGeometryCollectionSelectionCommands::Get().SelectNone);
		Section.AddMenuEntry(FGeometryCollectionSelectionCommands::Get().SelectInverseGeometry);
	}

	ViewMenuExtender = FLevelEditorModule::FLevelEditorMenuExtender::CreateRaw(
		this, &IGeometryCollectionEditorPlugin::OnExtendLevelEditorViewMenu
	);

	FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	TArray<FLevelEditorModule::FLevelEditorMenuExtender>& MenuExtenders =
		LevelEditor.GetAllLevelEditorToolbarViewMenuExtenders();
	MenuExtenders.Add(ViewMenuExtender);
	ViewMenuExtenderHandle = MenuExtenders.Last().GetHandle();
}
