// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

class FGeometryCollectionEditorStyle final : public FSlateStyleSet
{
public:
	FGeometryCollectionEditorStyle() : FSlateStyleSet("GeometryCollectionEditorStyle")
	{
		const FVector2D Icon16x16(16.f, 16.f);
		const FVector2D Icon64x64(64.f, 64.f);

		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/GeometryCollectionPlugin/Resources"));

		Set("ClassIcon.GeometryCollection", new FSlateImageBrush(RootToContentDir(TEXT("GeometryCollection_16x.png")), Icon16x16));
		Set("ClassThumbnail.GeometryCollection", new FSlateImageBrush(RootToContentDir(TEXT("GeometryCollection_64x.png")), Icon64x64));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	virtual ~FGeometryCollectionEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

public:

	static FGeometryCollectionEditorStyle& Get()
	{
		if (!Singleton.IsSet())
		{
			Singleton.Emplace();
		}
		return Singleton.GetValue();
	}

	static void Destroy()
	{
		Singleton.Reset();
	}

private:
	static TOptional<FGeometryCollectionEditorStyle> Singleton;
};
