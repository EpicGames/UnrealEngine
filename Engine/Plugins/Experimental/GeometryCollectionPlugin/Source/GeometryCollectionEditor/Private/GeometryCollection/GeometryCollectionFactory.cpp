// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionFactory.h"

#include "Dataflow/AssetDefinition_DataflowAsset.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionConversion.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Selection.h"
#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionFactory)

#define LOCTEXT_NAMESPACE "GeometryCollection"

/////////////////////////////////////////////////////
// GeometryCollectionFactory

UGeometryCollectionFactory::UGeometryCollectionFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UGeometryCollection::StaticClass();
}

UGeometryCollection* UGeometryCollectionFactory::StaticFactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return static_cast<UGeometryCollection*>(NewObject<UGeometryCollection>(InParent, Class, Name, Flags | RF_Transactional | RF_Public | RF_Standalone));
}

void AppendActorComponentsRecursive(
	const AActor* Actor,
	TArray< GeometryCollectionStaticMeshConversionTuple >& StaticMeshList,
	TArray< GeometryCollectionSkeletalMeshConversionTuple >& SkeletalMeshList,
	TArray< GeometryCollectionTuple >& GeometryCollectionList)
{
	FTransform ActorTransform = Actor->GetTransform();

	TArray<UStaticMeshComponent*> StaticMeshComponents;
	Actor->GetComponents(StaticMeshComponents);
	if (StaticMeshComponents.Num() > 0)
	{
		for (int Index = 0; Index < StaticMeshComponents.Num(); Index++)
		{
			if (StaticMeshComponents[Index]->GetStaticMesh())
			{
				StaticMeshList.Add(GeometryCollectionStaticMeshConversionTuple(
					StaticMeshComponents[Index]->GetStaticMesh(),
					StaticMeshComponents[Index],
					StaticMeshComponents[Index]->GetComponentTransform()));
			}
		}
	}

	TArray<USkeletalMeshComponent*> SkeletalMeshComponents;
	Actor->GetComponents(SkeletalMeshComponents);
	if (SkeletalMeshComponents.Num() > 0)
	{
		for (int Index = 0; Index < SkeletalMeshComponents.Num(); Index++)
		{
			if (SkeletalMeshComponents[Index]->GetSkeletalMeshAsset())
			{
				SkeletalMeshList.Add(GeometryCollectionSkeletalMeshConversionTuple(
					SkeletalMeshComponents[Index]->GetSkeletalMeshAsset(),
					SkeletalMeshComponents[Index],
					SkeletalMeshComponents[Index]->GetComponentTransform()));
			}
		}
	}

	TArray<UGeometryCollectionComponent*> GeometryCollectionComponents;
	Actor->GetComponents(GeometryCollectionComponents);
	if (GeometryCollectionComponents.Num() > 0)
	{
		for (int Index = 0; Index < GeometryCollectionComponents.Num(); Index++)
		{
			if (GeometryCollectionComponents[Index]->GetRestCollection())
			{
				GeometryCollectionList.Add(GeometryCollectionTuple(
					GeometryCollectionComponents[Index]->GetRestCollection(),
					GeometryCollectionComponents[Index],
					GeometryCollectionComponents[Index]->GetComponentTransform()));
			}
		}
	}

	TArray<UChildActorComponent*> ChildActorComponents;
	Actor->GetComponents(ChildActorComponents);
	for (UChildActorComponent* ChildComponent : ChildActorComponents)
	{
		AActor* ChildActor = ChildComponent->GetChildActor();
		if (ChildActor)
		{
			AppendActorComponentsRecursive(ChildActor, StaticMeshList, SkeletalMeshList, GeometryCollectionList);
		}
	}
}

UObject* UGeometryCollectionFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	FTransform LastTransform = FTransform::Identity;
	TArray< GeometryCollectionStaticMeshConversionTuple > StaticMeshList;
	TArray< GeometryCollectionSkeletalMeshConversionTuple > SkeletalMeshList;
	TArray< GeometryCollectionTuple > GeometryCollectionList;

	TArray<FAssetData> SelectedAssets;
	GEditor->GetContentBrowserSelections(SelectedAssets);

	for (const FAssetData& AssetData : SelectedAssets)
	{
		if (AssetData.GetAsset()->IsA<UStaticMesh>())
		{
			UStaticMeshComponent *DummyValue(NULL);
			StaticMeshList.Add(GeometryCollectionStaticMeshConversionTuple(static_cast<const UStaticMesh *>(AssetData.GetAsset()), DummyValue, FTransform()));
		}
		else if (AssetData.GetAsset()->IsA<USkeletalMesh>())
		{
			USkeletalMeshComponent *DummyValue(NULL);
			SkeletalMeshList.Add(GeometryCollectionSkeletalMeshConversionTuple(static_cast<const USkeletalMesh *>(AssetData.GetAsset()), DummyValue, FTransform()));
		}
		else if (AssetData.GetAsset()->IsA<UGeometryCollection>())
		{
			UGeometryCollectionComponent *DummyValue(NULL);
			GeometryCollectionList.Add(GeometryCollectionTuple(static_cast<const UGeometryCollection *>(AssetData.GetAsset()), DummyValue, FTransform()));
		}

	}

	if (USelection* SelectedActors = GEditor->GetSelectedActors())
	{
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			if (AActor* Actor = Cast<AActor>(*Iter))
			{
				AppendActorComponentsRecursive(Actor, StaticMeshList, SkeletalMeshList, GeometryCollectionList);

				if (SelectedActors->GetBottom<AActor>() == Actor)
				{
					LastTransform = Actor->GetTransform();
					LastTransform.SetScale3D(FVector(1.f, 1.f, 1.f));
				}
			}
		}
	}

	UGeometryCollection* NewGeometryCollection = StaticFactoryCreateNew(Class, InParent, Name, Flags, Context, Warn);

	for (GeometryCollectionStaticMeshConversionTuple & StaticMeshData : StaticMeshList)
	{
		FGeometryCollectionEngineConversion::AppendStaticMesh(StaticMeshData.Get<0>(), StaticMeshData.Get<1>(), StaticMeshData.Get<2>(), NewGeometryCollection, false);
	}

	for (GeometryCollectionSkeletalMeshConversionTuple & SkeletalMeshData : SkeletalMeshList)
	{
		FGeometryCollectionEngineConversion::AppendSkeletalMesh(SkeletalMeshData.Get<0>(), SkeletalMeshData.Get<1>(), SkeletalMeshData.Get<2>(), NewGeometryCollection, false);
	}

	for (GeometryCollectionTuple & GeometryCollectionData : GeometryCollectionList)
	{
		NewGeometryCollection->AppendGeometry(*GeometryCollectionData.Get<0>(), false, GeometryCollectionData.Get<2>());
	}

	// De-duplicate materials and add selection material
	NewGeometryCollection->InitializeMaterials(false);
	GeometryCollectionAlgo::PrepareForSimulation(NewGeometryCollection->GetGeometryCollection().Get());

	// Initial pivot : 
	// Offset everything from the last selected element so the transform will align with the null space. 
	if (TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> Collection = NewGeometryCollection->GetGeometryCollection())
	{
		TManagedArray<int32> & Parent = Collection->Parent;
		TManagedArray<FTransform3f>& Transform = Collection->Transform;

		for(int TransformGroupIndex =0; TransformGroupIndex<Collection->Transform.Num(); TransformGroupIndex++)
		{
			if (Parent[TransformGroupIndex] == FGeometryCollection::Invalid)
			{
				Transform[TransformGroupIndex] = Transform[TransformGroupIndex].GetRelativeTransform(FTransform3f(LastTransform));
			}
		}
	}

	UE::DataflowAssetDefinitionHelpers::SetDataflowFromTemplatePicker(NewGeometryCollection);

	// Assign a default dataflow template if available 
	/*if (const UDataflow* DataflowTemplateObject = LoadObject<UDataflow>(nullptr, TEXT("/GeometryCollectionPlugin/DF_GC_Template_StaticMesh.DF_GC_Template_StaticMesh")))
	{
		if (UDataflow* DataflowEmbeddedAsset = DuplicateObject<UDataflow>(DataflowTemplateObject, NewGeometryCollection, TEXT("EmbeddedDataflow")))
		{
			NewGeometryCollection->GetDataflowInstance().SetDataflowAsset(DataflowEmbeddedAsset);
		}
	}*/

	NewGeometryCollection->Modify();
	return NewGeometryCollection;
}

#undef LOCTEXT_NAMESPACE




