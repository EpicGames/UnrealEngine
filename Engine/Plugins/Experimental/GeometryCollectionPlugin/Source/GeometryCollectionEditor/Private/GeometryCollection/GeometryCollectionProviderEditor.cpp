// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionProviderEditor.h"

#include "GeometryCollection/GeometryCollectionCache.h"
#include "GeometryCollection/GeometryCollectionObject.h"

#include "AssetToolsModule.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"

UGeometryCollectionCache* FTargetCacheProviderEditor::GetCacheForCollection(const UGeometryCollection* InCollection)
{
	// Put it in the same folder as the collection
	UPackage* CollectionPackage = InCollection->GetOutermost();
	check(CollectionPackage);

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	FString PackageName;
	FString AssetName;
	AssetToolsModule.Get().CreateUniqueAssetName(CollectionPackage->GetName(), TEXT("_Cache"), PackageName, AssetName);

	UObject* NewCacheObj = AssetToolsModule.Get().CreateAsset(AssetName, FPackageName::GetLongPackagePath(PackageName), UGeometryCollectionCache::StaticClass(), nullptr);
	return CastChecked<UGeometryCollectionCache>(NewCacheObj);
}

