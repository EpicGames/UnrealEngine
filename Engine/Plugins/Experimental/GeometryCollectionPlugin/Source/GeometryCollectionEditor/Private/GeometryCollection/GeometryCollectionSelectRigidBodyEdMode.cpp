// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "GeometryCollection/GeometryCollectionSelectRigidBodyEdMode.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionActor.h"
#include "GeometryCollection/GeometryCollectionBoneNode.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionDebugDrawActor.h"
#include "GeometryCollection/GeometryCollectionHitProxy.h"
#include "Editor.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "EngineUtils.h"
#include "PropertyHandle.h"

DEFINE_LOG_CATEGORY_STATIC(LogGeometryCollectionSelectRigidBodyEdMode, Log, All);

const FEditorModeID FGeometryCollectionSelectRigidBodyEdMode::EditorModeID(TEXT("Chaos.Select"));
const int32 FGeometryCollectionSelectRigidBodyEdMode::MessageKey = GetTypeHash(EditorModeID);

bool FGeometryCollectionSelectRigidBodyEdMode::CanActivateMode()
{
	auto HasValidEditorLevelViewport = [](const TIndirectArray<FWorldContext>& WorldContexts, TMap<FName, FSlatePlayInEditorInfo>& SlatePlayInEditorMap)
	{
		// Code copied from UEditorEngine::ToggleBetweenPIEandSIE to prevent ensure() triggering in external window PIE
		for (const FWorldContext& WorldContext : WorldContexts)
		{
			if (WorldContext.WorldType == EWorldType::PIE && !WorldContext.RunAsDedicated)
			{
				const FSlatePlayInEditorInfo* const SlatePlayInEditorInfo = SlatePlayInEditorMap.Find(WorldContext.ContextHandle);
				return SlatePlayInEditorInfo ? SlatePlayInEditorInfo->DestinationSlateViewport.IsValid(): false;
			}
		}
		return false;
	};

	return GEditor && GEditor->PlayWorld                                                             // Playing in editor
		&& !IsRunningGame()                                                                          // Not running in -game mode
		&& HasValidEditorLevelViewport(GEditor->GetWorldContexts(), GEditor->SlatePlayInEditorMap);  // Not running in new PIE window
}

void FGeometryCollectionSelectRigidBodyEdMode::ActivateMode(TSharedRef<IPropertyHandle> PropertyHandleId, TSharedRef<IPropertyHandle> PropertyHandleSolver, TFunction<void()> OnEnterMode, TFunction<void()> OnExitMode)
{
	// Make sure we are playing in an editor window
	if (CanActivateMode())
	{
		// Running in PIE?
		if (!GEditor->bIsSimulatingInEditor)
		{
			// Eject PIE
			GEditor->RequestToggleBetweenPIEandSIE();

			// Log/display message
			UE_LOGF(LogGeometryCollectionSelectRigidBodyEdMode, Warning, "Player possession ejected by rigid body picker.");
			if (GEngine)
			{
				GEngine->AddOnScreenDebugMessage(-1, 5.f, FColor::Yellow, TEXT("Player possession ejected by rigid body picker."));
			}
		}

		// Activate editor mode
		GLevelEditorModeTools().ActivateMode(EditorModeID);
		if (FEdMode* const EdMode = GLevelEditorModeTools().GetActiveMode(EditorModeID))
		{
			// Set pointers
			FGeometryCollectionSelectRigidBodyEdMode* const GeometryCollectionSelectRigidBodyEdMode = static_cast<FGeometryCollectionSelectRigidBodyEdMode*>(EdMode);
			GeometryCollectionSelectRigidBodyEdMode->PropertyHandleId = PropertyHandleId;
			GeometryCollectionSelectRigidBodyEdMode->PropertyHandleSolver = PropertyHandleSolver;
			GeometryCollectionSelectRigidBodyEdMode->OnExitMode = OnExitMode;

			// Execute enter mode callback
			if (OnEnterMode)
			{
				OnEnterMode();
			}

			// Log/display message
			UE_LOGF(LogGeometryCollectionSelectRigidBodyEdMode, Log, "Click on the rigid body to select...");
			if (GEngine)
			{
				GEngine->AddOnScreenDebugMessage(MessageKey, 3600.f, FColor::White, TEXT("Click on the rigid body to select..."));
			}
		}
	}
	else
	{
		// Can't pick in non playing editor
		UE_LOGF(LogGeometryCollectionSelectRigidBodyEdMode, Error, "The rigid body picker is only available in play in editor.");
		if (GEngine)
		{
			GEngine->AddOnScreenDebugMessage(-1, 5.f, FColor::Red, TEXT("The rigid body picker is only available in play in editor."));
		}
	}
}

bool FGeometryCollectionSelectRigidBodyEdMode::IsModeActive()
{
	return GLevelEditorModeTools().IsModeActive(EditorModeID);
}

void FGeometryCollectionSelectRigidBodyEdMode::DeactivateMode()
{
	// De-activate editor mode
	if (GLevelEditorModeTools().IsModeActive(EditorModeID))
	{
		GLevelEditorModeTools().DeactivateMode(EditorModeID);

		if (GEngine)
		{
			GEngine->RemoveOnScreenDebugMessage(MessageKey);
		}
	}
}

void FGeometryCollectionSelectRigidBodyEdMode::Tick(FEditorViewportClient* /*ViewportClient*/, float /*DeltaTime*/)
{
	if (!GEditor || !GEditor->PlayWorld || !GEditor->bIsSimulatingInEditor)
	{
		// Left simulating in editor
		bIsHoveringGeometryCollection = false;
		DeactivateMode();
	}
}

bool FGeometryCollectionSelectRigidBodyEdMode::MouseMove(FEditorViewportClient* /*ViewportClient*/, FViewport* Viewport, int32 x, int32 y)
{
#if GEOMETRYCOLLECTION_EDITOR_SELECTION
	// Hover status
	const HHitProxy* const HitProxy = Viewport->GetHitProxy(x, y);
	bIsHoveringGeometryCollection = HitProxy && HitProxy->IsA(HGeometryCollection::StaticGetType());
	UE_CLOGF(bIsHoveringGeometryCollection, LogGeometryCollectionSelectRigidBodyEdMode, VeryVerbose, "Hovering GeometryCollectionActor %ls, Transform Index %d.", *static_cast<const HGeometryCollection*>(HitProxy)->Component->GetOwner()->GetName(), static_cast<const HGeometryCollection*>(HitProxy)->BoneIndex);
#endif // #if GEOMETRYCOLLECTION_EDITOR_SELECTION
	return true;
}

bool FGeometryCollectionSelectRigidBodyEdMode::HandleClick(FEditorViewportClient* /*InViewportClient*/, HHitProxy* HitProxy, const FViewportClick& /*Click*/)
{
	// Deactivate editor mode
	DeactivateMode();
	return true;
}

void FGeometryCollectionSelectRigidBodyEdMode::EnableTransformSelectionMode(bool bEnable)
{
	if (!GEditor || !GEditor->PlayWorld) { return; }

	// Enable each geometry collections' rigid body selection mode
	for (TActorIterator<AGeometryCollectionActor> ActorIterator(GEditor->PlayWorld); ActorIterator; ++ActorIterator)
	{
		UGeometryCollectionComponent* const GeometryCollectionComponent = ActorIterator->GetGeometryCollectionComponent();
		if (GeometryCollectionComponent)
		{
			UE_LOGF(LogGeometryCollectionSelectRigidBodyEdMode, Verbose, "EnableTransformSelectionMode(%d) called for AGeometryCollectionActor %ls.", bEnable, *ActorIterator->GetName());
#if GEOMETRYCOLLECTION_EDITOR_SELECTION
			GeometryCollectionComponent->EnableTransformSelectionMode(bEnable);
#endif // #if GEOMETRYCOLLECTION_EDITOR_SELECTION
		}
	}
}

#endif
