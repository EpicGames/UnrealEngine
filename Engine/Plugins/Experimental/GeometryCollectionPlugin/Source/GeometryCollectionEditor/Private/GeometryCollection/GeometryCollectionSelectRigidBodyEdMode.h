// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "EdMode.h"

class IPropertyHandle;

class FGeometryCollectionSelectRigidBodyEdMode : public FEdMode
{
public:
	static const FEditorModeID EditorModeID;

	/** Check whether this mode can be activated (needs to run in SIE/PIE). */
	static bool CanActivateMode();

	/** Activate this editor mode. */
	static void ActivateMode(TSharedRef<IPropertyHandle> PropertyHandleId, TSharedRef<IPropertyHandle> PropertyHandleSolver, TFunction<void()> OnEnterMode, TFunction<void()> OnExitMode);

	/** Deactivate this editor mode. */
	static void DeactivateMode();

	/** Return whether this editor mode is active. */
	static bool IsModeActive();

	FGeometryCollectionSelectRigidBodyEdMode() : bIsHoveringGeometryCollection(false) {}
	virtual ~FGeometryCollectionSelectRigidBodyEdMode() = default;

	/* FEdMode interface */
	virtual void Enter() override { EnableTransformSelectionMode(true); }
	virtual void Exit() override { EnableTransformSelectionMode(false); PropertyHandleId.Reset(); if (OnExitMode) { OnExitMode(); } }
	virtual bool IsCompatibleWith(FEditorModeID /*OtherModeID*/) const override { return false; }

	virtual bool GetCursor(EMouseCursor::Type& OutCursor) const { OutCursor = bIsHoveringGeometryCollection ? EMouseCursor::EyeDropper: EMouseCursor::SlashedCircle; return true; }

	virtual bool UsesTransformWidget() const override { return false; }
	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode /*CheckMode*/) const override { return false; }

	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;

	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;
	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override;

	virtual bool IsSelectionAllowed(AActor* /*InActor*/, bool /*bInSelection */) const override { return false; }

private:
	void EnableTransformSelectionMode(bool bEnable);

private:
	static const int32 MessageKey;
	TWeakPtr<IPropertyHandle> PropertyHandleId;  // Handle of the property that will get updated with the selected rigid body id
	TWeakPtr<IPropertyHandle> PropertyHandleSolver;  // Handle of the property that will get updated with the selected solver actor
	TFunction<void()> OnExitMode;  // Callback function called when the edit mode is deactivated
	bool bIsHoveringGeometryCollection;
};

#endif
