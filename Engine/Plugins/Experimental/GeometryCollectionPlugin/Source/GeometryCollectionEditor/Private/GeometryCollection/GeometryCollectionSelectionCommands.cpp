// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionSelectionCommands.h"

#define LOCTEXT_NAMESPACE "GeometryCollectionSelectionCommands"

void FGeometryCollectionSelectionCommands::RegisterCommands()
{
	UI_COMMAND(SelectAllGeometry, "Select All Geometry In Hierarchy", "Select all geometry in hierarchy", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectNone, "Deselect All Geometry In Hierarchy", "Deselect all geometry in hierarchy", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SelectInverseGeometry, "Select Inverse Geometry In Hierarchy", "Select inverse geometry in hierarchy", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE