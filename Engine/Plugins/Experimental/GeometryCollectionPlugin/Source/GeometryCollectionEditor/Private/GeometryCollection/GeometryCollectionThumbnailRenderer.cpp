// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionThumbnailRenderer.h"
#include "Misc/App.h"
#include "ShowFlags.h"
#include "SceneView.h"
#include "GeometryCollection/GeometryCollectionThumbnailScene.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "SceneInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionThumbnailRenderer)

UGeometryCollectionThumbnailRenderer::UGeometryCollectionThumbnailRenderer(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	ThumbnailScene = nullptr;
}

void UGeometryCollectionThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	UGeometryCollection* GeometryCollection = Cast<UGeometryCollection>(Object);
	if (IsValid(GeometryCollection))
	{
		if (ThumbnailScene == nullptr)
		{
			ThumbnailScene = new FGeometryCollectionThumbnailScene();
		}

		ThumbnailScene->SetGeometryCollection(GeometryCollection);
		ThumbnailScene->GetScene()->UpdateSpeedTreeWind(0.0);

		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game))
			.SetTime(UThumbnailRenderer::GetTime())
			.SetAdditionalViewFamily(bAdditionalViewFamily));

		ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
		ViewFamily.EngineShowFlags.MotionBlur = 0;
		ViewFamily.EngineShowFlags.LOD = 0;

		RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
		ThumbnailScene->SetGeometryCollection(nullptr);
	}
}

void UGeometryCollectionThumbnailRenderer::BeginDestroy()
{
	if (ThumbnailScene != nullptr)
	{
		delete ThumbnailScene;
		ThumbnailScene = nullptr;
	}

	Super::BeginDestroy();
}

