// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionThumbnailScene.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionActor.h"

FGeometryCollectionThumbnailScene::FGeometryCollectionThumbnailScene()
{
	bForceAllUsedMipsResident = false;

	// Create preview actor
	// checked
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bNoFail = true;
	SpawnInfo.ObjectFlags = RF_Transient;
	PreviewActor = GetWorld()->SpawnActor<AGeometryCollectionActor>(SpawnInfo);

	PreviewActor->GetGeometryCollectionComponent()->SetMobility(EComponentMobility::Movable);
	PreviewActor->SetActorEnableCollision(false);
}

void FGeometryCollectionThumbnailScene::SetGeometryCollection(UGeometryCollection* GeometryCollection)
{
	if (UGeometryCollectionComponent* Component = PreviewActor->GetGeometryCollectionComponent())
	{
		Component->SetRestCollection(GeometryCollection);
		Component->ForceNativeRendering(true);

		if (GeometryCollection)
		{
			FTransform MeshTransform = FTransform::Identity;

			PreviewActor->SetActorLocation(FVector(0, 0, 0), false);
			// use UpdateCachedBounds because UpdateBounds() may not properly refresh the bounds for geometry collection as it will not detect the move
			Component->UpdateCachedBounds();

			// Center the mesh at the world origin then offset to put it on top of the plane
			const FVector::FReal BoundsZOffset = GetBoundsZOffset(Component->Bounds);
			const FVector PlacedLocation = -Component->Bounds.Origin + FVector(0, 0, BoundsZOffset);

			PreviewActor->SetActorLocation(PlacedLocation, false);
			Component->UpdateCachedBounds();
			Component->RecreateRenderState_Concurrent();
		}
	}
}

void FGeometryCollectionThumbnailScene::GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const
{
	check(PreviewActor);
	check(PreviewActor->GetGeometryCollectionComponent());
	check(PreviewActor->GetGeometryCollectionComponent()->GetRestCollection());

	UGeometryCollectionComponent* Component = PreviewActor->GetGeometryCollectionComponent();

	const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;
	// Add extra size to view slightly outside of the sphere to compensate for perspective
	const float HalfMeshSize = Component->Bounds.SphereRadius * 1.15;
	const float BoundsZOffset = GetBoundsZOffset(Component->Bounds);
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);

	USceneThumbnailInfo* ThumbnailInfo = Cast<USceneThumbnailInfo>(PreviewActor->GetGeometryCollectionComponent()->GetRestCollection()->ThumbnailInfo);
	if (ThumbnailInfo)
	{
		if (TargetDistance + ThumbnailInfo->OrbitZoom < 0)
		{
			ThumbnailInfo->OrbitZoom = -TargetDistance;
		}
	}
	else
	{
		ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
	}

	OutOrigin = FVector(0, 0, -BoundsZOffset);
	OutOrbitPitch = ThumbnailInfo->OrbitPitch;
	OutOrbitYaw = ThumbnailInfo->OrbitYaw;
	OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
}
