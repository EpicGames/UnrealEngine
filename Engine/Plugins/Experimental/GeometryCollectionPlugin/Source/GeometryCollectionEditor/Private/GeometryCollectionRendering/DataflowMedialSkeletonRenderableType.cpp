// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollectionRendering/DataflowMedialSkeletonRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "Components/BoxComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Drawing/PointSetComponent.h"
#include "Drawing/LineSetComponent.h"
#include "Dataflow/DataflowRenderingViewMode.h"

#include "Dataflow/DataflowRenderingViewMode.h"
#include "Dataflow/DataflowEngineUtil.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"

#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "Dataflow/DataflowMedialSkeleton.h"
#include "DataflowRendering/DataflowDebugMeshComponent.h"
#include "IndexTypes.h"

#include "Engine/StaticMesh.h"

#include "UObject/ObjectPtr.h"

#include "Materials/MaterialInstanceDynamic.h"

#define LOCTEXT_NAMESPACE "DataflowMedialSkeletonRenderableType"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

UDataflowMedialSkeletonRenderSettings::UDataflowMedialSkeletonRenderSettings(const FObjectInitializer& ObjectInitializer)
{
	constexpr bool bOnlyRGB = true;

	ColorRamp.SetColorAtTime(0.0f, FLinearColor::Blue, bOnlyRGB);
	ColorRamp.SetColorAtTime(0.5f, FLinearColor::Yellow, bOnlyRGB);
	ColorRamp.SetColorAtTime(1.0f, FLinearColor::Red, bOnlyRGB);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Dataflow::Private
{
	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FMedialSkeletonRenderableType : public IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FDataflowMedialSkeleton, MedialSkeleton);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(MedialSkeleton);
		UE_DATAFLOW_IRENDERABLE_VIEW_MODE(FDataflowConstruction3DViewMode);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowMedialSkeletonRenderSettings);

	public:
		FMedialSkeletonRenderableType()
		{
			PointsMaterial = UE::Dataflow::RenderMaterial::GetDataflowPointsMaterial();
			LinesMaterial = UE::Dataflow::RenderMaterial::GetDataflowLinesMaterial();
		}

		virtual void GetPrimitiveComponents(const FRenderableTypeInstance& Instance, FRenderableComponents& OutComponents) const override
		{
			const UDataflowMedialSkeletonRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowMedialSkeletonRenderSettings>();
			if (Settings)
			{
				const FDataflowMedialSkeleton Skeleton = GetMedialSkeleton(Instance, {});

				static FName BaseParentName = TEXT("MedialSkeleton");
				UPrimitiveComponent* ParentComponent = OutComponents.AddNewComponent<UStaticMeshComponent>(BaseParentName);
				if (!ParentComponent)
				{
					return;
				}

				const TArray<FSphere>& SphereArray = Skeleton.Skeleton.Spheres;
				const int32 NumSpheres = SphereArray.Num();

				if (Settings->bShowSpheres)
				{
					if (NumSpheres > 0)
					{
						const FName SphereArrayComponentName = Instance.GetComponentName(TEXT("SphereArray"));

						UStaticMesh* SphereMesh = UE::Dataflow::RenderGeometry::GetDataflowSphere();
						if (SphereMesh)
						{
							UInstancedStaticMeshComponent* ISMComponent = OutComponents.AddNewComponent<UInstancedStaticMeshComponent>(SphereArrayComponentName, ParentComponent);

							if (ISMComponent)
							{
								ISMComponent->PreAllocateInstancesMemory(NumSpheres);
								ISMComponent->SetStaticMesh(SphereMesh);

								ISMComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);

								FBox ComponentBoundingBox;
								for (const FSphere& Sphere : SphereArray)
								{
									ComponentBoundingBox += FBox(Sphere.Center - FVector(Sphere.W), Sphere.Center + FVector(Sphere.W));

									FTransform SphereTransform;
									SphereTransform.SetTranslation(Sphere.Center);
									SphereTransform.SetScale3D(FVector(Sphere.W) / 50.f);

									ISMComponent->AddInstance(SphereTransform);
								}

								ISMComponent->SetNumCustomDataFloats(3); // RGB

								UMaterialInterface* SpheresMaterial = Settings->bWireframe
									? UE::Dataflow::RenderMaterial::GetDataflowColorWireframeMaterial()
									: UE::Dataflow::RenderMaterial::GetDataflowColorMaterial();

								if (SpheresMaterial)
								{
									UMaterialInstanceDynamic* MaterialInstance = UMaterialInstanceDynamic::Create(SpheresMaterial, ISMComponent);
									if (MaterialInstance)
									{
										ISMComponent->SetMaterial(0, MaterialInstance);

										MaterialInstance->SetScalarParameterValue(FName("Transparency"), Settings->Transparency);

										TArray<float> Colors; // NumInstances * 3
										Colors.Init(1.0, NumSpheres * 3);

										double SizeMin = DOUBLE_BIG_NUMBER;
										double SizeMax = -DOUBLE_BIG_NUMBER;

										for (const FSphere& Sphere : SphereArray)
										{
											double Radius = Sphere.W;

											if (Radius > 0.01)
											{
												if (Radius <= SizeMin)
												{
													SizeMin = Radius;
												}
												
												if (Radius >= SizeMax)
												{
													SizeMax = Radius;
												}
											}
										}

										if (SizeMax - SizeMin > UE_SMALL_NUMBER)
										{
											int32 Idx = 0;
											for (const FSphere& Sphere : SphereArray)
											{
												double Radius = Sphere.W;

												FLinearColor Color = Settings->Color;

												if (Settings->SpheresColorMethod == EDataflowMedialSkeletonSpheresColorMethodType::Random)
												{
													Color = UE::Dataflow::Color::GetRandomColor(Settings->RandomSeed, Idx);
												}
												else if (Settings->SpheresColorMethod == EDataflowMedialSkeletonSpheresColorMethodType::BySize)
												{
													double NormalizedValue = (Radius - SizeMin) / (SizeMax - SizeMin);
													Color = Settings->ColorRamp.GetLinearColorValue(NormalizedValue);
												}
												else if (Settings->SpheresColorMethod == EDataflowMedialSkeletonSpheresColorMethodType::ByIDs)
												{
													double NormalizedValue = double(Idx) / double(NumSpheres);
													Color = Settings->ColorRamp.GetLinearColorValue(NormalizedValue);
												}

												Colors[3 * Idx + 0] = Color.R;
												Colors[3 * Idx + 1] = Color.G;
												Colors[3 * Idx + 2] = Color.B;

												Idx++;
											}
										}

										ISMComponent->SetCustomData(0, NumSpheres - 1, Colors);
									}
								}
							}
						}
					}
				}

				UPointSetComponent* PointComponent = nullptr;
				if (Settings->bShowCenters)
				{
					if (NumSpheres > 0)
					{
						TArray<FRenderablePoint> RenderablePoints;
						RenderablePoints.SetNumUninitialized(NumSpheres);

						int32 Idx = 0;
						for (const FSphere& Sphere : SphereArray)
						{
							RenderablePoints[Idx] = FRenderablePoint(SphereArray[Idx].Center, Settings->CentersColor.ToFColor(true), Settings->Size);

							Idx++;
						}

						if (RenderablePoints.Num() > 0)
						{
							const FName PointComponentName = Instance.GetComponentName(TEXT("Centers"));

							PointComponent = OutComponents.AddNewComponent<UPointSetComponent>(PointComponentName, ParentComponent);
							if (PointComponent)
							{
								PointComponent->ReservePoints(NumSpheres);
								PointComponent->AddPoints(RenderablePoints);
								PointComponent->SetPointMaterial(PointsMaterial);

							}
						}
					}
				}

				if (Settings->bShowIDs)
				{
					if (NumSpheres > 0)
					{
						static FName DebugMeshComponentBaseName = TEXT("DebugInfo");

						if (UDataflowDebugMeshComponent* DebugDrawComponent = OutComponents.AddNewComponent<UDataflowDebugMeshComponent>(DebugMeshComponentBaseName, PointComponent))
						{
							DebugDrawComponent->IDs.Reserve(NumSpheres);
							for (int32 SphereIdx = 0; SphereIdx < NumSpheres; ++SphereIdx)
							{
								UDataflowDebugMeshComponent::FText3d Text;
								Text.Color = Settings->IDsColor.ToFColor(true);
								Text.Location = SphereArray[SphereIdx].Center + FVector(1.0, 0.0, -1.0);
								Text.Text = FString::FromInt(SphereIdx);
								DebugDrawComponent->IDs.Add(MoveTemp(Text));
							}
						}
					}
				}

				if (Settings->bShowLines)
				{
					TArray<FVector> LineStarts;
					TArray<FVector> LineEnds;

					for (int32 ClusterIdx = 0; ClusterIdx < Skeleton.Skeleton.ClusterNeighbors.Num(); ++ClusterIdx)
					{
						if (!ensure(Skeleton.Skeleton.Spheres.IsValidIndex(ClusterIdx)))
						{
							continue;
						}
						FVector3d A = Skeleton.Skeleton.Spheres[ClusterIdx].Center;

						for (int32 NbrIdx : Skeleton.Skeleton.ClusterNeighbors[ClusterIdx])
						{
							if (!ensure(Skeleton.Skeleton.Spheres.IsValidIndex(NbrIdx)))
							{
								continue;
							}

							FVector3d B = Skeleton.Skeleton.Spheres[NbrIdx].Center;

							LineStarts.Add(A);
							LineEnds.Add(B);
						}
					}

					const FName LineComponentName = Instance.GetComponentName(TEXT("Connectivity_Lines"));

					if (ULineSetComponent* LineComponent = OutComponents.AddNewComponent<ULineSetComponent>(LineComponentName, ParentComponent))
					{
						LineComponent->AddLines(LineStarts, LineEnds, Settings->LinesColor.ToFColor(true), Settings->LineThickness);
						LineComponent->SetLineMaterial(LinesMaterial);
					}

				}

				if (Settings->bShowTriangles)
				{
					using namespace UE::Geometry;
					TArray<FIndex3i> ClusterTriangles = Skeleton.Skeleton.ClusterTriangles;

					TArray<FVector> LineStarts; LineStarts.Reserve(ClusterTriangles.Num() * 3);
					TArray<FVector> LineEnds; LineEnds.Reserve(ClusterTriangles.Num() * 3);

					for (const FIndex3i& Triangle : ClusterTriangles)
					{
						if (SphereArray.IsValidIndex(Triangle.A) &&
							SphereArray.IsValidIndex(Triangle.B) &&
							SphereArray.IsValidIndex(Triangle.C))
						{
							FVector A = SphereArray[Triangle.A].Center;
							FVector B = SphereArray[Triangle.B].Center;
							FVector C = SphereArray[Triangle.C].Center;

							LineStarts.Add(A); LineEnds.Add(B);
							LineStarts.Add(B); LineEnds.Add(C);
							LineStarts.Add(C); LineEnds.Add(A);
						}
					}

					const FName LineComponentName = Instance.GetComponentName(TEXT("Triangles"));

					if (ULineSetComponent* LineComponent = OutComponents.AddNewComponent<ULineSetComponent>(LineComponentName, ParentComponent))
					{
						LineComponent->AddLines(LineStarts, LineEnds, Settings->TrianglesColor.ToFColor(true), Settings->TrianglesLineThickness);
						LineComponent->SetLineMaterial(LinesMaterial);
					}
				}
			}
		}

		UMaterialInterface* PointsMaterial = nullptr;
		UMaterialInterface* LinesMaterial = nullptr;
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void RegisterMedialSkeletonRenderableType()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FMedialSkeletonRenderableType);

		static const FName DisplayCategoryName(TEXT("Display"));
		static const FName SpheresCategoryName(TEXT("Spheres"));
		static const FName CentersCategoryName(TEXT("Centers"));
		static const FName LinesCategoryName(TEXT("Lines"));
		static const FName TrianglesCategoryName(TEXT("Triangles"));

		UDataflowRenderableTypeSettings::RegisterSection(
			UDataflowMedialSkeletonRenderSettings::StaticClass(),
			"MedialSkeleton",
			LOCTEXT("MedialSkeleton", "MedialSkeleton"),
			{ DisplayCategoryName, SpheresCategoryName, CentersCategoryName, LinesCategoryName, TrianglesCategoryName }
		);
	}
}

#undef LOCTEXT_NAMESPACE 
