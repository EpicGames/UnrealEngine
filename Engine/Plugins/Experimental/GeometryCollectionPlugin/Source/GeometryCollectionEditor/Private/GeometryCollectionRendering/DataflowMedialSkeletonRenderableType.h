// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowRendering/DataflowRenderableTypeSettings.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "DataflowMedialSkeletonRenderableType.generated.h"

UENUM(BlueprintType)
enum class EDataflowMedialSkeletonSpheresColorMethodType : uint8
{
	Constant UMETA(DisplayName = "Constant"),
	Random UMETA(DisplayName = "Random"),
	BySize UMETA(DisplayName = "By Size"),
	ByIDs UMETA(DisplayName = "By IDs")
};

UCLASS(MinimalAPI)
class UDataflowMedialSkeletonRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()

	UDataflowMedialSkeletonRenderSettings(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

public:
	/** Display surface as wireframe */
	UPROPERTY(EditAnywhere, Category = "Display")
	bool bWireframe = false;

	/** Transparency */
	UPROPERTY(EditAnywhere, Category = "Display", meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1))
	float Transparency = 0.75f;

	/** Display the medial spheres */
	UPROPERTY(EditAnywhere, Category = "Spheres")
	bool bShowSpheres = true;

	/** Method for coloring the medial spheres */
	UPROPERTY(EditAnywhere, Category = "Spheres")
	EDataflowMedialSkeletonSpheresColorMethodType SpheresColorMethod = EDataflowMedialSkeletonSpheresColorMethodType::Constant;

	/** Constant color for the spheres */
	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (EditCondition = "SpheresColorMethod == EDataflowMedialSkeletonSpheresColorMethodType::Constant"))
	FLinearColor Color = FLinearColor(0.2f, 0.97f, 1.0f);

	/** Seed for the random color */
	UPROPERTY(EditAnywhere, Category = "Spheres", meta = (UIMin = 0, EditCondition = "SpheresColorMethod == EDataflowMedialSkeletonSpheresColorMethodType::Random"))
	int32 RandomSeed = 0;

	/** Show medial spheres centers */
	UPROPERTY(EditAnywhere, Category = "Centers", meta = (InlineEditConditionToggle))
	bool bShowCenters = true;

	/** Color of the centers */
	UPROPERTY(EditAnywhere, Category = "Centers", meta = (EditCondition = bShowCenters))
	FLinearColor CentersColor = FLinearColor(0.0f, 0.466f, 1.0f);

	/** Size of the centers */
	UPROPERTY(EditAnywhere, Category = "Centers", meta = (UIMin = 0.01f, ClampMin = 0.01f))
	float Size = 10.0f;

	/** Display medial sphere IDs */
	UPROPERTY(EditAnywhere, Category = "Centers", meta = (InlineEditConditionToggle))
	bool bShowIDs = false;

	/** Color of the IDs text */
	UPROPERTY(EditAnywhere, Category = "Centers", meta = (EditCondition = bShowIDs))
	FLinearColor IDsColor = FLinearColor(0.0, 1.0, 1.0);

	/** Show connections */
	UPROPERTY(EditAnywhere, Category = "Lines", meta = (InlineEditConditionToggle))
	bool bShowLines = true;

	/** Color of the connections */
	UPROPERTY(EditAnywhere, Category = "Lines", meta = (EditCondition = bShowLines))
	FLinearColor LinesColor = FLinearColor(0.15f, 0.0f, 1.0f);

	/** Line thickness for the connections */
	UPROPERTY(EditAnywhere, Category = "Lines", meta = (UIMin = 1.f, ClampMin = 1.f))
	float LineThickness = 2.0;

	/** Display triangles */
	UPROPERTY(EditAnywhere, Category = "Triangles", meta = (InlineEditConditionToggle))
	bool bShowTriangles = true;

	/** Color of the triangles */
	UPROPERTY(EditAnywhere, Category = "Triangles", meta = (EditCondition = bShowTriangles))
	FLinearColor TrianglesColor = FLinearColor(1.0, 0.0, 1.0);

	/** Line thickness for the triangles */
	UPROPERTY(EditAnywhere, Category = "Triangles", meta = (UIMin = 1.f, ClampMin = 1.f))
	float TrianglesLineThickness = 6.0;

	/** Color ramp for coloring by size/IDs */
	UPROPERTY(EditAnywhere, Category = "Color", meta = (EditCondition = "SpheresColorMethod == EDataflowMedialSkeletonSpheresColorMethodType::BySize || SpheresColorMethod == EDataflowMedialSkeletonSpheresColorMethodType::ByIDs", EditConditionHides))
	FLinearColorRamp ColorRamp;
};

namespace UE::Dataflow::Private
{
	void RegisterMedialSkeletonRenderableType();
}
