// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "AssetDefinition_GeometryCollection.generated.h"

#define UE_API GEOMETRYCOLLECTIONEDITOR_API

UCLASS(MinimalAPI)
class UAssetDefinition_GeometryCollection : public UAssetDefinitionDefault
{
	GENERATED_BODY()

private:

	UE_API virtual FText GetAssetDisplayName() const override;
	UE_API virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	UE_API virtual FLinearColor GetAssetColor() const override;
	UE_API virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	UE_API virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData) const override;

	UE_API virtual	FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const;
	UE_API virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
};

#undef UE_API
