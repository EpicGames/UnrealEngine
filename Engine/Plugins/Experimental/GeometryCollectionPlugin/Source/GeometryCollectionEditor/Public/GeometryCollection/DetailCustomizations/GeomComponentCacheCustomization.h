// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

struct FAssetData;
class UGeometryCollection;
class UGeometryCollectionCache;

class FGeomComponentCacheParametersCustomization : public IPropertyTypeCustomization, public FGCObject
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	// FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FGeomComponentCacheParametersCustomization");
	}

private:

	// Get the collection attached to the parameter struct.
	const UGeometryCollection* GetCollection() const;
	TObjectPtr<const UGeometryCollection>* GetCollectionRef() const;

	// Whether we filter out caches. Condition is that the Id guids match.
	bool ShouldFilterAsset(const FAssetData& InData) const;

	// Check the conditions required for a cache to be compatible with recording and playback. bOutIdsMatch describes whether the cache
	// is compatible at all and bOutStatesMatch describes whether playback is possible using that cache and collection combo.
	void CheckTagsMatch(const UGeometryCollection* InCollection, const UGeometryCollectionCache* InCache, bool& bOutIdsMatch, bool& bOutStatesMatch);

	// Handle to the target cache property inside the parameters struct
	TSharedPtr<IPropertyHandle> TargetCacheHandle;
};