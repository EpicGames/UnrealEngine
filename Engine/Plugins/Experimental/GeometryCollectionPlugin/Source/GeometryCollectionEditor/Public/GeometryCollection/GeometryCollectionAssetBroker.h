// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ComponentAssetBroker.h"

/** AssetBroker class for GeometryCollection assets*/
class FGeometryCollectionAssetBroker : public IComponentAssetBroker
{
public:
	// Begin IComponentAssetBroker Implementation
	UClass* GetSupportedAssetClass() override;
	virtual bool AssignAssetToComponent(UActorComponent* InComponent, UObject* InAsset) override;
	virtual UObject* GetAssetFromComponent(UActorComponent* InComponent) override;
	// End IComponentAssetBroker Implementation
};
