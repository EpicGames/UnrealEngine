// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "GeometryCollectionCacheFactory.generated.h"

class UGeometryCollection;
class UGeometryCollectionCache;
class SWindow;
struct FAssetData;

UCLASS(Experimental)
class UGeometryCollectionCacheFactory : public UFactory
{
	GENERATED_BODY()

public:

	/** Config properties required for CreateNew */

	UPROPERTY()
	TObjectPtr<UGeometryCollection> TargetCollection;

	/** End required properties */

	UGeometryCollectionCacheFactory();

	/** UFactory Interface */
	virtual bool CanCreateNew() const override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual bool ShouldShowInNewMenu() const override;
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ConfigureProperties() override;
	/** End UFactory Interface */

private:

	/** Callback from window created in ConfigureProperties once a user picks a target Geometry Collection */
	void OnConfigSelection(const FAssetData& InSelectedAssetData);

	/** Window created in ConfigureProperties for target selection */
	TSharedPtr<SWindow> PickerWindow;
};