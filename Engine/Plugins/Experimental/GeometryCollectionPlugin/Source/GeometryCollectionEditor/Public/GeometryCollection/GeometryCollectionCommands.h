// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Engine/World.h"

#define UE_API GEOMETRYCOLLECTIONEDITOR_API

class UStaticMesh;
class USkeletalMesh;
class UGeometryCollection;

/**
* The public interface to this module
*/
class FGeometryCollectionCommands
{
public:

	/**
	*  Command invoked from "GeometryCollection.ToString", uses the selected GeometryCollectionActor to output the RestCollection to the Log
	*  @param World
	*/
	static UE_API void ToString(UWorld * World);

	/**
	*  Command invoked from "GeometryCollection.WriteToHeaderFile", uses the selected GeometryCollectionActor to output the RestCollection to a header file
	*  @param World
	*/
	static UE_API void WriteToHeaderFile(const TArray<FString>& Args, UWorld * World);

	/**
	*  Command invoked from "GeometryCollection.WriteToOBJFile", uses the selected GeometryCollectionActor to output the RestCollection to an OBJ file
	*  @param World
	*/
	static UE_API void WriteToOBJFile(const TArray<FString>& Args, UWorld * World);

	/**
	*  Command invoked from "GeometryCollectionAlgo.PrintStatistics", uses the selected GeometryCollectionActor to output statistics
	*  @param World
	*/
	static UE_API void PrintStatistics(UWorld * World);

	/**
	*  Command invoked from "GeometryCollectionAlgo.PrintDetailedStatistics", uses the selected GeometryCollectionActor to output detailed statistics
	*  @param World
	*/
	static UE_API void PrintDetailedStatistics(UWorld * World);

	/**
	*  Command invoked from "GeometryCollectionAlgo.PrintDetailedStatisticsSummary", uses the selected GeometryCollectionActor to output detailed statistics
	*  @param World
	*/
	static UE_API void PrintDetailedStatisticsSummary(UWorld * World);

	/**
	*
	*  @param World
	*/
	static UE_API void DeleteCoincidentVertices(const TArray<FString>& Args, UWorld * World);


	/**
	*  Command to set attributes within a named group. 
	*
	*    GeometryCollection.SetNamedAttributeValues <type> <Attribute> <Group> <Value> [<Regex Attribute> <Regex>]
	* 
	*    Where type is in SupportedAttributeTypes (bool,int32,float)
	*      and <Attribute> exists within <Group>
	*      with optional regex matching against <RegexAttribute> within <Group>
	*
	*  For Example : 
	*     GeometryCollection.SetNamedAttributeValues bool SimulatableParticlesAttribute Transform 0 BoneName Cube_1_1
	*
	*     This will set the bool attribute SimulatableParticlesAttribute of the Transform group to false where BoneName equals Cube_1_1
	*
	*/
	static UE_API void SetNamedAttributeValues(const TArray<FString>& Args, UWorld* World);

	/**
	*
	*  @param World
	*/
	static UE_API void DeleteZeroAreaFaces(const TArray<FString>& Args, UWorld * World);

	/**
	*
	*  @param World
	*/
	static UE_API void DeleteHiddenFaces(UWorld * World);

	/**
	*
	*  @param World
	*/
	static UE_API void DeleteStaleVertices(UWorld * World);

	/*
	*  Split across xz-plane
	*/
	static UE_API void SplitAcrossYZPlane(UWorld * World);

	/**
	* Remove the selected geometry entry
	*/
	static UE_API void DeleteGeometry(const TArray<FString>& Args, UWorld* World);

	/**
	* Select all geometry in hierarchy
	*/
	static UE_API void SelectAllGeometry(const TArray<FString>& Args, UWorld* World);

	/**
	* Select no geometry in hierarchy
	*/
	static UE_API void SelectNone(const TArray<FString>& Args, UWorld* World);

	/**
	* Select no geometry in hierarchy
	*/
	static UE_API void SelectLessThenVolume(const TArray<FString>& Args, UWorld* World);

	/**
	* Select inverse of currently selected geometry in hierarchy
	*/
	static UE_API void SelectInverseGeometry(const TArray<FString>& Args, UWorld* World);

	/*
	*  Ensure single root.
	*/
	static UE_API int32 EnsureSingleRoot(UGeometryCollection* RestCollection);

	/*
	*  Build Proximity Database
	*/
	static UE_API void BuildProximityDatabase(const TArray<FString>& Args, UWorld * World);

	/*
	*  Test Bone Asset
	*/
	static UE_API void SetupNestedBoneAsset(UWorld * World);

	/*
	*  Setup two clustered cubes
	*/
	static UE_API void SetupTwoClusteredCubesAsset(UWorld * World);

	/*
*  Remove Holes
*/
	static UE_API void HealGeometry(UWorld * World);

};

#undef UE_API
