// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Engine/World.h"

#define UE_API GEOMETRYCOLLECTIONEDITOR_API

class UStaticMesh;
class USkeletalMesh;
class UGeometryCollection;
class UMaterialInterface;
class UGeometryCollectionComponent;

typedef TTuple<const UStaticMesh *, const UStaticMeshComponent *, FTransform> GeometryCollectionStaticMeshConversionTuple;
typedef TTuple<const USkeletalMesh *, const USkeletalMeshComponent *, FTransform> GeometryCollectionSkeletalMeshConversionTuple;

/**
* The public interface to this module
*/
class FGeometryCollectionConversion
{
public:

	/**
	*  Appends a static mesh to a GeometryCollectionComponent.
	*  @param StaticMesh : Const mesh to read vertex/normals/index data from
	*  @param Materials : Materials fetched from the StaticMeshComponent used to configure this geometry
	*  @param StaticMeshTransform : Mesh transform.
	*  @param GeometryCollection  : Collection to append the mesh into.
	*/
	static UE_API void AppendStaticMesh(const UStaticMesh* StaticMesh, const TArray<UMaterialInterface*>& Materials, const FTransform& StaticMeshTransform, UGeometryCollection* GeometryCollectionObject, bool ReindexMaterials = true);

	/**
	*  Appends a static mesh to a GeometryCollectionComponent.
	*  @param StaticMeshComponent : Const mesh to read vertex/normals/index data from
	*  @param StaticMeshTransform : Mesh transform.
	*  @param GeometryCollection  : Collection to append the mesh into.
	*/
	static UE_API void AppendStaticMesh(const UStaticMesh * StaticMesh, const UStaticMeshComponent *StaticMeshComponent, const FTransform & StaticMeshTransform, UGeometryCollection * GeometryCollection, bool ReindexMaterials = true);

	/**
	*  Appends a skeletal mesh to a GeometryCollectionComponent.
	*  @param SkeletalMeshComponent : Const mesh to read vertex/normals/index data from
	*  @param SkeletalMeshTransform : Mesh transform.
	*  @param GeometryCollection    : Collection to append the mesh into.
	*/
	static UE_API void AppendSkeletalMesh(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent *SkeletalMeshComponent, const FTransform & SkeletalMeshTransform, UGeometryCollection * GeometryCollection, bool ReindexMaterials = true);

	/**
	*  Appends a GeometryCollection to a GeometryCollectionComponent.
	*  @param SourceGeometryCollection : Const GeometryCollection to read vertex/normals/index data from
	*  @param Materials : Materials fetched from the GeometryCollectionComponent used to configure this geometry
	*  @param GeometryCollectionTransform : GeometryCollection transform.
	*  @param TargetGeometryCollection  : Collection to append the GeometryCollection into.
	*/
	static UE_API void AppendGeometryCollection(const UGeometryCollection* SourceGeometryCollection, const TArray<UMaterialInterface*>& Materials, const FTransform& GeometryCollectionTransform, UGeometryCollection* TargetGeometryCollectionObject, bool ReindexMaterials = true);

	/**
	*  Appends a GeometryCollection to a GeometryCollectionComponent.
	*  @param GeometryCollectionComponent : Const GeometryCollection to read vertex/normals/index data from
	*  @param GeometryCollectionTransform : GeometryCollection transform.
	*  @param TargetGeometryCollection  : Collection to append the GeometryCollection into.
	*/
	static UE_API void AppendGeometryCollection(const UGeometryCollection* SourceGeometryCollection, const UGeometryCollectionComponent* GeometryCollectionComponent, const FTransform& GeometryCollectionTransform, UGeometryCollection* TargetGeometryCollectionObject, bool ReindexMaterials = true);


	/**
	*  Command invoked from "GeometryCollection.CreatGeometryCollection", uses the selected Actors to create a GeometryCollection Asset
	*  @param World
	*/
	static UE_API void CreateGeometryCollectionCommand(UWorld * World);

	/**
	*  Command invoked from "GeometryCollection.CreateFromSelectedActors", uses the selected Actors to create a GeometryCollection Asset
	*  @param World
	*/
	static UE_API void CreateFromSelectedActorsCommand(UWorld * World);

	/**
	*  Command invoked from "GeometryCollection.CreateFromSelectedAssets", uses the selected Assets to create a GeometryCollection Asset
	*  @param World
	*/
	static UE_API void CreateFromSelectedAssetsCommand(UWorld * World);
};

#undef UE_API
