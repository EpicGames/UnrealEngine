// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LevelEditor.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "GeometryCollection/GeometryCollectionProviderEditor.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/IToolkit.h"


class ISlateStyle;
class FSlateStyleSet;
class FAssetTypeActions_GeometryCollection;
class FAssetTypeActions_GeometryCollectionCache;
class FGeometryCollectionAssetBroker;
class FAssetEditorToolkit;
class IToolkitHost;

/**
 * The public interface to this module
 */
class IGeometryCollectionEditorPlugin : public IModuleInterface
{
public:
	virtual void StartupModule();
	virtual void ShutdownModule();

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IGeometryCollectionEditorPlugin& Get()
	{
		return FModuleManager::LoadModuleChecked< IGeometryCollectionEditorPlugin >( "GeometryCollectionEditorPlugin" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "GeometryCollectionEditorPlugin" );
	}

	TSharedPtr<FSlateStyleSet> GetStyleSet() { return StyleSet; }

	static FName GetEditorStyleName();
	static const ISlateStyle* GetEditorStyle();

private:
	void RegisterMenus();

	void CreateGeometrySelectionMenu(FMenuBuilder& InMenuBuilder);
	TSharedRef<FExtender> OnExtendLevelEditorViewMenu(const TSharedRef<FUICommandList> InCommandList);

private:
	TArray<IConsoleObject*> EditorCommands;

	// Asset actions for new asset types
	FGeometryCollectionAssetBroker* AssetBroker;

	// Modular features
	// Provider for new caches requested from other modules
	FTargetCacheProviderEditor TargetCacheProvider;
	//////////////////////////////////////////////////////////////////////////

	// Styleset for geom collection tool brushes/fonts etc.
	TSharedPtr<FSlateStyleSet> StyleSet;

	FLevelEditorModule::FLevelEditorMenuExtender ViewMenuExtender;
	FDelegateHandle ViewMenuExtenderHandle;
	FDelegateHandle DataflowMenusHandle;
};

