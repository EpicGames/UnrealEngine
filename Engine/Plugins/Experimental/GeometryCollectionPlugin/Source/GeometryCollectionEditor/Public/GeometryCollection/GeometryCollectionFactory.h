// Copyright Epic Games, Inc. All Rights Reserved.

/** Factory which allows import of an GeometryCollectionAsset */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "GeometryCollection/GeometryCollectionObject.h"

#include "GeometryCollectionFactory.generated.h"

#define UE_API GEOMETRYCOLLECTIONEDITOR_API

class UGeometryCollection;
class UGeometryCollectionComponent;

typedef TTuple<const UGeometryCollection *, const UGeometryCollectionComponent *, FTransform> GeometryCollectionTuple;

/**
* Factory for Simple Cube
*/

UCLASS(MinimalAPI)
class UGeometryCollectionFactory : public UFactory
{
    GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	

	static UE_API UGeometryCollection* StaticFactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn);

};


#undef UE_API
