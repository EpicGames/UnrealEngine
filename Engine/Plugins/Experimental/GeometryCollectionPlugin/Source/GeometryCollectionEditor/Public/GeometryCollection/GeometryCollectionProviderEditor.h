// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCollection/GeometryCollectionCache.h"

class FTargetCacheProviderEditor : public ITargetCacheProvider
{
public:

	virtual UGeometryCollectionCache* GetCacheForCollection(const UGeometryCollection* InCollection) override;

};