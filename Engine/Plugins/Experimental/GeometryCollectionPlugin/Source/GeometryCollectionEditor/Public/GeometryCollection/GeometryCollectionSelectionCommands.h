// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/AppStyle.h"
#include "Framework/Commands/Commands.h"

class GEOMETRYCOLLECTIONEDITOR_API FGeometryCollectionSelectionCommands : public TCommands<FGeometryCollectionSelectionCommands>
{
public:
	FGeometryCollectionSelectionCommands()
		: TCommands<FGeometryCollectionSelectionCommands>
		(
			TEXT("GeometryCollectionSelection"),
			NSLOCTEXT("Contexts", "GeometryCollectionSelection", "Geometry Collection Selection Commands"),
			NAME_None,
			FAppStyle::GetAppStyleSetName()
		)
	{
	}

	virtual void RegisterCommands() override;

	/** Selects all Geometry Collection geometry */
	TSharedPtr<FUICommandInfo> SelectAllGeometry;

	/** Deselects all Geometry Collection geometry */
	TSharedPtr<FUICommandInfo> SelectNone;

	/** Selects inverse of currently seleted Geometry Collection geometry */
	TSharedPtr<FUICommandInfo> SelectInverseGeometry;
};