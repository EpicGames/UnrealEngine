// Copyright Epic Games, Inc. All Rights Reserved.

/**
*
* This thumbnail renderer displays a given GeometryCollection
*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/DefaultSizedThumbnailRenderer.h"
#include "GeometryCollectionThumbnailRenderer.generated.h"

class FCanvas;
class FGeometryCollectionThumbnailScene;
class FRenderTarget;

UCLASS(config = Editor, MinimalAPI)
class UGeometryCollectionThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_UCLASS_BODY()

	// Begin UThumbnailRenderer Object
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	// End UThumbnailRenderer Object

	// Begin UObject implementation
	virtual void BeginDestroy() override;
	// End UObject implementation
private:
	FGeometryCollectionThumbnailScene* ThumbnailScene;
};
