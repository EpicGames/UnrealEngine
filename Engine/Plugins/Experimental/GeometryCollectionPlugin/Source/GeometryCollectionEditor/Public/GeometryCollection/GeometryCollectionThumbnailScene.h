// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ThumbnailHelpers.h"

class AGeometryCollectionActor;
class UGeometryCollection;

class FGeometryCollectionThumbnailScene : public FThumbnailPreviewScene
{
public:
	/** Constructor */
	FGeometryCollectionThumbnailScene();

	/** Sets the geometry collection to use in the next CreateView() */
	void SetGeometryCollection(UGeometryCollection* GeometryCollection);

protected:
	// FThumbnailPreviewScene implementation
	virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

private:
	/** The actor used to display all geometry collection thumbnails */
	AGeometryCollectionActor* PreviewActor;
};
