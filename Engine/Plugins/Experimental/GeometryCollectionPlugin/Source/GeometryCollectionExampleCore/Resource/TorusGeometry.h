// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
namespace GeometryCollectionExample
{
	class TorusGeometry
	{

	public:
		TorusGeometry() {}
		~TorusGeometry() {}

		static const TArray<float>	RawVertexArray;
		static const TArray<int32>	RawIndicesArray;
	};


	const TArray<float> TorusGeometry::RawVertexArray = {
															120.000000, -0.000193, -0.000002,
															114.126831, 37.081871, -0.000002,
															97.082130, 70.534103, -0.000002,
															70.534348, 97.081955, -0.000002,
															37.082165, 114.126740, -0.000002,
															0.000116, 120.000000, -0.000002,
															-37.081944, 114.126816, -0.000002,
															-70.534164, 97.082092, -0.000002,
															-97.082001, 70.534286, -0.000002,
															-114.126762, 37.082092, -0.000002,
															-120.000000, 0.000039, -0.000002,
															-114.126793, -37.082016, -0.000002,
															-97.082047, -70.534225, -0.000002,
															-70.534225, -97.082039, -0.000002,
															-37.082043, -114.126778, -0.000002,
															-0.000005, -120.000000, -0.000002,
															37.082035, -114.126785, -0.000002,
															70.534225, -97.082039, -0.000002,
															97.082039, -70.534225, -0.000002,
															114.126785, -37.082039, -0.000002,
															115.320885, -0.000185, -12.855755,
															109.676727, 35.635952, -12.855755,
															93.296646, 67.783791, -12.855755,
															67.784035, 93.296471, -12.855755,
															35.636234, 109.676643, -12.855755,
															0.000111, 115.320885, -12.855755,
															-35.636021, 109.676712, -12.855755,
															-67.783852, 93.296608, -12.855755,
															-93.296516, 67.783966, -12.855755,
															-109.676659, 35.636162, -12.855755,
															-115.320885, 0.000038, -12.855755,
															-109.676689, -35.636089, -12.855755,
															-93.296562, -67.783905, -12.855755,
															-67.783905, -93.296555, -12.855755,
															-35.636116, -109.676674, -12.855755,
															-0.000005, -115.320885, -12.855755,
															35.636112, -109.676682, -12.855755,
															67.783913, -93.296555, -12.855755,
															93.296555, -67.783913, -12.855755,
															109.676682, -35.636116, -12.855755,
															103.472961, -0.000166, -19.696156,
															98.408676, 31.974760, -19.696156,
															83.711464, 60.819767, -19.696156,
															60.819984, 83.711311, -19.696156,
															31.975012, 98.408600, -19.696156,
															0.000100, 103.472961, -19.696156,
															-31.974821, 98.408661, -19.696156,
															-60.819824, 83.711426, -19.696156,
															-83.711349, 60.819931, -19.696156,
															-98.408615, 31.974947, -19.696156,
															-103.472961, 0.000034, -19.696156,
															-98.408646, -31.974882, -19.696156,
															-83.711388, -60.819874, -19.696156,
															-60.819874, -83.711388, -19.696156,
															-31.974907, -98.408630, -19.696156,
															-0.000005, -103.472961, -19.696156,
															31.974901, -98.408638, -19.696156,
															60.819881, -83.711388, -19.696156,
															83.711388, -60.819881, -19.696156,
															98.408638, -31.974905, -19.696156,
															90.000000, -0.000144, -17.320509,
															85.595123, 27.811403, -17.320509,
															72.811600, 52.900574, -17.320509,
															52.900764, 72.811462, -17.320509,
															27.811625, 85.595055, -17.320509,
															0.000087, 90.000000, -17.320509,
															-27.811459, 85.595108, -17.320509,
															-52.900623, 72.811569, -17.320509,
															-72.811501, 52.900715, -17.320509,
															-85.595070, 27.811567, -17.320509,
															-90.000000, 0.000029, -17.320509,
															-85.595093, -27.811512, -17.320509,
															-72.811539, -52.900665, -17.320509,
															-52.900665, -72.811531, -17.320509,
															-27.811533, -85.595085, -17.320509,
															-0.000004, -90.000000, -17.320509,
															27.811527, -85.595085, -17.320509,
															52.900673, -72.811531, -17.320509,
															72.811531, -52.900673, -17.320509,
															85.595085, -27.811531, -17.320509,
															81.206146, -0.000130, -6.840407,
															77.231667, 25.093966, -6.840407,
															65.697212, 47.731686, -6.840407,
															47.731857, 65.697098, -6.840407,
															25.094164, 77.231606, -6.840407,
															0.000078, 81.206146, -6.840407,
															-25.094015, 77.231659, -6.840407,
															-47.731731, 65.697189, -6.840407,
															-65.697121, 47.731812, -6.840407,
															-77.231621, 25.094114, -6.840407,
															-81.206146, 0.000026, -6.840407,
															-77.231644, -25.094063, -6.840407,
															-65.697159, -47.731770, -6.840407,
															-47.731770, -65.697151, -6.840407,
															-25.094082, -77.231628, -6.840407,
															-0.000004, -81.206146, -6.840407,
															25.094078, -77.231636, -6.840407,
															47.731773, -65.697151, -6.840407,
															65.697151, -47.731773, -6.840407,
															77.231636, -25.094080, -6.840407,
															81.206146, -0.000130, 6.840396,
															77.231667, 25.093966, 6.840396,
															65.697212, 47.731686, 6.840396,
															47.731857, 65.697098, 6.840396,
															25.094164, 77.231606, 6.840396,
															0.000078, 81.206146, 6.840396,
															-25.094015, 77.231659, 6.840396,
															-47.731731, 65.697189, 6.840396,
															-65.697121, 47.731812, 6.840396,
															-77.231621, 25.094114, 6.840396,
															-81.206146, 0.000026, 6.840396,
															-77.231644, -25.094063, 6.840396,
															-65.697159, -47.731770, 6.840396,
															-47.731770, -65.697151, 6.840396,
															-25.094082, -77.231628, 6.840396,
															-0.000004, -81.206146, 6.840396,
															25.094078, -77.231636, 6.840396,
															47.731773, -65.697151, 6.840396,
															65.697151, -47.731773, 6.840396,
															77.231636, -25.094080, 6.840396,
															89.999992, -0.000144, 17.320503,
															85.595116, 27.811401, 17.320503,
															72.811592, 52.900570, 17.320503,
															52.900757, 72.811462, 17.320503,
															27.811623, 85.595047, 17.320503,
															0.000087, 89.999992, 17.320503,
															-27.811455, 85.595100, 17.320503,
															-52.900620, 72.811562, 17.320503,
															-72.811493, 52.900711, 17.320503,
															-85.595062, 27.811565, 17.320503,
															-89.999992, 0.000029, 17.320503,
															-85.595085, -27.811508, 17.320503,
															-72.811531, -52.900661, 17.320503,
															-52.900661, -72.811523, 17.320503,
															-27.811531, -85.595078, 17.320503,
															-0.000004, -89.999992, 17.320503,
															27.811525, -85.595085, 17.320503,
															52.900669, -72.811523, 17.320503,
															72.811523, -52.900669, 17.320503,
															85.595085, -27.811527, 17.320503,
															103.472954, -0.000166, 19.696157,
															98.408676, 31.974756, 19.696157,
															83.711456, 60.819763, 19.696157,
															60.819981, 83.711304, 19.696157,
															31.975010, 98.408592, 19.696157,
															0.000100, 103.472954, 19.696157,
															-31.974819, 98.408653, 19.696157,
															-60.819820, 83.711426, 19.696157,
															-83.711342, 60.819923, 19.696157,
															-98.408607, 31.974945, 19.696157,
															-103.472954, 0.000034, 19.696157,
															-98.408638, -31.974880, 19.696157,
															-83.711388, -60.819870, 19.696157,
															-60.819870, -83.711380, 19.696157,
															-31.974905, -98.408623, 19.696157,
															-0.000005, -103.472954, 19.696157,
															31.974899, -98.408630, 19.696157,
															60.819874, -83.711380, 19.696157,
															83.711380, -60.819874, 19.696157,
															98.408630, -31.974901, 19.696157,
															115.320877, -0.000185, 12.855764,
															109.676720, 35.635952, 12.855764,
															93.296638, 67.783783, 12.855764,
															67.784027, 93.296471, 12.855764,
															35.636230, 109.676636, 12.855764,
															0.000111, 115.320877, 12.855764,
															-35.636021, 109.676704, 12.855764,
															-67.783852, 93.296600, 12.855764,
															-93.296509, 67.783966, 12.855764,
															-109.676651, 35.636158, 12.855764,
															-115.320877, 0.000038, 12.855764,
															-109.676682, -35.636089, 12.855764,
															-93.296555, -67.783905, 12.855764,
															-67.783905, -93.296547, 12.855764,
															-35.636116, -109.676666, 12.855764,
															-0.000005, -115.320877, 12.855764,
															35.636108, -109.676674, 12.855764,
															67.783913, -93.296547, 12.855764,
															93.296547, -67.783913, 12.855764,
															109.676674, -35.636112, 12.855764
	};

	const TArray<int32> TorusGeometry::RawIndicesArray = {
															0, 1, 21,
															1, 2, 22,
															2, 3, 23,
															3, 4, 24,
															4, 5, 25,
															5, 6, 26,
															6, 7, 27,
															7, 8, 28,
															8, 9, 29,
															9, 10, 30,
															10, 11, 31,
															11, 12, 32,
															12, 13, 33,
															13, 14, 34,
															14, 15, 35,
															15, 16, 36,
															16, 17, 37,
															17, 18, 38,
															18, 19, 39,
															19, 0, 20,
															20, 21, 41,
															21, 22, 42,
															22, 23, 43,
															23, 24, 44,
															24, 25, 45,
															25, 26, 46,
															26, 27, 47,
															27, 28, 48,
															28, 29, 49,
															29, 30, 50,
															30, 31, 51,
															31, 32, 52,
															32, 33, 53,
															33, 34, 54,
															34, 35, 55,
															35, 36, 56,
															36, 37, 57,
															37, 38, 58,
															38, 39, 59,
															39, 20, 40,
															40, 41, 61,
															41, 42, 62,
															42, 43, 63,
															43, 44, 64,
															44, 45, 65,
															45, 46, 66,
															46, 47, 67,
															47, 48, 68,
															48, 49, 69,
															49, 50, 70,
															50, 51, 71,
															51, 52, 72,
															52, 53, 73,
															53, 54, 74,
															54, 55, 75,
															55, 56, 76,
															56, 57, 77,
															57, 58, 78,
															58, 59, 79,
															59, 40, 60,
															60, 61, 81,
															61, 62, 82,
															62, 63, 83,
															63, 64, 84,
															64, 65, 85,
															65, 66, 86,
															66, 67, 87,
															67, 68, 88,
															68, 69, 89,
															69, 70, 90,
															70, 71, 91,
															71, 72, 92,
															72, 73, 93,
															73, 74, 94,
															74, 75, 95,
															75, 76, 96,
															76, 77, 97,
															77, 78, 98,
															78, 79, 99,
															79, 60, 80,
															80, 81, 101,
															81, 82, 102,
															82, 83, 103,
															83, 84, 104,
															84, 85, 105,
															85, 86, 106,
															86, 87, 107,
															87, 88, 108,
															88, 89, 109,
															89, 90, 110,
															90, 91, 111,
															91, 92, 112,
															92, 93, 113,
															93, 94, 114,
															94, 95, 115,
															95, 96, 116,
															96, 97, 117,
															97, 98, 118,
															98, 99, 119,
															99, 80, 100,
															100, 101, 121,
															101, 102, 122,
															102, 103, 123,
															103, 104, 124,
															104, 105, 125,
															105, 106, 126,
															106, 107, 127,
															107, 108, 128,
															108, 109, 129,
															109, 110, 130,
															110, 111, 131,
															111, 112, 132,
															112, 113, 133,
															113, 114, 134,
															114, 115, 135,
															115, 116, 136,
															116, 117, 137,
															117, 118, 138,
															118, 119, 139,
															119, 100, 120,
															120, 121, 141,
															121, 122, 142,
															122, 123, 143,
															123, 124, 144,
															124, 125, 145,
															125, 126, 146,
															126, 127, 147,
															127, 128, 148,
															128, 129, 149,
															129, 130, 150,
															130, 131, 151,
															131, 132, 152,
															132, 133, 153,
															133, 134, 154,
															134, 135, 155,
															135, 136, 156,
															136, 137, 157,
															137, 138, 158,
															138, 139, 159,
															139, 120, 140,
															140, 141, 161,
															141, 142, 162,
															142, 143, 163,
															143, 144, 164,
															144, 145, 165,
															145, 146, 166,
															146, 147, 167,
															147, 148, 168,
															148, 149, 169,
															149, 150, 170,
															150, 151, 171,
															151, 152, 172,
															152, 153, 173,
															153, 154, 174,
															154, 155, 175,
															155, 156, 176,
															156, 157, 177,
															157, 158, 178,
															158, 159, 179,
															159, 140, 160,
															160, 161, 1,
															161, 162, 2,
															162, 163, 3,
															163, 164, 4,
															164, 165, 5,
															165, 166, 6,
															166, 167, 7,
															167, 168, 8,
															168, 169, 9,
															169, 170, 10,
															170, 171, 11,
															171, 172, 12,
															172, 173, 13,
															173, 174, 14,
															174, 175, 15,
															175, 176, 16,
															176, 177, 17,
															177, 178, 18,
															178, 179, 19,
															179, 160, 0,
															0, 19, 179,
															19, 18, 178,
															18, 17, 177,
															17, 16, 176,
															16, 15, 175,
															15, 14, 174,
															14, 13, 173,
															13, 12, 172,
															12, 11, 171,
															11, 10, 170,
															10, 9, 169,
															9, 8, 168,
															8, 7, 167,
															7, 6, 166,
															6, 5, 165,
															5, 4, 164,
															4, 3, 163,
															3, 2, 162,
															2, 1, 161,
															1, 0, 160,
															160, 179, 159,
															179, 178, 158,
															178, 177, 157,
															177, 176, 156,
															176, 175, 155,
															175, 174, 154,
															174, 173, 153,
															173, 172, 152,
															172, 171, 151,
															171, 170, 150,
															170, 169, 149,
															169, 168, 148,
															168, 167, 147,
															167, 166, 146,
															166, 165, 145,
															165, 164, 144,
															164, 163, 143,
															163, 162, 142,
															162, 161, 141,
															161, 160, 140,
															140, 159, 139,
															159, 158, 138,
															158, 157, 137,
															157, 156, 136,
															156, 155, 135,
															155, 154, 134,
															154, 153, 133,
															153, 152, 132,
															152, 151, 131,
															151, 150, 130,
															150, 149, 129,
															149, 148, 128,
															148, 147, 127,
															147, 146, 126,
															146, 145, 125,
															145, 144, 124,
															144, 143, 123,
															143, 142, 122,
															142, 141, 121,
															141, 140, 120,
															120, 139, 119,
															139, 138, 118,
															138, 137, 117,
															137, 136, 116,
															136, 135, 115,
															135, 134, 114,
															134, 133, 113,
															133, 132, 112,
															132, 131, 111,
															131, 130, 110,
															130, 129, 109,
															129, 128, 108,
															128, 127, 107,
															127, 126, 106,
															126, 125, 105,
															125, 124, 104,
															124, 123, 103,
															123, 122, 102,
															122, 121, 101,
															121, 120, 100,
															100, 119, 99,
															119, 118, 98,
															118, 117, 97,
															117, 116, 96,
															116, 115, 95,
															115, 114, 94,
															114, 113, 93,
															113, 112, 92,
															112, 111, 91,
															111, 110, 90,
															110, 109, 89,
															109, 108, 88,
															108, 107, 87,
															107, 106, 86,
															106, 105, 85,
															105, 104, 84,
															104, 103, 83,
															103, 102, 82,
															102, 101, 81,
															101, 100, 80,
															80, 99, 79,
															99, 98, 78,
															98, 97, 77,
															97, 96, 76,
															96, 95, 75,
															95, 94, 74,
															94, 93, 73,
															93, 92, 72,
															92, 91, 71,
															91, 90, 70,
															90, 89, 69,
															89, 88, 68,
															88, 87, 67,
															87, 86, 66,
															86, 85, 65,
															85, 84, 64,
															84, 83, 63,
															83, 82, 62,
															82, 81, 61,
															81, 80, 60,
															60, 79, 59,
															79, 78, 58,
															78, 77, 57,
															77, 76, 56,
															76, 75, 55,
															75, 74, 54,
															74, 73, 53,
															73, 72, 52,
															72, 71, 51,
															71, 70, 50,
															70, 69, 49,
															69, 68, 48,
															68, 67, 47,
															67, 66, 46,
															66, 65, 45,
															65, 64, 44,
															64, 63, 43,
															63, 62, 42,
															62, 61, 41,
															61, 60, 40,
															40, 59, 39,
															59, 58, 38,
															58, 57, 37,
															57, 56, 36,
															56, 55, 35,
															55, 54, 34,
															54, 53, 33,
															53, 52, 32,
															52, 51, 31,
															51, 50, 30,
															50, 49, 29,
															49, 48, 28,
															48, 47, 27,
															47, 46, 26,
															46, 45, 25,
															45, 44, 24,
															44, 43, 23,
															43, 42, 22,
															42, 41, 21,
															41, 40, 20,
															20, 39, 19,
															39, 38, 18,
															38, 37, 17,
															37, 36, 16,
															36, 35, 15,
															35, 34, 14,
															34, 33, 13,
															33, 32, 12,
															32, 31, 11,
															31, 30, 10,
															30, 29, 9,
															29, 28, 8,
															28, 27, 7,
															27, 26, 6,
															26, 25, 5,
															25, 24, 4,
															24, 23, 3,
															23, 22, 2,
															22, 21, 1,
															21, 20, 0
	};


}