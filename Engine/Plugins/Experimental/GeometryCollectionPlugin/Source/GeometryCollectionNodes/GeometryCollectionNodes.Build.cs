// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GeometryCollectionNodes : ModuleRules
	{
        public GeometryCollectionNodes(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"ChaosCore",
					"Chaos",
					"DataflowCore",
					"DataflowEngine",
					"DataflowEnginePlugin",
					"DataflowNodes",
					"GeometryCollectionEngine",
					"GeometryCore",
					"GeometryDataflowNodes",
					"Voronoi",
					"GeometryFramework",
					"MeshDescription",
					"StaticMeshDescription",
					"PlanarCut",
					"Engine",
					"DataflowMedialSkeleton"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] 
				{
					"DynamicMesh",
					"MeshConversion",
					"MeshConversionEngineTypes",
					"FractureEngine",
					"GeometryAlgorithms",
					"SkeletalMeshDescription"
				}
			);

			if (Target.bCompileAgainstEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"DataflowEditor",
						"UnrealEd",
						"SkeletalMeshUtilitiesCommon", // for skeletal mesh conversion
						"ModelingComponentsEditorOnly", // for FSubdividePoly (OpenSubdiv)
					}
					);
			}
		}
	}
}
