// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/CreateColorArrayFromFloatArrayNode.h"
#include "Dataflow/DataflowCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CreateColorArrayFromFloatArrayNode)

void FCreateColorArrayFromFloatArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&ColorArray))
	{
		TArray<FLinearColor> ColorArrayVal;
		const TArray<float>& InFloatArray = GetValue<TArray<float>>(Context, &FloatArray);
		int Num  = InFloatArray.Num();
		if (Num)
		{
			float MaxValue = 0;
			if (bNormalizeInput)
			{
				for (int32 Idx = 0; Idx < Num; ++Idx)
				{
					MaxValue = FMath::Max(MaxValue, InFloatArray[Idx]);
				}
			}
			else
			{
				MaxValue = 1.0;
			}

			if (FMath::IsNearlyZero(MaxValue))
			{
				ColorArrayVal.Init(FLinearColor::Black, Num);
			}
			else
			{
				ColorArrayVal.Init(Color, Num);
				for (int32 Idx = 0; Idx < Num; ++Idx)
				{
					ColorArrayVal[Idx] *= InFloatArray[Idx] / MaxValue;
				}
			}
		}

		SetValue(Context, MoveTemp(ColorArrayVal), &ColorArray);
	}
}
