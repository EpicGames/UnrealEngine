// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/DataflowNodeAndPinTypeColors.h"
#include "Dataflow/DataflowNode.h"
#include "Math/Color.h" 
#include "Dataflow/DataflowNodeColorsRegistry.h"

namespace UE::Dataflow
{
	void RegisterGeometryCollectionNodesColors()
	{
		static const FLinearColor CDefaultNodeBodyTintColor = FLinearColor(0.f, 0.f, 0.f, 0.5f);
		static const float CDefaultWireThickness = 1.5f;
		static const float CDoubleWireThickness = 3.f;

		// Node colors
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("GeometryCollection|Fracture", FLinearColor(1.f, 0.159421f, 0.107247f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("GeometryCollection|Cluster", FLinearColor(1.f, 0.159421f, 0.107247f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Generators", FLinearColor(.4f, 0.8f, 0.f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("GeometryCollection|Development", FLinearColor(1.f, 0.159421f, 0.107247f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("GeometryCollection|Mesh", FLinearColor(1.f, 0.159421f, 0.107247f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("SphereCovering", FLinearColor(1.f, 0.16f, 0.05f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("GeometryCollection|Edit", FLinearColor(0.f, 1.f, 0.05f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Fields", FLinearColor(1.f, 0.527115f, 0.215861f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Materials", FLinearColor(0.000574f, 0.720486f, 0.000574f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Mesh", FLinearColor(0.001103f, 0.695758f, 1.000000f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("GeometryCollection", FLinearColor(0.f, 1.f, 0.115661f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Development", FLinearColor(1.f, 0.f, 0.f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Utilities|String", FLinearColor(0.5f, 0.f, 0.5f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Fracture", FLinearColor(1.f, 0.159421f, 0.107247f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Utilities", FLinearColor(0.387153f, 0.387153f, 0.387153f, 2.935552f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Override", FLinearColor(1.f, 0.4f, 0.4f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("GeometryCollection|Sampling", FLinearColor(.1f, 1.f, 0.6f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("GeometryCollection|Selection", FLinearColor(1.f, 0.367096f, 0.001948f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("GeometryCollection|Texture", FLinearColor(1.f, 0.074733f, 0.048698f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("GeometryCollection|UV", FLinearColor(0.f, 0.926815f, 1.f), CDefaultNodeBodyTintColor);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Convert", FLinearColor(1.f, 1.f, 0.1f), CDefaultNodeBodyTintColor);

		// PinType colors
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FManagedArrayCollection", FLinearColor(0.803819f, 0.803819f, 0.803819f, 1.0f), CDoubleWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("TArray<FString>", FLinearColor(1.0f, 0.172585f, 0.0f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FBox", FLinearColor(0.013575f, 0.770000f, 0.429609f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("FSphere", FLinearColor(0.2f, 0.6f, 1.f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("TArray<TObjectPtr<UMaterial>>", FLinearColor(0.f, 0.2f, 0.0f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("TArray<TObjectPtr<UMaterialInterface>>", FLinearColor(0.f, 0.4f, 0.0f, 1.0f), CDoubleWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("TArray<FGeometryCollectionAutoInstanceMesh>", FLinearColor(0.f, 1.f, 1.0f, 1.0f), CDefaultWireThickness);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY_PIN_SETTINGS_BY_TYPE("TObjectPtr<UStaticMesh>", FLinearColor(0.f, 1.f, 1.f, 1.0f), CDefaultWireThickness);
	}
}
