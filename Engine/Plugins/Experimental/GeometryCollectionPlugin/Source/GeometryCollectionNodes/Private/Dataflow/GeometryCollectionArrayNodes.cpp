// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionArrayNodes.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowAnyTypeRegistry.h"
#include "Dataflow/DataflowTypePolicy.h"

//#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionArrayNodes)

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionArrayNodes)

namespace UE::Dataflow
{
	namespace Private
	{
		static bool AutoConvertArrayToSingleFilter(FName FromType, FName ToType)
		{
			const FName ElementType = FDataflowArrayTypePolicy::GetElementType(FromType);
			return (ToType == ElementType);
		}

		static bool AutoConvertSingleToArrayFilter(FName FromType, FName ToType)
		{
			if (FDataflowArrayTypePolicy::SupportsTypeStatic(FromType))
			{
				// Cannot convert arrays to nested arrays.
				return false;
			}
			const FName ElementType = FDataflowArrayTypePolicy::GetElementType(ToType);
			return (FromType == ElementType);
		}
	}

	void GeometryCollectionArrayNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFloatArrayToIntArrayDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBoolArrayToFaceSelectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFloatArrayToVertexSelectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFloatArrayNormalizeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVectorArrayNormalizeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FUnionIntArraysDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRemoveFloatArrayElementDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFloatArrayComputeStatisticsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRandomizeFloatArrayDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowGetArraySizeNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowGetArrayElementNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowConvertToArrayNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMakeArrayNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMakeManagedArrayCollectionArrayNode);
		

		// Deprecated nodes
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetFloatArrayElementDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetArrayElementDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetNumArrayElementsDataflowNode);

		UE_DATAFLOW_REGISTER_AUTOCONVERT_WITH_FILTER(FDataflowArrayTypes, FDataflowAnyType, FDataflowGetArrayElementNode, &Private::AutoConvertArrayToSingleFilter);
		UE_DATAFLOW_REGISTER_AUTOCONVERT_WITH_FILTER(FDataflowAnyType, FDataflowArrayTypes, FDataflowConvertToArrayNode, &Private::AutoConvertSingleToArrayFilter);
	}
}

void FGetFloatArrayElementDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&FloatValue))
	{
		const TArray<float>& InArray = GetValue(Context, &FloatArray);
		const int32& InIndex = GetValue(Context, &Index);
		const float OutputValue = InArray.IsValidIndex(InIndex) ? InArray[InIndex] : 0.0f;

		SetValue(Context, OutputValue, &FloatValue);
	}
}

void FFloatArrayToIntArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<int32>>(&IntArray))
	{
		TArray<float> FloatVal = GetValue<TArray<float>>(Context, &FloatArray);
		TArray<int32> RetVal; RetVal.SetNumUninitialized(FloatVal.Num());
		if (Function == EFloatArrayToIntArrayFunctionEnum::Dataflow_FloatToInt_Function_Floor)
		{
			for (int32 i = 0; i < FloatVal.Num(); i++)
			{
				RetVal[i] = FMath::FloorToInt32(FloatVal[i]);
			}
		}
		else if (Function == EFloatArrayToIntArrayFunctionEnum::Dataflow_FloatToInt_Function_Ceil)
		{
			for (int32 i = 0; i < FloatVal.Num(); i++)
			{
				RetVal[i] = FMath::CeilToInt32(FloatVal[i]);
			}
		}
		else if (Function == EFloatArrayToIntArrayFunctionEnum::Dataflow_FloatToInt_Function_Round)
		{
			for (int32 i = 0; i < FloatVal.Num(); i++)
			{
				RetVal[i] = FMath::RoundToInt32(FloatVal[i]);
			}
		}
		else if (Function == EFloatArrayToIntArrayFunctionEnum::Dataflow_FloatToInt_Function_Truncate)
		{
			for (int32 i = 0; i < FloatVal.Num(); i++)
			{
				RetVal[i] = int32(FMath::TruncToFloat(FloatVal[i]));
			}
		}

		else if (Function == EFloatArrayToIntArrayFunctionEnum::Dataflow_FloatToInt_NonZeroToIndex)
		{
			int32 RetValIndex = 0;
			for (int32 i = 0; i < FloatVal.Num(); i++)
			{
				if (FloatVal[i] != 0.0)
				{
					RetVal[RetValIndex] = i;
					RetValIndex++;
				}
			}
			RetVal.SetNum(RetValIndex);
		}
		else if (Function == EFloatArrayToIntArrayFunctionEnum::Dataflow_FloatToInt_ZeroToIndex)
		{
			int32 RetValIndex = 0;
			for (int32 i = 0; i < FloatVal.Num(); i++)
			{
				if (FloatVal[i] == 0.0)
				{
					RetVal[RetValIndex] = i;
					RetValIndex++;
				}
			}
			RetVal.SetNum(RetValIndex);
		}

		SetValue(Context, MoveTemp(RetVal), &IntArray);
	}
}

void FGetArrayElementDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FVector>(&Point))
	{
		const TArray<FVector>& Array = GetValue<TArray<FVector>>(Context, &Points);
		if (Array.Num() > 0 && Index >= 0 && Index < Array.Num())
		{
			SetValue(Context, Array[Index], &Point);
			return;
		}

		SetValue(Context, FVector(0.f), &Point);
	}
}

void FGetNumArrayElementsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<int32>(&NumElements))
	{
		if (IsConnected<TArray<float>>(&FloatArray))
		{
			SetValue(Context, GetValue<TArray<float>>(Context, &FloatArray).Num(), &NumElements);
			return;
		}
		else if (IsConnected<TArray<int32>>(&IntArray))
		{
			SetValue(Context, GetValue<TArray<int32>>(Context, &IntArray).Num(), &NumElements);
			return;
		}
		else if (IsConnected<TArray<FVector>>(&Points))
		{
			SetValue(Context, GetValue<TArray<FVector>>(Context, &Points).Num(), &NumElements);
			return;
		}
		else if (IsConnected<TArray<FVector3f>>(&Vector3fArray))
		{
			SetValue(Context, GetValue<TArray<FVector3f>>(Context, &Vector3fArray).Num(), &NumElements);
			return;
		}

		SetValue(Context, 0, &NumElements);
	}
}

void FBoolArrayToFaceSelectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowFaceSelection>(&FaceSelection))
	{
		const TArray<bool>& InBoolAttributeData = GetValue<TArray<bool>>(Context, &BoolAttributeData);

		FDataflowFaceSelection NewFaceSelection;
		NewFaceSelection.Initialize(InBoolAttributeData.Num(), false);
		NewFaceSelection.SetFromArray(InBoolAttributeData);

		SetValue(Context, MoveTemp(NewFaceSelection), &FaceSelection);
	}
}


void FFloatArrayToVertexSelectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowVertexSelection>(&VertexSelection))
	{
		const TArray<float>& InFloatArray = GetValue<TArray<float>>(Context, &FloatArray);

		FDataflowVertexSelection NewVertexSelection;
		NewVertexSelection.Initialize(InFloatArray.Num(), false);

		for (int32 Idx = 0; Idx < InFloatArray.Num(); ++Idx)
		{
			if (Operation == ECompareOperation1Enum::Dataflow_Compare_Equal)
			{
				if (InFloatArray[Idx] == Threshold)
				{
					NewVertexSelection.SetSelected(Idx);
				}
			}
			else if (Operation == ECompareOperation1Enum::Dataflow_Compare_Smaller)
			{
				if (InFloatArray[Idx] < Threshold)
				{
					NewVertexSelection.SetSelected(Idx);
				}
			}
			else if (Operation == ECompareOperation1Enum::Dataflow_Compare_SmallerOrEqual)
			{
				if (InFloatArray[Idx] <= Threshold)
				{
					NewVertexSelection.SetSelected(Idx);
				}
			}
			else if (Operation == ECompareOperation1Enum::Dataflow_Compare_Greater)
			{
				if (InFloatArray[Idx] > Threshold)
				{
					NewVertexSelection.SetSelected(Idx);
				}
			}
			else if (Operation == ECompareOperation1Enum::Dataflow_Compare_GreaterOrEqual)
			{
				if (InFloatArray[Idx] >= Threshold)
				{
					NewVertexSelection.SetSelected(Idx);
				}
			}
		}

		SetValue(Context, MoveTemp(NewVertexSelection), &VertexSelection);
	}
}

void FFloatArrayNormalizeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<float>>(&OutFloatArray))
	{
		const TArray<float>& InInFloatArray = GetValue<TArray<float>>(Context, &InFloatArray);
		const FDataflowVertexSelection& InSelection = GetValue<FDataflowVertexSelection>(Context, &Selection);
		const float InMinRange = GetValue<float>(Context, &MinRange);
		const float InMaxRange = GetValue<float>(Context, &MaxRange);

		const int32 NumElems = InFloatArray.Num();

		TArray<float> NewFloatArray;
		NewFloatArray.Init(false, NumElems);

		if (IsConnected<FDataflowVertexSelection>(&Selection))
		{
			if (InInFloatArray.Num() == InSelection.Num())
			{
				// Compute Min/Max
				float MinValue = 1e9, MaxValue = 1e-9f;

				for (int32 Idx = 0; Idx < NumElems; ++Idx)
				{
					if (InSelection.IsSelected(Idx))
					{
						if (InInFloatArray[Idx] < MinValue)
						{
							MinValue = InInFloatArray[Idx];
						}

						if (InInFloatArray[Idx] > MaxValue)
						{
							MaxValue = InInFloatArray[Idx];
						}
					}
				}

				for (int32 Idx = 0; Idx < NumElems; ++Idx)
				{
					if (InSelection.IsSelected(Idx))
					{
						// Normalize it
						NewFloatArray[Idx] = (NewFloatArray[Idx] - MinValue) / (MaxValue - MinValue);

						// Transform it into (MinRange, MaxRange)
						NewFloatArray[Idx] = InMinRange + NewFloatArray[Idx] * (InMaxRange - InMinRange);
					}
				}

				SetValue(Context, MoveTemp(NewFloatArray), &OutFloatArray);
				return;
			}
		}
		else
		{
			// Compute Min/Max
			float MinValue = 1e9f, MaxValue = 1e-9f;

			for (int32 Idx = 0; Idx < NumElems; ++Idx)
			{
				if (InInFloatArray[Idx] < MinValue)
				{
					MinValue = InInFloatArray[Idx];
				}

				if (InInFloatArray[Idx] > MaxValue)
				{
					MaxValue = InInFloatArray[Idx];
				}
			}

			for (int32 Idx = 0; Idx < NumElems; ++Idx)
			{
				// Normalize it
				NewFloatArray[Idx] = (NewFloatArray[Idx] - MinValue) / (MaxValue - MinValue);

				// Transform it into (MinRange, MaxRange)
				NewFloatArray[Idx] = InMinRange + NewFloatArray[Idx] * (InMaxRange - InMinRange);
			}

			SetValue(Context, MoveTemp(NewFloatArray), &OutFloatArray);
			return;
		}

		SetValue(Context, TArray<float>(), &OutFloatArray);
	}
}


void FVectorArrayNormalizeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<FVector>>(&OutVectorArray))
	{
		const TArray<FVector>& InInVectorArray = GetValue<TArray<FVector>>(Context, &InVectorArray);
		const FDataflowVertexSelection& InSelection = GetValue<FDataflowVertexSelection>(Context, &Selection);
		const float InMagnitude = GetValue<float>(Context, &Magnitude);

		const int32 NumElems = InInVectorArray.Num();

		TArray<FVector> NewVectorArray;
		NewVectorArray.Init(FVector(0.f), NumElems);

		if (IsConnected<FDataflowVertexSelection>(&Selection))
		{
			if (InInVectorArray.Num() == InSelection.Num())
			{
				for (int32 Idx = 0; Idx < NumElems; ++Idx)
				{
					FVector Vector = InInVectorArray[Idx];

					if (InSelection.IsSelected(Idx))
					{
						Vector.Normalize();
						Vector *= InMagnitude;
						NewVectorArray[Idx] = Vector;
					}
				}

				SetValue(Context, MoveTemp(NewVectorArray), &OutVectorArray);
				return;
			}
		}
		else
		{
			for (int32 Idx = 0; Idx < NumElems; ++Idx)
			{
				FVector Vector = InInVectorArray[Idx];

				Vector.Normalize();
				Vector *= InMagnitude;
				NewVectorArray[Idx] = Vector;
			}

			SetValue(Context, MoveTemp(NewVectorArray), &OutVectorArray);
			return;
		}

		SetValue<TArray<FVector>>(Context, TArray<FVector>(), &OutVectorArray);
	}
}

void FUnionIntArraysDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<int32>>(&OutArray))
	{
		TArray<int32> Array1 = GetValue<TArray<int32>>(Context, &InArray1);
		TArray<int32> Array2 = GetValue<TArray<int32>>(Context, &InArray2);
		TArray<int32> OutputArray;
		for (int32 i = 0; i < Array1.Num(); i++)
		{
			OutputArray.AddUnique(Array1[i]);
		}
		for (int32 i = 0; i < Array2.Num(); i++)
		{
			OutputArray.AddUnique(Array2[i]);
		}
		SetValue(Context, MoveTemp(OutputArray), &OutArray);
	}
}

void FRemoveFloatArrayElementDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&FloatArray))
	{
		TArray<float> Array = GetValue<TArray<float>>(Context, &FloatArray);
		if (Array.Num() > 0 && Array.IsValidIndex(Index))
		{
			if (bPreserveOrder)
			{
				Array.RemoveAt(Index);
			}
			else
			{
				Array.RemoveAtSwap(Index, EAllowShrinking::No);
			}

			SetValue(Context, Array, &FloatArray);
			return;
		}

		SetValue(Context, Array, &FloatArray);
	}
}

// -----------------------------------------------------------------------------------------------

static float ComputeMin(const TArray<float>& FloatArray, TArray<int32>& IndexArr)
{
	float MinVal = TNumericLimits<float>::Max();
	for (int32 Idx = 0; Idx < FloatArray.Num(); ++Idx)
	{
		if (FloatArray[Idx] < MinVal)
		{
			MinVal = FloatArray[Idx];
		}
	}

	for (int32 Idx = 0; Idx < FloatArray.Num(); ++Idx)
	{
		if (FloatArray[Idx] == MinVal)
		{
			IndexArr.Add(Idx);
		}
	}

	return MinVal;
}

static float ComputeMax(const TArray<float>& FloatArray, TArray<int32>& IndexArr)
{
	float MaxVal = TNumericLimits<float>::Min();
	for (int32 Idx = 0; Idx < FloatArray.Num(); ++Idx)
	{
		if (FloatArray[Idx] > MaxVal)
		{
			MaxVal = FloatArray[Idx];
		}
	}

	for (int32 Idx = 0; Idx < FloatArray.Num(); ++Idx)
	{
		if (FloatArray[Idx] == MaxVal)
		{
			IndexArr.Add(Idx);
		}
	}

	return MaxVal;
}

static float ComputeMean(const TArray<float>& FloatArray, TArray<int32>& IndexArr)
{
	double SumVal = 0.f;

	for (int32 Idx = 0; Idx < FloatArray.Num(); ++Idx)
	{
		SumVal += FloatArray[Idx];
	}

	return (float)(SumVal / (double)FloatArray.Num());
}

static float ComputeMedian(TArray<float> FloatArray, TArray<int32>& IndexArr)
{
	FloatArray.Sort();
	float MedianVal;
	int32 NumElems = FloatArray.Num();

	if (NumElems % 2 == 0)
	{
		MedianVal = (FloatArray[NumElems / 2 - 1] + FloatArray[NumElems / 2]) / 2.f;

		IndexArr.Add(NumElems / 2 - 1);
		IndexArr.Add(NumElems / 2);
	}
	else
	{
		MedianVal = FloatArray[(NumElems - 1) / 2];

		IndexArr.Add((NumElems - 1) / 2);
	}

	return MedianVal;
}

static float ComputeMode(const TArray<float>& FloatArray, TArray<int32>& IndexArr)
{
	float MaxVal = 0.f;
	int32 MaxCount = 0;

	for (int32 Idx1 = 0; Idx1 < FloatArray.Num(); ++Idx1)
	{
		int32 Count = 0;

		for (int32 Idx2 = 0; Idx2 < FloatArray.Num(); ++Idx2)
		{
			if (FloatArray[Idx2] == FloatArray[Idx1])
			{
				Count++;
			}

			if (Count > MaxCount)
			{
				MaxCount = Count;
				MaxVal = FloatArray[Idx1];
			}
		}
	}

	for (int32 Idx = 0; Idx < FloatArray.Num(); ++Idx)
	{
		if (FloatArray[Idx] == MaxVal)
		{
			IndexArr.Add(Idx);
		}
	}

	return MaxVal;
}

static float ComputeSum(const TArray<float>& FloatArray, TArray<int32>& IndexArr)
{
	double SumVal = 0.f;

	for (int32 Idx = 0; Idx < FloatArray.Num(); ++Idx)
	{
		SumVal += FloatArray[Idx];
	}

	return (float)SumVal;
}

// -----------------------------------------------------------------------------------------------

void FFloatArrayComputeStatisticsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Value) || Out->IsA(&Indices))
	{
		const TArray<float>& InFloatArray = GetValue(Context, &FloatArray);
		const FDataflowTransformSelection& InTransformSelection = GetValue(Context, &TransformSelection);

		TArray<float> Array;
		if (IsConnected<FDataflowTransformSelection>(&TransformSelection) && InTransformSelection.Num() == InFloatArray.Num())
		{
			for (int32 Idx = 0; Idx < InFloatArray.Num(); ++Idx)
			{
				if (InTransformSelection.IsSelected(Idx))
				{
					Array.Add(InFloatArray[Idx]);
				}
			}
		}
		else
		{
			for (int32 Idx = 0; Idx < InFloatArray.Num(); ++Idx)
			{
				Array.Add(InFloatArray[Idx]);
			}
		}

		float OutValue;
		TArray<int32> OutIndices;

		switch (OperationName)
		{
		case EStatisticsOperationEnum::Dataflow_EStatisticsOperationEnum_Min:
			OutValue = ComputeMin(Array, OutIndices);
			break;
		case EStatisticsOperationEnum::Dataflow_EStatisticsOperationEnum_Max:
			OutValue = ComputeMax(Array, OutIndices);
			break;
		case EStatisticsOperationEnum::Dataflow_EStatisticsOperationEnum_Mean:
			OutValue = ComputeMean(Array, OutIndices);
			break;
		case EStatisticsOperationEnum::Dataflow_EStatisticsOperationEnum_Median:
			OutValue = ComputeMedian(Array, OutIndices);
			break;
		case EStatisticsOperationEnum::Dataflow_EStatisticsOperationEnum_Mode:
			OutValue = ComputeMode(Array, OutIndices);
			break;
		case EStatisticsOperationEnum::Dataflow_EStatisticsOperationEnum_Sum:
			OutValue = ComputeSum(Array, OutIndices);
			break;
		}

		SetValue(Context, OutValue, &Value);
		SetValue(Context, OutIndices, &Indices);
	}
}

void FRandomizeFloatArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&FloatArray))
	{
		const TArray<float> InFloatArray = GetValue(Context, &FloatArray);
		const float InRandomRangeMin = GetValue(Context, &RandomRangeMin);
		const float InRandomRangeMax = GetValue(Context, &RandomRangeMax);
		const int32 InRandomSeed = GetValue(Context, &RandomSeed);
		TArray<float> OutFloatArray;

		FRandomStream RandStream(InRandomSeed);

		const int32 NumPoints = InFloatArray.Num();
		if (NumPoints > 0)
		{
			OutFloatArray.Reserve(NumPoints);
			for (int32 Idx = 0; Idx < NumPoints; ++Idx)
			{
				OutFloatArray.Emplace(RandStream.FRandRange(InRandomRangeMin, InRandomRangeMax));
			}
		}

		SetValue(Context, MoveTemp(OutFloatArray), &FloatArray);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////

FDataflowGetArraySizeNode::FDataflowGetArraySizeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Array);
	RegisterOutputConnection(&Size);
}

void FDataflowGetArraySizeNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Size))
	{
		const UE::Dataflow::FContextValue InArray = GetValue(Context, &Array);
		SetValue(Context, InArray.Num(), &Size);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////

FDataflowGetArrayElementNode::FDataflowGetArrayElementNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	static FName ArrayElementDependencyGroup("ArrayElement");
	
	RegisterInputConnection(&Array)
		.SetTypeDependencyGroup(ArrayElementDependencyGroup);
	RegisterInputConnection(&Index);
	RegisterOutputConnection(&Array, &Array)
		.SetTypeDependencyGroup(ArrayElementDependencyGroup);
	RegisterOutputConnection(&Element)
		.SetTypeDependencyGroup(ArrayElementDependencyGroup);
}

FName FDataflowGetArrayElementNode::GetDependentConnectionType(const FDataflowConnection& SourceConnection, const FDataflowConnection& DependentConnection) const
{
	if (SourceConnection.IsA(&Array) && DependentConnection.IsA(&Element))
	{
		return FDataflowArrayTypePolicy::GetElementType(SourceConnection.GetType());
	}
	else if (SourceConnection.IsA(&Element) && DependentConnection.IsA(&Array))
	{
		return FDataflowArrayTypePolicy::GetArrayType(SourceConnection.GetType());
	}
	// default for all the rest 
	return FDataflowNode::GetDependentConnectionType(SourceConnection, DependentConnection);
}

void FDataflowGetArrayElementNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Array))
	{
		ForwardInput(Context, &Array, &Array);
	}
	else if (Out->IsA(&Element))
	{
		const UE::Dataflow::FContextValue InArray = GetValue(Context, &Array);
		const int32 InIndex = GetValue(Context, &Index);
		UE::Dataflow::FContextValue OutElement = InArray.GetAt(InIndex);
		SetValue(Context, MoveTemp(OutElement), &Element);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////
FDataflowConvertToArrayNode::FDataflowConvertToArrayNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	static FName ArrayElementDependencyGroup("ArrayElement");
	
	RegisterInputConnection(&Element)
		.SetTypeDependencyGroup(ArrayElementDependencyGroup);

	RegisterOutputConnection(&Array)
		.SetTypeDependencyGroup(ArrayElementDependencyGroup);
}

FName FDataflowConvertToArrayNode::GetDependentConnectionType(const FDataflowConnection& SourceConnection, const FDataflowConnection& DependentConnection) const
{
	if (SourceConnection.IsA(&Array) && DependentConnection.IsA(&Element))
	{
		return FDataflowArrayTypePolicy::GetElementType(SourceConnection.GetType());
	}
	else if (SourceConnection.IsA(&Element) && DependentConnection.IsA(&Array))
	{
		if (FDataflowArrayTypePolicy::SupportsTypeStatic(SourceConnection.GetType()))
		{
			// Special case: Array elements are left unchanged.
			return SourceConnection.GetType();
		}

		return FDataflowArrayTypePolicy::GetArrayType(SourceConnection.GetType());
	}
	// default for all the rest 
	return FDataflowNode::GetDependentConnectionType(SourceConnection, DependentConnection);
}

void FDataflowConvertToArrayNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Array))
	{
		const UE::Dataflow::FContextValue InElement = GetValue(Context, &Element);
		UE::Dataflow::FContextValue OutArray = InElement.ToArray();
		SetValue(Context, MoveTemp(OutArray), &Array);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////
FDataflowMakeArrayNode::FDataflowMakeArrayNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{	

	RegisterOutputConnection(&Array)
		.SetTypeDependencyGroup(ArrayElementDependencyGroup);

	// Add initial pins
	for (int32 Index = 0; Index < NumInitialElements; ++Index)
	{
		AddPins();
	}
	check(GetNumInputs() == NumRequiredInputs + NumInitialElements);  // Update NumRequiredInputs if you add more Inputs. This is used by Serialize.
}

FName FDataflowMakeArrayNode::GetDependentConnectionType(const FDataflowConnection& SourceConnection, const FDataflowConnection& DependentConnection) const
{
	auto IsElementConnection = [this](const FDataflowConnection& Connection)
		{
			for (const FDataflowAllTypes& Element : Elements)
			{
				if (Connection.IsA(&Element))
				{
					return true;
				}
			}
			return false;
		};

	if (SourceConnection.IsA(&Array) && IsElementConnection(DependentConnection))
	{
		return FDataflowArrayTypePolicy::GetElementType(SourceConnection.GetType());
	}
	else if (DependentConnection.IsA(&Array) && IsElementConnection(SourceConnection))
	{
		if (FDataflowArrayTypePolicy::SupportsTypeStatic(SourceConnection.GetType()))
		{
			// Special case: Array elements are left unchanged.
			return SourceConnection.GetType();
		}
		return FDataflowArrayTypePolicy::GetArrayType(SourceConnection.GetType());
	}

	// default for all the rest 
	return FDataflowNode::GetDependentConnectionType(SourceConnection, DependentConnection);
}

void FDataflowMakeArrayNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Array))
	{
		TArray<const UE::Dataflow::FContextValue> InElements;

		InElements.Reserve(Elements.Num());

		bool bIsAppendingArrays = false;
		for (int32 Index = 0; Index < Elements.Num(); ++Index)
		{
			if (IsConnected(GetConnectionReference(Index)))
			{
				const UE::Dataflow::FContextValue& InElement =  InElements.Emplace_GetRef(GetValue(Context, GetConnectionReference(Index)));
				if (InElement.IsArray())
				{
					bIsAppendingArrays = true;
				}
			}
		}

		if (bIsAppendingArrays)
		{
			UE::Dataflow::FContextValue OutArray = UE::Dataflow::FContextValue::AppendArrays(Context, InElements);
			SetValue(Context, MoveTemp(OutArray), &Array);
		}
		else
		{
			UE::Dataflow::FContextValue OutArray = UE::Dataflow::FContextValue::MakeArray(Context, InElements);
			SetValue(Context, MoveTemp(OutArray), &Array);
		}
	}
}

TArray<UE::Dataflow::FPin> FDataflowMakeArrayNode::AddPins()
{
	const int32 Index = Elements.AddDefaulted();
	FDataflowInput& Input = RegisterInputArrayConnection(GetConnectionReference(Index));
	if (Index > 0)
	{
		if (const FDataflowInput* Input0 = FindInput(GetConnectionReference(0)))
		{
			if (Input0->HasConcreteType())
			{
				SetConnectionConcreteType(&Input, Input0->GetType(), ArrayElementDependencyGroup);
			}
		}
	}
	Input.SetTypeDependencyGroup(ArrayElementDependencyGroup);
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}

TArray<UE::Dataflow::FPin> FDataflowMakeArrayNode::GetPinsToRemove() const
{
	const int32 Index = Elements.Num() - 1;
	check(Elements.IsValidIndex(Index));
	if (const FDataflowInput* const Input = FindInput(GetConnectionReference(Index)))
	{
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	}
	return Super::GetPinsToRemove();
}

void FDataflowMakeArrayNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = Elements.Num() - 1;
	check(Elements.IsValidIndex(Index));
#if DO_CHECK
	const FDataflowInput* const Input = FindInput(GetConnectionReference(Index));
	check(Input);
	check(Input->GetName() == Pin.Name);
	check(Input->GetType() == Pin.Type);
#endif
	Elements.SetNum(Index);

	return Super::OnPinRemoved(Pin);
}

UE::Dataflow::TConnectionReference<FDataflowAllTypes> FDataflowMakeArrayNode::GetConnectionReference(int32 Index) const
{
	return { &Elements[Index], Index, &Elements };
}

void FDataflowMakeArrayNode::PostSerialize(const FArchive& Ar)
{
	// because we add pins we need to make sure we restore them when loading
	// to make sure they can get properly reconnected
	if (Ar.IsLoading())
	{
		for (int32 Index = 0; Index < NumInitialElements; ++Index)
		{
			check(FindInput(GetConnectionReference(Index)));
		}

		for (int32 Index = NumInitialElements; Index < Elements.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetConnectionReference(Index))
				.SetTypeDependencyGroup(ArrayElementDependencyGroup);
		}
		if (Ar.IsTransacting())
		{
			const int32 OrigNumRegisteredInputs = GetNumInputs();
			check(OrigNumRegisteredInputs >= NumRequiredInputs + NumInitialElements);
			const int32 OrigNumElements = Elements.Num();
			const int32 OrigNumRegisteredElements = (OrigNumRegisteredInputs - NumRequiredInputs);
			if (OrigNumRegisteredElements > OrigNumElements)
			{
				// Inputs have been removed.
				// Temporarily expand Collections so we can get connection references.
				Elements.SetNum(OrigNumRegisteredElements);
				for (int32 Index = OrigNumElements; Index < Elements.Num(); ++Index)
				{
					UnregisterInputConnection(GetConnectionReference(Index));
				}
				Elements.SetNum(OrigNumElements);
			}
		}
		else
		{
			ensureAlways(Elements.Num() + NumRequiredInputs == GetNumInputs());
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////
FDataflowMakeManagedArrayCollectionArrayNode::FDataflowMakeManagedArrayCollectionArrayNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	static FName ArrayElementDependencyGroup("ArrayElement");

	RegisterInputConnection(&Array);
	RegisterInputConnection(&Element);
	RegisterOutputConnection(&Array, &Array);
}

void FDataflowMakeManagedArrayCollectionArrayNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Array))
	{
		if (IsConnected(&Element))
		{
			TArray<FManagedArrayCollection> OutArray = GetValue(Context, &Array);
			OutArray.Emplace(GetValue(Context, &Element));
			SetValue(Context, MoveTemp(OutArray), &Array);
			return;
		}

		SafeForwardInput(Context, &Array, &Array);
	}
}
