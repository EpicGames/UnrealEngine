// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionAssetNodes.h"
#include "Dataflow/DataflowCore.h"

#include "Engine/StaticMesh.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "GeometryCollection/Facades/CollectionInstancedMeshFacade.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "Components/StaticMeshComponent.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "PreviewScene.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionAssetNodes)

namespace UE::Dataflow
{
	void GeometryCollectionEngineAssetNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGeometryCollectionTerminalDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetGeometryCollectionAssetDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetGeometryCollectionSourcesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCreateGeometryCollectionFromSourcesDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGeometryCollectionToCollectionDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBlueprintToCollectionDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeRootProxyMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeRootProxyMeshArrayDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAddRootProxyMeshToArrayDataflowNode);

		// deprecated nodes (need to stay registered)
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGeometryCollectionTerminalDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCreateGeometryCollectionFromSourcesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGeometryCollectionToCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBlueprintToCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAssetRandomSeedDataflowNode);
	}
}

// ===========================================================================================================================
FMakeRootProxyMeshDataflowNode::FMakeRootProxyMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&Transform);
	RegisterInputConnection(&OverrideMaterials);

	RegisterOutputConnection(&RootProxyMesh);
}

void FMakeRootProxyMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&RootProxyMesh))
	{
		FDataflowRootProxyMesh OutRootProxyMesh;
		OutRootProxyMesh.Mesh = GetValue(Context, &Mesh);
		OutRootProxyMesh.Transform = GetValue(Context, &Transform);
		OutRootProxyMesh.OverrideMaterials = GetValue(Context, &OverrideMaterials);

		SetValue(Context, MoveTemp(OutRootProxyMesh), &RootProxyMesh);
	}
}

// ===========================================================================================================================
FMakeRootProxyMeshArrayDataflowNode::FMakeRootProxyMeshArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&RootProxyMeshes);
}

void FMakeRootProxyMeshArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&RootProxyMeshes))
	{
		SetValue(Context, RootProxyMeshes, &RootProxyMeshes);
	}
}

// ===========================================================================================================================
FAddRootProxyMeshToArrayDataflowNode::FAddRootProxyMeshToArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&RootProxyMeshes);
	RegisterInputConnection(&RootProxyMesh);

	RegisterOutputConnection(&RootProxyMeshes, &RootProxyMeshes);
}

void FAddRootProxyMeshToArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&RootProxyMeshes))
	{
		const FDataflowRootProxyMesh& InRootProxyMesh = GetValue(Context, &RootProxyMesh);

		TArray<FDataflowRootProxyMesh> OutArray;
		OutArray.Add(InRootProxyMesh);

		SetValue(Context, MoveTemp(OutArray), &RootProxyMeshes);
	}
}

// ===========================================================================================================================

FGeometryCollectionTerminalDataflowNode_v2::FGeometryCollectionTerminalDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Materials);
	RegisterInputConnection(&InstancedMeshes);
	RegisterInputConnection(&RootProxyMeshes);

	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Materials, &Materials);
	RegisterOutputConnection(&InstancedMeshes, &InstancedMeshes);
}

void FGeometryCollectionTerminalDataflowNode_v2::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	using FGeometryCollectionPtr = TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe>;
	using FMaterialArray = TArray<TObjectPtr<UMaterialInterface>>;
	using FInstancedMeshesArray = TArray<FGeometryCollectionAutoInstanceMesh>;
	using FRootProxyMeshesArray = TArray<FDataflowRootProxyMesh>;

	if (UGeometryCollection* CollectionAsset = Cast<UGeometryCollection>(Asset.Get()))
	{
		// make sure to always reset the root proxies even if the collection is empty 
		if (IsConnected(&RootProxyMeshes))
		{
			CollectionAsset->RootProxyData.Reset();
		}

		if (FGeometryCollectionPtr GeometryCollection = CollectionAsset->GetGeometryCollection())
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
			const FMaterialArray& InMaterials = GetValue(Context, &Materials);
			const FInstancedMeshesArray& InInstancedMeshes = GetValue(Context, &InstancedMeshes);
			const FRootProxyMeshesArray& InRootProxyMeshes = GetValue(Context, &RootProxyMeshes);

			constexpr bool bHasInternalMaterial = false; // with dataflow there's no assumption of internal materials
			CollectionAsset->ResetFrom(InCollection, InMaterials, bHasInternalMaterial);

			CollectionAsset->SetAutoInstanceMeshes(InInstancedMeshes);

			if (IsConnected(&RootProxyMeshes))
			{
				const GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(*GeometryCollection);
				const FTransform RootTransform = TransformFacade.GetRootTransform();

				FGeometryCollectionProxyMeshData& Data = CollectionAsset->RootProxyData;
				for (const FDataflowRootProxyMesh& ProxyMesh : InRootProxyMeshes)
				{
					Data.ProxyMeshes.Add(ProxyMesh.Mesh);
					// root proxy transforms are relative to the root transform but FDataflowRootProxyMesh is in component space 
					Data.MeshTransforms.Add(FTransform3f(ProxyMesh.Transform.GetRelativeTransform(RootTransform)));
					Data.MeshOverrideMaterials.AddDefaulted_GetRef().Materials = ProxyMesh.OverrideMaterials;
				}
			}

#if WITH_EDITOR
			// make sure we rebuild the render data when we are done setting everything 
			CollectionAsset->RebuildRenderData();
			// also make sure all components using it are getting a notification about it
			CollectionAsset->PropagateTransformUpdateToComponents();
#endif
		}
	}
}

void FGeometryCollectionTerminalDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context) const
{
	// simply forward all inputs to corresponding outputs
	SafeForwardInput(Context, &Collection, &Collection);
	SafeForwardInput(Context, &Materials, &Materials);
	SafeForwardInput(Context, &InstancedMeshes, &InstancedMeshes);
}

// ===========================================================================================================================
// DEPRECATED 5.6 : see FGeometryCollectionTerminalDataflowNode_v2
FGeometryCollectionTerminalDataflowNode::FGeometryCollectionTerminalDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterInputConnection(&Materials);
	RegisterInputConnection(&MaterialInstances);
	RegisterOutputConnection(&Materials, &Materials);
	RegisterOutputConnection(&MaterialInstances, &MaterialInstances);
	RegisterInputConnection(&InstancedMeshes);
	RegisterOutputConnection(&InstancedMeshes, &InstancedMeshes);
}



void FGeometryCollectionTerminalDataflowNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	using FGeometryCollectionPtr = TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe>;
	using FMaterialArray = TArray<TObjectPtr<UMaterial>>;
	using FMaterialInstanceArray = TArray<TObjectPtr<UMaterialInterface>>;
	using FInstancedMeshesArray = TArray<FGeometryCollectionAutoInstanceMesh>;

	if (UGeometryCollection* CollectionAsset = Cast<UGeometryCollection>(Asset.Get()))
	{
		if (FGeometryCollectionPtr GeometryCollection = CollectionAsset->GetGeometryCollection())
		{
			// need to make a copy since we may need to add attributes
			FManagedArrayCollection InCollection = GetValue(Context, &Collection);
			
			// for now make sure we have the right interfaces setup on the source collection so that it does not trigger an ensure during the call to ResetFrom 
			// (because of the discrepancy between convex properties attributes as they are created by default in FGeometryCollection but may be missing from the input collection )
			// todo(dataflow) we should make this more automatic in the future
			FGeometryCollectionConvexPropertiesInterface ConvexPropertiesInterface(&InCollection);
			ConvexPropertiesInterface.InitializeInterface();

			const FMaterialArray& InMaterials = GetValue(Context, &Materials);
			const FMaterialInstanceArray& InMaterialInstances = GetValue(Context, &MaterialInstances);
			const FInstancedMeshesArray& InInstancedMeshes = GetValue(Context, &InstancedMeshes);

			const bool bHasInternalMaterial = false; // with data flow there's no assumption of internal materials
			if (InMaterialInstances.Num() > 0)
			{
				CollectionAsset->ResetFrom(InCollection, InMaterialInstances, false);
			}
			else
			{
				CollectionAsset->ResetFrom(InCollection, InMaterials, false);
			}
			CollectionAsset->SetAutoInstanceMeshes(InInstancedMeshes);

#if WITH_EDITOR
			// make sure we rebuild the render data when we are done setting everything 
			CollectionAsset->RebuildRenderData();
			// also make sure all components using it are getting a notification about it
			CollectionAsset->PropagateTransformUpdateToComponents();
#endif
		}
	}
}

void FGeometryCollectionTerminalDataflowNode::Evaluate(UE::Dataflow::FContext& Context) const
{
	// simply forward all inputs to corresponding outputs
	SafeForwardInput(Context, &Collection, &Collection);
	SafeForwardInput(Context, &Materials, &Materials);
	SafeForwardInput(Context, &MaterialInstances, &MaterialInstances);
	SafeForwardInput(Context, &InstancedMeshes, &InstancedMeshes);
}

// ===========================================================================================================================

FGetGeometryCollectionAssetDataflowNode::FGetGeometryCollectionAssetDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Asset);
}

void FGetGeometryCollectionAssetDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	ensure(Out->IsA(&Asset));

	TObjectPtr<UGeometryCollection> CollectionAsset(nullptr);
	if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
	{
		CollectionAsset = Cast<UGeometryCollection>(EngineContext->Owner);
	}
	SetValue(Context, CollectionAsset, &Asset);
}

// ===========================================================================================================================

FGetGeometryCollectionSourcesDataflowNode::FGetGeometryCollectionSourcesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
		: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Asset);
	RegisterOutputConnection(&Sources);
}

void FGetGeometryCollectionSourcesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	ensure(Out->IsA(&Sources));

	TArray<FGeometryCollectionSource> OutSources;
	
	if (const TObjectPtr<const UGeometryCollection> InAsset = GetValue(Context, &Asset))
	{
#if WITH_EDITORONLY_DATA
		OutSources = InAsset->GeometrySource; 
#else
		ensureMsgf(false, TEXT("FGetGeometryCollectionSourcesDataflowNode - GeometrySource is only available in editor, returning an empty array"));
#endif

	}

	SetValue(Context, OutSources, &Sources);
}

// ===========================================================================================================================
// DEPRECATED 5.6 : see FCreateGeometryCollectionFromSourcesDataflowNode_v2

FCreateGeometryCollectionFromSourcesDataflowNode::FCreateGeometryCollectionFromSourcesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
		: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Sources);
	RegisterOutputConnection(&Collection);
	RegisterOutputConnection(&Materials);
	RegisterOutputConnection(&MaterialInstances);
	RegisterOutputConnection(&InstancedMeshes);
}

void FCreateGeometryCollectionFromSourcesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	ensure(Out->IsA(&Collection) || Out->IsA(&Materials) || Out->IsA(&MaterialInstances) || Out->IsA(&InstancedMeshes));
	
	const TArray<FGeometryCollectionSource>& InSources = GetValue(Context, &Sources);

	FGeometryCollection OutCollection;
	TArray<TObjectPtr<UMaterialInterface>> OutMaterialInstances;
	TArray<FGeometryCollectionAutoInstanceMesh> OutInstancedMeshes;

	// make sure we have an attribute for instanced meshes
	GeometryCollection::Facades::FCollectionInstancedMeshFacade InstancedMeshFacade(OutCollection);
	InstancedMeshFacade.DefineSchema();

	constexpr bool bReindexMaterialsInLoop = false;
	for (int32 SourceIndex = 0; SourceIndex < InSources.Num(); SourceIndex++)
	{
		const FGeometryCollectionSource& Source = InSources[SourceIndex];
		const int32 NumTransformsBeforeAppending = OutCollection.NumElements(FGeometryCollection::TransformGroup);

		// todo: change AppendGeometryCollectionSource to take a FManagedArrayCollection so we could move the collection when assigning it to the output
		FGeometryCollectionEngineConversion::AppendGeometryCollectionSource(Source, OutCollection, MutableView(OutMaterialInstances), bReindexMaterialsInLoop);

		// todo(chaos) if the source is a geometry collection this will not work properly 
		FGeometryCollectionAutoInstanceMesh InstancedMesh;
		InstancedMesh.Mesh = Cast<UStaticMesh>(Source.SourceGeometryObject.TryLoad());
		InstancedMesh.Materials = Source.SourceMaterial;
		InstancedMesh.NumInstances = 0;
		InstancedMesh.CustomData.Reset();

		int32 InstancedMeshIndex = OutInstancedMeshes.Find(InstancedMesh);
		if (InstancedMeshIndex == INDEX_NONE)
		{
			InstancedMeshIndex = OutInstancedMeshes.Add(InstancedMesh);
		}
		OutInstancedMeshes[InstancedMeshIndex].NumInstances++;
		OutInstancedMeshes[InstancedMeshIndex].CustomData.Append(Source.InstanceCustomData);

		// add the instanced mesh  for all the newly added transforms 
		const int32 NumTransformsAfterAppending = OutCollection.NumElements(FGeometryCollection::TransformGroup);
		//ensure((NumTransformsAfterAppending - NumTransformsBeforeAppending) == 1);
		for (int32 TransformIndex = NumTransformsBeforeAppending; TransformIndex < NumTransformsAfterAppending; TransformIndex++)
		{
			InstancedMeshFacade.SetIndex(TransformIndex, InstancedMeshIndex);
		}
	}
	if (bReindexMaterialsInLoop == false)
	{
		OutCollection.ReindexMaterials();
	}

	// add the instanced mesh indices

	const int32 NumTransforms = InstancedMeshFacade.GetNumIndices();
	for (int32 TransformIndex = 0; TransformIndex < NumTransforms; TransformIndex++)
	{
	}

	// make sure we have only one root
	if (FGeometryCollectionClusteringUtility::ContainsMultipleRootBones(&OutCollection))
	{
		FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(&OutCollection);
	}

	// make sure we have a level attribute
	Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(OutCollection);
	HierarchyFacade.GenerateLevelAttribute();

	TArray<TObjectPtr<UMaterial>> OutMaterials;
	FGeometryCollectionEngineConversion::GetMaterialsFromInstances(OutMaterialInstances, OutMaterials);

	// we have to make a copy since we have generated a FGeometryCollection which is inherited from FManagedArrayCollection
	SetValue(Context, static_cast<const FManagedArrayCollection&>(OutCollection), &Collection);
	SetValue(Context, MoveTemp(OutMaterials), &Materials);
	SetValue(Context, MoveTemp(OutMaterialInstances), &MaterialInstances);
	SetValue(Context, MoveTemp(OutInstancedMeshes), &InstancedMeshes);
}

// ===========================================================================================================================

FCreateGeometryCollectionFromSourcesDataflowNode_v2::FCreateGeometryCollectionFromSourcesDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Sources);
	RegisterOutputConnection(&Collection);
	RegisterOutputConnection(&Materials);
	RegisterOutputConnection(&InstancedMeshes);
	RegisterOutputConnection(&RootProxyMeshes);
}

void FCreateGeometryCollectionFromSourcesDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	ensure(Out->IsA(&Collection) || Out->IsA(&Materials) || Out->IsA(&InstancedMeshes) || Out->IsA(&RootProxyMeshes));

	TArray<FGeometryCollectionSource> InSources;
	if (IsConnected(&Sources))
	{
		InSources = GetValue(Context, &Sources);
	}
	else
	{
		// not connected let's try to get them from the current asset
		if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
		{
			if (UGeometryCollection* CollectionAsset = Cast<UGeometryCollection>(EngineContext->Owner))
			{
#if WITH_EDITORONLY_DATA
				InSources = CollectionAsset->GeometrySource;
#else
				ensureMsgf(false, TEXT("FCreateGeometryCollectionFromSourcesDataflowNode - GeometrySource is only available in editor, returning an empty array"));
#endif
			}
		}
	}
	if (InSources.IsEmpty())
	{
		Context.Warning(TEXT("Geometry source array is empty, this will result in an empty collection"), this, Out);
	}

	FGeometryCollection OutCollection;
	TArray<TObjectPtr<UMaterialInterface>> OutMaterials;
	TArray<FGeometryCollectionAutoInstanceMesh> OutInstancedMeshes;
	TArray<FDataflowRootProxyMesh> OutRootProxyMeshes;

	// make sure we have an attribute for instanced meshes
	GeometryCollection::Facades::FCollectionInstancedMeshFacade InstancedMeshFacade(OutCollection);
	InstancedMeshFacade.DefineSchema();

	constexpr bool bReindexMaterialsInLoop = false;
	for (int32 SourceIndex = 0; SourceIndex < InSources.Num(); SourceIndex++)
	{
		const FGeometryCollectionSource& Source = InSources[SourceIndex];
		const int32 NumTransformsBeforeAppending = OutCollection.NumElements(FGeometryCollection::TransformGroup);

		// todo: change AppendGeometryCollectionSource to take a FManagedArrayCollection so we could move the collection when assigning it to the output
		FGeometryCollectionEngineConversion::AppendGeometryCollectionSource(Source, OutCollection, MutableView(OutMaterials), bReindexMaterialsInLoop);

		UStaticMesh* StaticMeshObject = Cast<UStaticMesh>(Source.SourceGeometryObject.TryLoad());

		// todo(chaos) if the source is a geometry collection this will not work properly 
		FGeometryCollectionAutoInstanceMesh InstancedMesh;
		InstancedMesh.Mesh = StaticMeshObject;
		InstancedMesh.Materials = Source.SourceMaterial;
		InstancedMesh.NumInstances = 0;
		InstancedMesh.CustomData.Reset();

		int32 InstancedMeshIndex = OutInstancedMeshes.Find(InstancedMesh);
		if (InstancedMeshIndex == INDEX_NONE)
		{
			InstancedMeshIndex = OutInstancedMeshes.Add(InstancedMesh);
		}
		OutInstancedMeshes[InstancedMeshIndex].NumInstances++;
		OutInstancedMeshes[InstancedMeshIndex].CustomData.Append(Source.InstanceCustomData);

		// add the instanced mesh  for all the newly added transforms 
		const int32 NumTransformsAfterAppending = OutCollection.NumElements(FGeometryCollection::TransformGroup);
		//ensure((NumTransformsAfterAppending - NumTransformsBeforeAppending) == 1);
		for (int32 TransformIndex = NumTransformsBeforeAppending; TransformIndex < NumTransformsAfterAppending; TransformIndex++)
		{
			InstancedMeshFacade.SetIndex(TransformIndex, InstancedMeshIndex);
		}

		// Root proxy meshes
		FDataflowRootProxyMesh RootProxyMesh;
		RootProxyMesh.Mesh = StaticMeshObject;
		RootProxyMesh.Transform = Source.LocalTransform;
		RootProxyMesh.OverrideMaterials = Source.SourceMaterial;
		OutRootProxyMeshes.Add(RootProxyMesh);

	}
	if (bReindexMaterialsInLoop == false)
	{
		OutCollection.ReindexMaterials();
	}

	// make sure we have only one root
	if (FGeometryCollectionClusteringUtility::ContainsMultipleRootBones(&OutCollection))
	{
		FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(&OutCollection);
	}

	// make sure we have a level attribute
	Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(OutCollection);
	HierarchyFacade.GenerateLevelAttribute();

	// we have to make a copy since we have generated a FGeometryCollection which is inherited from FManagedArrayCollection
	SetValue(Context, static_cast<const FManagedArrayCollection&>(OutCollection), &Collection);
	SetValue(Context, MoveTemp(OutMaterials), &Materials);
	SetValue(Context, MoveTemp(OutInstancedMeshes), &InstancedMeshes);
	SetValue(Context, MoveTemp(OutRootProxyMeshes), &RootProxyMeshes);
}

// ===========================================================================================================================
// DEPRECATED 5.6 : see FGeometryCollectionToCollectionDataflowNode_v2

FGeometryCollectionToCollectionDataflowNode::FGeometryCollectionToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Collection);
	RegisterOutputConnection(&Materials);
	RegisterOutputConnection(&MaterialInstances);
	RegisterOutputConnection(&InstancedMeshes);
}

void FGeometryCollectionToCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	ensure(Out->IsA(&Collection) || Out->IsA(&Materials) || Out->IsA(&MaterialInstances) || Out->IsA(&InstancedMeshes));

	FManagedArrayCollection OutCollection;
	TArray<TObjectPtr<UMaterialInterface>> OutMaterialInstances;
	TArray<FGeometryCollectionAutoInstanceMesh> OutInstancedMeshes;

	if (GeometryCollection)
	{
		FGeometryCollectionEngineConversion::ConvertGeometryCollectionToGeometryCollection(GeometryCollection, OutCollection, OutMaterialInstances, OutInstancedMeshes);
	}

	TArray<TObjectPtr<UMaterial>> OutMaterials;
	FGeometryCollectionEngineConversion::GetMaterialsFromInstances(OutMaterialInstances, OutMaterials);

	// Set Outputs
	SetValue(Context, MoveTemp(OutCollection), &Collection);
	SetValue(Context, MoveTemp(OutMaterials), &Materials);
	SetValue(Context, MoveTemp(OutMaterialInstances), &MaterialInstances);
	SetValue(Context, MoveTemp(OutInstancedMeshes), &InstancedMeshes);
}

// ===========================================================================================================================

FGeometryCollectionToCollectionDataflowNode_v2::FGeometryCollectionToCollectionDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&GeometryCollection);

	RegisterOutputConnection(&Collection);
	RegisterOutputConnection(&Materials);
	RegisterOutputConnection(&InstancedMeshes);
	RegisterOutputConnection(&RootProxyMeshes);
}

void FGeometryCollectionToCollectionDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	ensure(Out->IsA(&Collection) || Out->IsA(&Materials) || Out->IsA(&InstancedMeshes) || Out->IsA(&RootProxyMeshes));

	FManagedArrayCollection OutCollection;
	TArray<TObjectPtr<UMaterialInterface>> OutMaterials;
	TArray<FGeometryCollectionAutoInstanceMesh> OutInstancedMeshes;
	TArray<FDataflowRootProxyMesh> OutRootProxyMeshes;

	const TObjectPtr<UGeometryCollection> InGeometryCollection = GetValue(Context, &GeometryCollection);
	if (InGeometryCollection)
	{
		FGeometryCollectionEngineConversion::ConvertGeometryCollectionToGeometryCollection(InGeometryCollection, OutCollection, OutMaterials, OutInstancedMeshes);

		FTransform RootTransform = FTransform::Identity;
		if (InGeometryCollection->GetGeometryCollection())
		{
			const GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(*InGeometryCollection->GetGeometryCollection());
			RootTransform = TransformFacade.GetRootTransform();
		}

		const FGeometryCollectionProxyMeshData& RootProxyData = InGeometryCollection->RootProxyData;
		for (int32 MeshIndex = 0; MeshIndex < InGeometryCollection->RootProxyData.ProxyMeshes.Num(); MeshIndex++)
		{
			FDataflowRootProxyMesh RootProxyMesh;
			RootProxyMesh.Mesh = RootProxyData.ProxyMeshes[MeshIndex];
			RootProxyMesh.Transform = FTransform(RootProxyData.GetMeshTransform(MeshIndex)) * RootTransform;
			if (RootProxyData.MeshOverrideMaterials.IsValidIndex(MeshIndex))
			{
				RootProxyMesh.OverrideMaterials = RootProxyData.MeshOverrideMaterials[MeshIndex].Materials;
			}
			OutRootProxyMeshes.Emplace(RootProxyMesh);
		}
	}

	// Set Outputs
	SetValue(Context, MoveTemp(OutCollection), &Collection);
	SetValue(Context, MoveTemp(OutMaterials), &Materials);
	SetValue(Context, MoveTemp(OutInstancedMeshes), &InstancedMeshes);
	SetValue(Context, MoveTemp(OutRootProxyMeshes), &RootProxyMeshes);
}

// ===========================================================================================================================
// DEPRECATED 5.6 : see FBlueprintToCollectionDataflowNode

FBlueprintToCollectionDataflowNode::FBlueprintToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Collection);
	RegisterOutputConnection(&Materials);
	RegisterOutputConnection(&MaterialInstances);
	RegisterOutputConnection(&InstancedMeshes);
}

void FBlueprintToCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	ensure(Out->IsA(&Collection) || Out->IsA(&Materials) || Out->IsA(&MaterialInstances) || Out->IsA(&InstancedMeshes));

	FManagedArrayCollection OutCollection;
	TArray<TObjectPtr<UMaterialInterface>> OutMaterialInstances;
	TArray<FGeometryCollectionAutoInstanceMesh> OutInstancedMeshes;

	if (Blueprint)
	{
		if (TUniquePtr<FPreviewScene> PreviewScene = MakeUnique<FPreviewScene>(FPreviewScene::ConstructionValues()))
		{
			if (UWorld* PreviewWorld = PreviewScene->GetWorld())
			{
				FActorSpawnParameters SpawnInfo;
				SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
				SpawnInfo.bNoFail = true;
				SpawnInfo.ObjectFlags = RF_Transient;

				if (AActor* PreviewActor = PreviewWorld->SpawnActor(Blueprint->GeneratedClass, nullptr, SpawnInfo))
				{
					FGeometryCollectionEngineConversion::FSkeletalMeshToCollectionConversionParameters ConversionParameters;
					FGeometryCollectionEngineConversion::ConvertActorToGeometryCollection(PreviewActor, OutCollection, OutMaterialInstances, OutInstancedMeshes, ConversionParameters, bSplitComponents);
				}
			}
		}
	}

	TArray<TObjectPtr<UMaterial>> OutMaterials;
	FGeometryCollectionEngineConversion::GetMaterialsFromInstances(OutMaterialInstances, OutMaterials);

	// Set Outputs
	SetValue(Context, MoveTemp(OutCollection), &Collection);
	SetValue(Context, MoveTemp(OutMaterials), &Materials);
	SetValue(Context, MoveTemp(OutMaterialInstances), &MaterialInstances);
	SetValue(Context, MoveTemp(OutInstancedMeshes), &InstancedMeshes);
}

// ===========================================================================================================================

FBlueprintToCollectionDataflowNode_v2::FBlueprintToCollectionDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Collection);
	RegisterOutputConnection(&Materials);
	RegisterOutputConnection(&InstancedMeshes);
	RegisterOutputConnection(&RootProxyMeshes);
}

void FBlueprintToCollectionDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	ensure(Out->IsA(&Collection) || Out->IsA(&Materials) || Out->IsA(&InstancedMeshes) || Out->IsA(&RootProxyMeshes));

	FManagedArrayCollection OutCollection;
	TArray<TObjectPtr<UMaterialInterface>> OutMaterials;
	TArray<FGeometryCollectionAutoInstanceMesh> OutInstancedMeshes;
	TArray<FDataflowRootProxyMesh> OutRootProxyMeshes;

	if (Blueprint)
	{
		if (TUniquePtr<FPreviewScene> PreviewScene = MakeUnique<FPreviewScene>(FPreviewScene::ConstructionValues()))
		{
			if (UWorld* PreviewWorld = PreviewScene->GetWorld())
			{
				FActorSpawnParameters SpawnInfo;
				SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
				SpawnInfo.bNoFail = true;
				SpawnInfo.ObjectFlags = RF_Transient;

				if (AActor* PreviewActor = PreviewWorld->SpawnActor(Blueprint->GeneratedClass, nullptr, SpawnInfo))
				{
					FGeometryCollectionEngineConversion::FSkeletalMeshToCollectionConversionParameters ConversionParameters;
					FGeometryCollectionEngineConversion::ConvertActorToGeometryCollection(PreviewActor, OutCollection, OutMaterials, OutInstancedMeshes, ConversionParameters, bSplitComponents);

					// root proxies
					const FTransform ActorTransform(PreviewActor->GetTransform());

					TInlineComponentArray<UStaticMeshComponent*> StaticMeshComponents(PreviewActor);
					for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
					{
						if (StaticMeshComponent)
						{
							if (UStaticMesh* StaticMeshObject = StaticMeshComponent->GetStaticMesh())
							{
								FTransform MeshTransform(StaticMeshComponent->GetComponentTransform());
								MeshTransform.SetTranslation((MeshTransform.GetTranslation() - ActorTransform.GetTranslation()));

								FDataflowRootProxyMesh RootProxyMesh;
								RootProxyMesh.Mesh = StaticMeshObject;
								RootProxyMesh.Transform = MeshTransform;
								RootProxyMesh.OverrideMaterials = StaticMeshComponent->GetMaterials();
								OutRootProxyMeshes.Add(RootProxyMesh);
							}
						}
					}
				}
			}
		}
	}

	// Set Outputs
	SetValue(Context, MoveTemp(OutCollection), &Collection);
	SetValue(Context, MoveTemp(OutMaterials), &Materials);
	SetValue(Context, MoveTemp(OutInstancedMeshes), &InstancedMeshes);
	SetValue(Context, MoveTemp(OutRootProxyMeshes), &RootProxyMeshes);
}

// ===========================================================================================================================

FAssetRandomSeedDataflowNode::FAssetRandomSeedDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Seed);
	RegisterOutputConnection(&RandomSeed);
}

void FAssetRandomSeedDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&RandomSeed))
	{
		const int32 InSeed = GetValue(Context, &Seed);

		if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
		{
			if (EngineContext->Owner)
			{
				const int32 HashVal = (int32)HashCombine(GetTypeHash(EngineContext->Owner->GetPathName()) + GetTypeHash(InSeed));
				SetValue(Context, HashVal, &RandomSeed);

				return;
			}	
		}

		SetValue(Context, InSeed, &RandomSeed);
	}
}
