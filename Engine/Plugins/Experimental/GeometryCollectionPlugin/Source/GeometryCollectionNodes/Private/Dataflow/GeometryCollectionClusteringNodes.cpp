// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionClusteringNodes.h"
#include "Dataflow/DataflowCore.h"

#include "ChaosLog.h"
#include "Engine/StaticMesh.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionEngineUtility.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealTypePrivate.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "StaticMeshAttributes.h"
#include "DynamicMeshEditor.h"
#include "Operations/MeshBoolean.h"

#include "EngineGlobals.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "Voronoi/Voronoi.h"
#include "PlanarCut.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "FractureEngineClustering.h"
#include "FractureEngineSelection.h"
#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowDebugDraw.h"
#if WITH_EDITOR
#include "Dataflow/DataflowRenderingViewMode.h"
#endif
#include "Dataflow/DataflowEngineUtil.h"
#include "Dataflow/GeometryCollectionUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionClusteringNodes)

namespace UE::Dataflow
{

	void GeometryCollectionClusteringNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAutoClusterDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FClusterFlattenDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FClusterUnclusterDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FClusterDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FClusterMergeToNeighborsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FClusterMergeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FClusterIsolatedRootsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FClusterMagnetDataflowNode);
	}
}

FAutoClusterDataflowNode::FAutoClusterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&TransformSelection);
	RegisterInputConnection(&ClusterSites)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&ClusterFraction)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&SiteSize)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&ClusterGridWidth)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&ClusterGridDepth)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&ClusterGridHeight)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&MinimumSize)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&bPreferConvexity)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&ConcavityTolerance)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterOutputConnection(&Collection, &Collection);
}

/* ---------------------------------------------------------------------------------------------------------------------------------*/

void FAutoClusterDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const FDataflowTransformSelection& InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);

		if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
		{
			Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(*GeomCollection);
			HierarchyFacade.GenerateLevelAttribute();

			EClusterSizeMethodEnum InClusterSizeMethod = ClusterSizeMethod;
			int32 InClusterSites = GetValue<int32>(Context, &ClusterSites);
			float InClusterFraction = GetValue<float>(Context, &ClusterFraction);
			float InSiteSize = GetValue<float>(Context, &SiteSize);
			bool InAutoCluster = AutoCluster;
			bool InEnforceSiteParameters = EnforceSiteParameters;
			bool InAvoidIsolated = AvoidIsolated;
			int32 InGridX = GetValue<int32>(Context, &ClusterGridWidth);
			int32 InGridY = GetValue<int32>(Context, &ClusterGridDepth);
			int32 InGridZ = GetValue<int32>(Context, &ClusterGridHeight);
			float InMinimumClusterSize = GetValue<float>(Context, &MinimumSize);
			int32 InKMeansIterations = ClusterSizeMethod == EClusterSizeMethodEnum::Dataflow_ClusterSizeMethod_ByGrid ? DriftIterations : 500;
			bool bInPreferConvexity = GetValue(Context, &bPreferConvexity);
			float InConcavityTolerance = GetValue(Context, &ConcavityTolerance);

			// only cluster if the selection matches 
			if (InTransformSelection.Num() == GeomCollection->NumElements(FTransformCollection::TransformGroup))
			{
				TArray<int32> SelectedBones;
				InTransformSelection.AsArray(SelectedBones);


				FFractureEngineClustering::AutoCluster(*GeomCollection,
					SelectedBones,
					(EFractureEngineClusterSizeMethod)InClusterSizeMethod,
					InClusterSites,
					InClusterFraction,
					InSiteSize,
					InAutoCluster,
					InAvoidIsolated,
					InEnforceSiteParameters,
					InGridX, InGridY, InGridZ, InMinimumClusterSize, InKMeansIterations,
					bInPreferConvexity, InConcavityTolerance);

				FGeometryCollectionProximityUtility ProximityUtility(GeomCollection.Get());
				ProximityUtility.UpdateProximity();
			}
			else
			{
				UE_LOGF(LogChaos, Warning, "Dataflow: AutoCluster Node input selection size does not match the collection size, skipping clustering");
			}
			SetValue<const FManagedArrayCollection&>(Context, *GeomCollection, &Collection);
		}
	}
}

/* ---------------------------------------------------------------------------------------------------------------------------------*/

void FClusterFlattenDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		if (InCollection.NumElements(FTransformCollection::TransformAttribute) > 0)
		{
			if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
			{
				Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(*GeomCollection);
				HierarchyFacade.GenerateLevelAttribute();

				TArray<int32> ToFlatten;
				if (IsConnected(&OptionalTransformSelection))
				{
					const FDataflowTransformSelection& InTransformSelection = GetValue(Context, &OptionalTransformSelection);
					ToFlatten = InTransformSelection.AsArray();
					GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(*GeomCollection);
					SelectionFacade.Sanitize(ToFlatten);
					SelectionFacade.FilterSelectionBySimulationType(ToFlatten, FGeometryCollection::FST_Clustered);
				}
				else
				{
					const int32 RootClusterIndex = HierarchyFacade.GetRootIndex();
					ToFlatten.Add(RootClusterIndex);
				}

				for (int32 ToFlattenIdx : ToFlatten)
				{
					TArray<int32> LeafBones;
					FGeometryCollectionClusteringUtility::GetLeafBones(GeomCollection.Get(), ToFlattenIdx, true, LeafBones);
					FGeometryCollectionClusteringUtility::ClusterBonesUnderExistingNode(GeomCollection.Get(), ToFlattenIdx, LeafBones);
				}

				FGeometryCollectionClusteringUtility::RemoveDanglingClusters(GeomCollection.Get());

				HierarchyFacade.GenerateLevelAttribute();
				SetValue(Context, (const FManagedArrayCollection&)(*GeomCollection), &Collection);
			}
		}
		else
		{
			SetValue(Context, InCollection, &Collection);
		}
	}
}


void FClusterUnclusterDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		if (InCollection.NumElements(FTransformCollection::TransformAttribute) > 0)
		{
			const FDataflowTransformSelection& InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);
			if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
			{
				Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(*GeomCollection);
				HierarchyFacade.GenerateLevelAttribute();

				TArray<int32> Selection = InTransformSelection.AsArray();
				GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(*GeomCollection);
				SelectionFacade.ConvertSelectionToClusterNodes(Selection, false);
				SelectionFacade.RemoveRootNodes(Selection);
				if (!Selection.IsEmpty())
				{
					FGeometryCollectionClusteringUtility::CollapseHierarchyOneLevel(GeomCollection.Get(), Selection);
					FGeometryCollectionClusteringUtility::RemoveDanglingClusters(GeomCollection.Get());

					HierarchyFacade.GenerateLevelAttribute();
				}
				SetValue<const FManagedArrayCollection&>(Context, *GeomCollection, &Collection);
			}
		}
		else
		{
			SetValue(Context, InCollection, &Collection);
		}
	}
}


void FClusterDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const FDataflowTransformSelection& InTransformSelection = GetValue(Context, &TransformSelection);
		if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
		{
			TArray<int32> Selection = InTransformSelection.AsArray();
			FFractureEngineClustering::ClusterSelected(*GeomCollection, Selection);
			SetValue(Context, (const FManagedArrayCollection&)(*GeomCollection), &Collection);
		}
	}
}

void FClusterMergeToNeighborsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const FDataflowTransformSelection& InTransformSelection = GetValue(Context, &TransformSelection);
		if (InTransformSelection.NumSelected() == 0)
		{
			SetValue(Context, InCollection, &Collection);
			return;
		}
		if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
		{
			Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(*GeomCollection);
			HierarchyFacade.GenerateLevelAttribute();

			double InMinVolumeCubeRoot = (double)GetValue(Context, &MinVolumeCubeRoot);
			double InMinVolume = InMinVolumeCubeRoot * InMinVolumeCubeRoot * InMinVolumeCubeRoot;
			bool bInOnlyToConnected = GetValue(Context, &bOnlyToConnected);
			bool bInOnlySameParent = GetValue(Context, &bOnlySameParent);
			UE::PlanarCut::ENeighborSelectionMethod InNeighborSelectionMethod =
				NeighborSelectionMethod == EClusterNeighborSelectionMethodEnum::Dataflow_ClusterNeighborSelectionMethod_LargestNeighbor ?
				UE::PlanarCut::ENeighborSelectionMethod::LargestNeighbor : UE::PlanarCut::ENeighborSelectionMethod::NearestCenter;

			TArray<int32> Selection = InTransformSelection.AsArray();
			TArray<double> Volumes;
			FindBoneVolumes(
				*GeomCollection,
				TArrayView<const int32>(),
				Volumes, 1.0, true);
			MergeClusters(
				*GeomCollection,
				Volumes,
				InMinVolume,
				Selection,
				InNeighborSelectionMethod,
				bInOnlyToConnected,
				bInOnlySameParent
			);
			SetValue(Context, (const FManagedArrayCollection&)(*GeomCollection), &Collection);
			return;
		}

		SetValue(Context, InCollection, &Collection);
	}
}

void FClusterMergeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const FDataflowTransformSelection& InTransformSelection = GetValue(Context, &TransformSelection);
		if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
		{
			Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(*GeomCollection);
			HierarchyFacade.GenerateLevelAttribute();

			TArray<int32> Selection = InTransformSelection.AsArray();
			FFractureEngineClustering::MergeSelectedClusters(*GeomCollection, Selection);
			SetValue(Context, (const FManagedArrayCollection&)(*GeomCollection), &Collection);
		}
	}
}

void FClusterIsolatedRootsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		int32 NumTransforms = InCollection.NumElements(FGeometryCollection::TransformGroup);
		// Only if there is a single transform, re-parent it under a new transform
		if (NumTransforms == 1)
		{
			if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
			{
				FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(GeomCollection.Get());
				SetValue(Context, (const FManagedArrayCollection&)(*GeomCollection), &Collection);
				return;
			}
		}
		SetValue(Context, InCollection, &Collection);
	}
}

void FClusterMagnetDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
		{
			FDataflowTransformSelection InTransformSelection = GetValue(Context, &TransformSelection);
			int32 InIterations = FMath::Max(1, GetValue(Context, &Iterations));
			TArray<int32> InSelection = InTransformSelection.AsArray();
			if (FFractureEngineClustering::ClusterMagnet(*GeomCollection, InSelection, InIterations))
			{
				SetValue(Context, (const FManagedArrayCollection&)(*GeomCollection), &Collection);
				return;
			}
		}
		SetValue(Context, InCollection, &Collection);
	}
}


