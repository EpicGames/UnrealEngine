// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionConversionNodes.h"
#include "Dataflow/DataflowCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionConversionNodes)

namespace UE::Dataflow
{
	void GeometryCollectionConversionNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVectorToStringDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFloatToStringDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FIntToStringDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBoolToStringDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FIntToFloatDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFloatToDoubleDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FIntToDoubleDataflowNode)
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFloatToIntDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FIntToBoolDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBoolToIntDataflowNode);
	}
}

void FVectorToStringDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FString>(&String))
	{
		FString Value = GetValue<FVector>(Context, &Vector).ToString();
		SetValue(Context, Value, &String);
	}
}

void FFloatToStringDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FString>(&String))
	{
		FString Value = FString::Printf(TEXT("%f"), GetValue<float>(Context, &Float));
		SetValue(Context, Value, &String);
	}
}

void FIntToStringDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FString>(&String))
	{
		FString Value = FString::Printf(TEXT("%d"), GetValue<int32>(Context, &Int));
		SetValue(Context, Value, &String);
	}
}

void FBoolToStringDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FString>(&String))
	{
		FString Value = FString::Printf(TEXT("%s"), GetValue<bool>(Context, &Bool) ? TEXT("true") : TEXT("false"));
		SetValue(Context, Value, &String);
	}
}

void FIntToFloatDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Float))
	{
		float Value = float(GetValue<int32>(Context, &Int));
		SetValue(Context, Value, &Float);
	}
}

void FIntToDoubleDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Double))
	{
		double Value = double(GetValue<int32>(Context, &Int));
		SetValue(Context, Value, &Double);
	}
}

void FFloatToDoubleDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Double))
	{
		double Value = double(GetValue<float>(Context, &Float));
		SetValue(Context, Value, &Double);
	}
}

void FFloatToIntDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<int32>(&Int))
	{
		float FloatVal = GetValue<float>(Context, &Float);
		if (Function == EFloatToIntFunctionEnum::Dataflow_FloatToInt_Function_Floor)
		{
			SetValue<int32>(Context, FMath::FloorToInt32(FloatVal), &Int);
		}
		else if (Function == EFloatToIntFunctionEnum::Dataflow_FloatToInt_Function_Ceil)
		{
			SetValue<int32>(Context, FMath::CeilToInt32(FloatVal), &Int);
		}
		else if (Function == EFloatToIntFunctionEnum::Dataflow_FloatToInt_Function_Round)
		{
			SetValue<int32>(Context, FMath::RoundToInt32(FloatVal), &Int);
		}
		else if (Function == EFloatToIntFunctionEnum::Dataflow_FloatToInt_Function_Truncate)
		{
			SetValue<int32>(Context, int32(FMath::TruncToFloat(FloatVal)), &Int);
		}
	}
}

void FIntToBoolDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<bool>(&Bool))
	{
		bool Value = GetValue<int32>(Context, &Int, Int) == 0 ? false : true;
		SetValue(Context, Value, &Bool);
	}
}

void FBoolToIntDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<int32>(&Int))
	{
		int32 Value = (int32)GetValue<bool>(Context, &Bool, Bool);
		SetValue(Context, Value, &Int);
	}
}


