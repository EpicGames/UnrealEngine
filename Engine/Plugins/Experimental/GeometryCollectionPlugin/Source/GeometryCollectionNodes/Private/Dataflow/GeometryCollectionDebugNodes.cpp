// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionDebugNodes.h"
#include "Dataflow/DataflowCore.h"

#include "Containers/UnrealString.h"

#include "Misc/FileHelper.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshEditor.h"
#include "Generators/BoxSphereGenerator.h"
#include "FractureEngineConvex.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionDebugNodes)

namespace UE::Dataflow
{
	void GeometryCollectionDebugNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvexHullToMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSphereCoveringToMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMeshToOBJStringDebugDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSphereCoveringCountSpheresNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FWriteStringToFile);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSpheresToMeshDataflowNode);
	}
}

FConvexHullToMeshDataflowNode::FConvexHullToMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&OptionalSelectionFilter);
	RegisterOutputConnection(&Mesh);
	RegisterOutputConnection(&Meshes);
}

void FConvexHullToMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		TArray<int32> TransformSelection;
		bool bHasSelection = IsConnected(&OptionalSelectionFilter);
		if (bHasSelection)
		{
			const FDataflowTransformSelection& InOptionalSelectionFilter = GetValue(Context, &OptionalSelectionFilter);
			TransformSelection = InOptionalSelectionFilter.AsArray();
		}

		FDynamicMesh3 HullsMesh;
		UE::FractureEngine::Convex::GetConvexHullsAsDynamicMesh(InCollection, HullsMesh, bHasSelection, TransformSelection, bUseRobustHulls);

		TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
		NewMesh->SetMesh(MoveTemp(HullsMesh));
		SetValue(Context, NewMesh, &Mesh);
	}
	else if (Out->IsA(&Meshes))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		TArray<int32> TransformSelection;
		bool bHasSelection = IsConnected(&OptionalSelectionFilter);
		if (bHasSelection)
		{
			const FDataflowTransformSelection& InOptionalSelectionFilter = GetValue(Context, &OptionalSelectionFilter);
			TransformSelection = InOptionalSelectionFilter.AsArray();
		}

		TArray<FDynamicMesh3> HullsMeshes;
		UE::FractureEngine::Convex::GetConvexHullsAsDynamicMeshes(InCollection, HullsMeshes, bHasSelection, TransformSelection, bUseRobustHulls);

		TArray<TObjectPtr<UDynamicMesh>> NewMeshes;
		NewMeshes.SetNum(HullsMeshes.Num());

		for (FDynamicMesh3& HullMesh : HullsMeshes)
		{
			TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
			NewMesh->SetMesh(MoveTemp(HullMesh));
			NewMeshes.Add(NewMesh);
		}
		SetValue(Context, MoveTemp(NewMeshes), &Meshes);
	}

}

void FSphereCoveringToMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
		NewMesh->Reset();

		FDynamicMesh3 Accum;
		FDynamicMeshEditor Editor(&Accum);
		FMeshIndexMappings IndexMaps_Unused;
		FDataflowSphereCovering InSphereCovering = GetValue(Context, &SphereCovering);
		for (int32 SphereIdx = 0; SphereIdx < InSphereCovering.Spheres.Num(); ++SphereIdx)
		{
			FBoxSphereGenerator SphereGen;
			SphereGen.EdgeVertices.A = SphereGen.EdgeVertices.B = SphereGen.EdgeVertices.C = FMath::Max(2, VerticesAlongEachSide);
			SphereGen.Radius = InSphereCovering.Spheres.GetRadius(SphereIdx);
			FVector Center = InSphereCovering.Spheres.GetCenter(SphereIdx);
			FDynamicMesh3 Sphere(&SphereGen.Generate());
			Editor.AppendMesh(&Sphere, IndexMaps_Unused,
				[Center](int VID, const FVector3d& Pos) { return Pos + Center; });
			IndexMaps_Unused.Reset();
		}
		NewMesh->SetMesh(MoveTemp(Accum));

		SetValue(Context, NewMesh, &Mesh);

	}
}

void FMeshToOBJStringDebugDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&StringOBJ))
	{
		const UDynamicMesh* InMesh = GetValue(Context, &Mesh);
		const UE::Geometry::FDynamicMesh3& MeshRef = InMesh->GetMeshRef();
		FString Build;
		Build.Reset(MeshRef.MaxVertexID() * 40 + MeshRef.TriangleCount() * 24);
		for (int32 VID = 0; VID < MeshRef.MaxVertexID(); ++VID)
		{
			FVector V = FVector::ZeroVector;
			if (MeshRef.IsVertex(VID))
			{
				V = MeshRef.GetVertex(VID);
			}
			Build += FString::Printf(TEXT("v %f %f %f\n"), V.X, V.Y, V.Z);
		}

		// Uvs
		const UE::Geometry::FDynamicMeshUVOverlay* UvOverlay = nullptr;

		if (MeshRef.HasVertexUVs())
		{
			for (int32 VID = 0; VID < MeshRef.MaxVertexID(); ++VID)
			{
				const FVector2f Uv = MeshRef.GetVertexUV(VID);
				Build += FString::Printf(TEXT("vt %f %f\n"), Uv.X, Uv.Y);
			}
		}
		else if (MeshRef.Attributes()->NumUVLayers() > 0)
		{
			UvOverlay = MeshRef.Attributes()->GetUVLayer(0);
			if (UvOverlay)
			{
				for (int32 UvIndex = 0; UvIndex < UvOverlay->ElementCount(); ++UvIndex)
				{
					const FVector2f Uv = UvOverlay->GetElement(UvIndex);
					Build += FString::Printf(TEXT("vt %f %f\n"), Uv.X, Uv.Y);
				}
			}
		}

		// faces
		bool bInInvertFaces = GetValue(Context, &bInvertFaces);
		for (int32 TID = 0; TID < MeshRef.MaxTriangleID(); ++TID)
		{
			if (MeshRef.IsTriangle(TID))
			{
				UE::Geometry::FIndex3i T = MeshRef.GetTriangle(TID);
				// Note: OBJ viewers generally expect the opposite triangle winding from UE meshes, so inverted == the UE order
				if (!bInInvertFaces)
				{
					Swap(T.B, T.C);
				}
				if (UvOverlay)
				{
					UE::Geometry::FIndex3i UvIndices = UvOverlay->GetTriangle(TID);
					if (!bInInvertFaces)
					{
						Swap(UvIndices.B, UvIndices.C);
					}
					Build += FString::Printf(TEXT("f %d/%d %d/%d %d/%d\n"), T.A + 1, UvIndices.A + 1, T.B + 1, UvIndices.B + 1, T.C + 1, UvIndices.C + 1);
				}
				else
				{
					Build += FString::Printf(TEXT("f %d %d %d\n"), T.A + 1, T.B + 1, T.C + 1);
				}
			}
		}
		SetValue(Context, MoveTemp(Build), &StringOBJ);
	}
}

void FWriteStringToFile::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	FString InContents = GetValue(Context, &FileContents);
	FString InPath = GetValue(Context, &FilePath);

	bool bSuccess = FFileHelper::SaveStringToFile(InContents, *InPath);
	if (!bSuccess)
	{
		UE_LOGF(LogChaos, Warning, "Failed to write to file %ls:\n\n%ls", *InPath, *InContents);
	}
}

/* --------------------------------------------------------------------------------------------------------------------------- */

FSpheresToMeshDataflowNode::FSpheresToMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Spheres);
	RegisterOutputConnection(&Mesh);
	RegisterOutputConnection(&Meshes);
}

void FSpheresToMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		TArray<FSphere> InSphereArr = GetValue(Context, &Spheres);

		TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
		NewMesh->Reset();

		FDynamicMesh3 Accum;
		FDynamicMeshEditor Editor(&Accum);
		FMeshIndexMappings IndexMaps_Unused;
		for (int32 SphereIdx = 0; SphereIdx < InSphereArr.Num(); ++SphereIdx)
		{
			FBoxSphereGenerator SphereGen;
			SphereGen.EdgeVertices.A = SphereGen.EdgeVertices.B = SphereGen.EdgeVertices.C = FMath::Max(2, VerticesAlongEachSide);
			SphereGen.Radius = InSphereArr[SphereIdx].W;
			FVector Center = InSphereArr[SphereIdx].Center;

			FDynamicMesh3 Sphere(&SphereGen.Generate());
			Editor.AppendMesh(&Sphere, IndexMaps_Unused,
				[Center](int VID, const FVector3d& Pos) { return Pos + Center; });
			IndexMaps_Unused.Reset();
		}
		NewMesh->SetMesh(MoveTemp(Accum));

		SetValue(Context, NewMesh, &Mesh);
	}
	else if (Out->IsA(&Meshes))
	{
		TArray<FSphere> InSphereArr = GetValue(Context, &Spheres);

		TArray<TObjectPtr<UDynamicMesh>> NewMeshes;
		NewMeshes.SetNum(Spheres.Num());

		for (FSphere& Sphere : InSphereArr)
		{
			FBoxSphereGenerator SphereGen;
			SphereGen.EdgeVertices.A = SphereGen.EdgeVertices.B = SphereGen.EdgeVertices.C = FMath::Max(2, VerticesAlongEachSide);
			SphereGen.Radius = Sphere.W;
			SphereGen.Box.Frame.Origin = Sphere.Center;

			FDynamicMesh3 SphereMesh(&SphereGen.Generate());

			TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
			NewMesh->SetMesh(MoveTemp(SphereMesh));
			NewMeshes.Add(NewMesh);
		}

		SetValue(Context, MoveTemp(NewMeshes), &Meshes);
	}
}

/* --------------------------------------------------------------------------------------------------------------------------- */

