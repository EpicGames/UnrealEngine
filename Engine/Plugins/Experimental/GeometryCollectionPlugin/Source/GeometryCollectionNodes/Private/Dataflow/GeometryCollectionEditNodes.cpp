// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionEditNodes.h"
#include "Dataflow/DataflowCore.h"

#include "Engine/StaticMesh.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionEngineUtility.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealTypePrivate.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "StaticMeshAttributes.h"
#include "DynamicMeshEditor.h"
#include "Operations/MeshBoolean.h"

#include "EngineGlobals.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "Voronoi/Voronoi.h"
#include "PlanarCut.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "FractureEngineEdit.h"
#include "FractureEngineSelection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionEditNodes)

namespace UE::Dataflow
{

	void GeometryCollectionEditNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FPruneInCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetVisibilityInCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMergeInCollectionDataflowNode);
	}
}


void FPruneInCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const FDataflowTransformSelection& InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);

		const int32 NumTransforms = InCollection.NumElements(FGeometryCollection::TransformGroup);
		if (InTransformSelection.Num() == NumTransforms)
		{
			if (InTransformSelection.AnySelected())
			{
				if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
				{
					TArray<int32> BoneIndices;
					InTransformSelection.AsArray(BoneIndices);

					FFractureEngineEdit::DeleteBranch(*GeomCollection, BoneIndices);

					SetValue<const FManagedArrayCollection&>(Context, *GeomCollection, &Collection);
					return;
				}
			}
		}
		else
		{
			// ERROR: InTransformSelection's length doesn't much number of transforms in InCollection
			FString ErrorStr = "TransformSelection's number of elements doesn't match number of transforms in Collection.";
			UE_LOGF(LogTemp, Error, "[Dataflow ERROR] %ls", *ErrorStr);
		}

		SetValue(Context, InCollection, &Collection);
	}
}


void FSetVisibilityInCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		const FDataflowTransformSelection& InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);
		const FDataflowFaceSelection& InFaceSelection = GetValue<FDataflowFaceSelection>(Context, &FaceSelection);

		if (IsConnected<FDataflowTransformSelection>(&TransformSelection))
		{
			FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
			const int32 NumTransforms = InCollection.NumElements(FGeometryCollection::TransformGroup);

			if (InTransformSelection.Num() == NumTransforms)
			{
				if (InTransformSelection.AnySelected())
				{
					TArray<int32> BoneIndices;
					InTransformSelection.AsArray(BoneIndices);

					FFractureEngineEdit::SetVisibilityInCollectionFromTransformSelection(InCollection, BoneIndices, Visibility == EVisibiltyOptionsEnum::Dataflow_VisibilityOptions_Visible);

					SetValue(Context, MoveTemp(InCollection), &Collection);
					return;
				}
			}
			else
			{
				// ERROR: InTransformSelection's length doesn't much number of transforms in InCollection
				FString ErrorStr = "TransformSelection's number of elements doesn't match number of transforms in Collection.";
				UE_LOGF(LogTemp, Error, "[Dataflow ERROR] %ls", *ErrorStr);
			}
		}
		else if (IsConnected<FDataflowFaceSelection>(&FaceSelection))
		{
			FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
			const int32 NumFaces = InCollection.NumElements(FGeometryCollection::FacesGroup);

			if (InFaceSelection.Num() == NumFaces)
			{
				if (InFaceSelection.AnySelected())
				{
					TArray<int32> FaceIndexArr;
					InFaceSelection.AsArray(FaceIndexArr);

					FFractureEngineEdit::SetVisibilityInCollectionFromFaceSelection(InCollection, FaceIndexArr, Visibility == EVisibiltyOptionsEnum::Dataflow_VisibilityOptions_Visible);

					SetValue(Context, MoveTemp(InCollection), &Collection);
					return;
				}
			}
			else
			{
				// ERROR: InTransformSelection's length doesn't much number of transforms in InCollection
				FString ErrorStr = "TransformSelection's number of elements doesn't match number of transforms in Collection.";
				UE_LOGF(LogTemp, Error, "[Dataflow ERROR] %ls", *ErrorStr);
			}
		}
		else
		{
			// ERROR: InTransformSelection's length doesn't much number of transforms in InCollection
			FString ErrorStr = "A TransformSelection or a FaceSelection must be connected as input.";
			UE_LOGF(LogTemp, Error, "[Dataflow ERROR] %ls", *ErrorStr);
		}

		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		SetValue(Context, InCollection, &Collection);
	}
}


void FMergeInCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const FDataflowTransformSelection& InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);

		const int32 NumTransforms = InCollection.NumElements(FGeometryCollection::TransformGroup);
		if (InTransformSelection.Num() == NumTransforms)
		{
			if (InTransformSelection.AnySelected())
			{
				if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
				{
					TArray<int32> BoneIndices;
					InTransformSelection.AsArray(BoneIndices);

					FFractureEngineEdit::Merge(*GeomCollection, BoneIndices);

					SetValue<const FManagedArrayCollection&>(Context, *GeomCollection, &Collection);
					return;
				}
			}
		}
		else
		{
			// ERROR: InTransformSelection's length doesn't much number of transforms in InCollection
			FString ErrorStr = "TransformSelection's number of elements doesn't match number of transforms in Collection.";
			UE_LOGF(LogTemp, Error, "[Dataflow ERROR] %ls", *ErrorStr);
		}

		SetValue(Context, InCollection, &Collection);
	}
}
