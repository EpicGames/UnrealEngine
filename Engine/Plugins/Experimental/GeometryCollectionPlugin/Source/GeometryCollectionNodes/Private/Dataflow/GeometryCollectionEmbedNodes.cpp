// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionEmbedNodes.h"
#include "Dataflow/DataflowCore.h"


//#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionEmbedNodes)

namespace UE::Dataflow
{

	void GeometryCollectionEmbedNodes()
	{
	}
}


