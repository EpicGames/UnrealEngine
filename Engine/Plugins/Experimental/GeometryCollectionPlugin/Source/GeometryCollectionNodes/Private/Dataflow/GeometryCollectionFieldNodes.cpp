// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionFieldNodes.h"
#include "Dataflow/DataflowCore.h"

#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionEngineUtility.h"
#include "GeometryCollection/GeometryCollectionEngineRemoval.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealTypePrivate.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "StaticMeshAttributes.h"
#include "DynamicMeshEditor.h"
#include "Operations/MeshBoolean.h"
#include "Materials/Material.h"

#include "EngineGlobals.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "Voronoi/Voronoi.h"
#include "PlanarCut.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "FractureEngineClustering.h"
#include "FractureEngineSelection.h"
#include "GeometryCollection/Facades/CollectionBoundsFacade.h"
#include "GeometryCollection/Facades/CollectionAnchoringFacade.h"
#include "GeometryCollection/Facades/CollectionRemoveOnBreakFacade.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/DynamicMesh3.h"

#include "Field/FieldSystem.h"
#include "Field/FieldSystemNodes.h"

#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"


//#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionFieldNodes)

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionFieldNodes)

namespace UE::Dataflow
{
	void GeometryCollectionFieldNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRadialFalloffFieldDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBoxFalloffFieldDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FPlaneFalloffFieldDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRadialIntMaskFieldDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FUniformScalarFieldDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FUniformVectorFieldDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRadialVectorFieldDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRandomVectorFieldDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FNoiseFieldDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FUniformIntegerFieldDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FWaveScalarFieldDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSumScalarFieldDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSumVectorFieldDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFieldMakeDenseFloatArrayDataflowNode);
	}
}

// --------------------------------------------------------------------------------------------------------------------

static void FieldComputeRemap(const TArray<FVector3f>& InSamplePositions,
	const FDataflowVertexSelection& InSampleIndices,
	TArray<FVector3f>& OutNewSamplePositions,
	TArray<int32>& OutRemapArray)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	//
	// Build the SamplePositions array with only the selected indices
	// and build the remap array
	//
	OutNewSamplePositions.Init(FVector3f(0.f), NumInSamplePositions);
	OutRemapArray.Init(0, NumInSamplePositions);

	int32 NewIdx = 0;
	for (int32 Idx = 0; Idx < NumInSamplePositions; ++Idx)
	{
		if (InSampleIndices.IsSelected(Idx))
		{
			OutNewSamplePositions[NewIdx] = InSamplePositions[Idx];
			OutRemapArray[NewIdx] = Idx;

			NewIdx++;
		}
	}

	OutNewSamplePositions.SetNum(NewIdx, EAllowShrinking::Yes);
	OutRemapArray.SetNum(NewIdx, EAllowShrinking::Yes);
}

// --------------------------------------------------------------------------------------------------------------------

static void RadialFalloffFieldEvaluate(const TArray<FVector3f>& InSamplePositions,
	const FSphere& InSphere,
	const FVector& InTranslation,
	const float InMagnitude,
	const float InMinRange,
	const float InMaxRange,
	const float InDefault,
	const EDataflowFieldFalloffType InFalloffType,
	TFieldArrayView<float>& OutResultsView)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Setup the data for the field 
	FFieldExecutionDatas ExecutionDatas;

	FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, NumInSamplePositions);
	ExecutionDatas.SamplePositions.Init(FVector(0.f), NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; Idx++)
	{
		ExecutionDatas.SamplePositions[Idx] = (FVector)InSamplePositions[Idx];
	}

	FRadialFalloff RadialFalloffField(InMagnitude, 
		InMinRange, 
		InMaxRange, 
		InDefault, 
		InSphere.W, 
		InSphere.Center + InTranslation, 
		(EFieldFalloffType)InFalloffType);

	FFieldContext FieldContext{
		ExecutionDatas,
		FFieldContext::UniquePointerMap(),
		0.0
	};

	// Evalute the field
	RadialFalloffField.Evaluate(FieldContext, OutResultsView);
}

static void RadialFalloffFieldProcess(const TArray<FVector3f>& InSamplePositions,
	const FSphere& InSphere,
	const FVector& InTranslation,
	const float InMagnitude,
	const float InMinRange,
	const float InMaxRange,
	const float InDefault,
	const EDataflowFieldFalloffType InFalloffType,
	FDataflowVertexSelection& OutFieldSelectionMask,
	TArray<float>& OutFieldFloatOutput)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	TArray<float> ResultsArray;
	ResultsArray.Init(false, NumInSamplePositions);
	TFieldArrayView<float> ResultsViewTest(ResultsArray, 0, ResultsArray.Num());

	//
	// Get the samples which effected by the field by evaluating the field with a default 
	// value outside of the (MinRange, MaxRange)
	//
	const float NewDefault = InMinRange - 1000.f;

	RadialFalloffFieldEvaluate(InSamplePositions,
		InSphere,
		InTranslation,
		InMagnitude,
		InMinRange,
		InMaxRange,
		NewDefault,
		InFalloffType,
		ResultsViewTest);

	for (int32 Idx = 0; Idx < NumInSamplePositions; ++Idx)
	{
		if (ResultsViewTest[Idx] != NewDefault)
		{
			OutFieldSelectionMask.SetSelected(Idx);
		}
	}

	// Compute the field
	ResultsArray.Init(false, NumInSamplePositions);
	TFieldArrayView<float> ResultsView(ResultsArray, 0, ResultsArray.Num());

	RadialFalloffFieldEvaluate(InSamplePositions,
		InSphere,
		InTranslation,
		InMagnitude,
		InMinRange,
		InMaxRange,
		InDefault,
		InFalloffType,
		ResultsView);

	for (int32 Idx = 0; Idx < NumInSamplePositions; ++Idx)
	{
		OutFieldFloatOutput[Idx] = ResultsView[Idx];
	}
}

void FRadialFalloffFieldDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<float>>(&FieldFloatResult) ||
		Out->IsA<TArray<int32>>(&FieldRemap) ||
		Out->IsA<FDataflowVertexSelection>(&FieldSelectionMask) ||
		Out->IsA<int32>(&NumSamplePositions))
	{
		const TArray<FVector3f>& InSamplePositions = GetValue<TArray<FVector3f>>(Context, &SamplePositions);
		const FDataflowVertexSelection& InSampleIndices = GetValue<FDataflowVertexSelection>(Context, &SampleIndices);
		const FSphere& InSphere = GetValue<FSphere>(Context, &Sphere);
		const FVector& InTranslation = GetValue<FVector>(Context, &Translation);
		const float InMagnitude = GetValue<float>(Context, &Magnitude);
		const float InMinRange = GetValue<float>(Context, &MinRange);
		const float InMaxRange = GetValue<float>(Context, &MaxRange);
		const float InDefault = GetValue<float>(Context, &Default);

		//
		// SampleIndices input not connected, all the elements from SamplePositions array
		// will be processed and FieldRemap output will be set empty
		//
		if (!IsConnected<FDataflowVertexSelection>(&SampleIndices))
		{
			const int32 NumInSamplePositions = InSamplePositions.Num();

			//
			// Process the field
			//
			FDataflowVertexSelection NewFieldSelectionMask;
			NewFieldSelectionMask.Initialize(NumInSamplePositions, false);

			TArray<float> NewFieldFloatResult;
			NewFieldFloatResult.Init(false, NumInSamplePositions);

			RadialFalloffFieldProcess(InSamplePositions,
				InSphere,
				InTranslation,
				InMagnitude,
				InMinRange,
				InMaxRange,
				InDefault,
				FalloffType,
				NewFieldSelectionMask,
				NewFieldFloatResult);

			// Set the outputs
			SetValue<TArray<float>>(Context, MoveTemp(NewFieldFloatResult), &FieldFloatResult);
			SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
			SetValue<FDataflowVertexSelection>(Context, MoveTemp(NewFieldSelectionMask), &FieldSelectionMask);
			SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

			return;
		}
		//
		// SampleIndices input connected, the selected elements from SamplePositions array
		// will be processed and FieldRemap output will contain the remap info
		// IMPORTANT: Number of elements in SamplePositions and SampleIndices must be the same
		//
		else
		{
			if (InSamplePositions.Num() == InSampleIndices.Num())
			{
				TArray<FVector3f> NewSamplePositions;
				TArray<int32> NewRemapArray;

				FieldComputeRemap(InSamplePositions, InSampleIndices, NewSamplePositions, NewRemapArray);

				const int32 NumNewSamplePositions = NewSamplePositions.Num();

				//
				// Process the field
				//
				FDataflowVertexSelection NewFieldSelectionMask;
				NewFieldSelectionMask.Initialize(NumNewSamplePositions, false);

				TArray<float> NewFieldFloatResult;
				NewFieldFloatResult.Init(false, NumNewSamplePositions);

				RadialFalloffFieldProcess(NewSamplePositions,
					InSphere,
					InTranslation,
					InMagnitude,
					InMinRange,
					InMaxRange,
					InDefault,
					FalloffType,
					NewFieldSelectionMask,
					NewFieldFloatResult);

				// Set the outputs
				SetValue<TArray<float>>(Context, MoveTemp(NewFieldFloatResult), &FieldFloatResult);
				SetValue<TArray<int32>>(Context, MoveTemp(NewRemapArray), &FieldRemap);
				SetValue<FDataflowVertexSelection>(Context, MoveTemp(NewFieldSelectionMask), &FieldSelectionMask);
				SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

				return;
			}
		}

		// Set the outputs
		SetValue<TArray<float>>(Context, TArray<float>(), &FieldFloatResult);
		SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
		SetValue<FDataflowVertexSelection>(Context, FDataflowVertexSelection(), &FieldSelectionMask);
		SetValue<int32>(Context, 0, &NumSamplePositions);
	}
}


// --------------------------------------------------------------------------------------------------------------------

static void BoxFalloffFieldEvaluate(const TArray<FVector3f>& InSamplePositions,
	const FBox& InBox,
	FTransform InTransform,
	const float InMagnitude,
	const float InMinRange,
	const float InMaxRange,
	const float InDefault,
	const EDataflowFieldFalloffType InFalloffType,
	TFieldArrayView<float>& OutResultsView)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Setup the data for the field 
	FFieldExecutionDatas ExecutionDatas;

	FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, NumInSamplePositions);
	ExecutionDatas.SamplePositions.Init(FVector(0.f), NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; Idx++)
	{
		ExecutionDatas.SamplePositions[Idx] = (FVector)InSamplePositions[Idx];
	}

	//
	// Translation and Scale needs to be adjusted based on the InBox
	//
	FVector Center = InBox.GetCenter();
	FVector Translation = InTransform.GetTranslation();
	Translation -= Center;

	InTransform.SetTranslation(Translation);

	FVector Extent(InBox.Max - InBox.Min);
	FVector Scale = InTransform.GetScale3D();

	Scale.X *= 100.f / Extent.X;
	Scale.Y *= 100.f / Extent.Y;
	Scale.Z *= 100.f / Extent.Z;

	InTransform.SetScale3D(Scale);

	FBoxFalloff BoxFalloffField(InMagnitude,
		InMinRange,
		InMaxRange,
		InDefault,
		InTransform,
		(EFieldFalloffType)InFalloffType);

	FFieldContext FieldContext{
		ExecutionDatas,
		FFieldContext::UniquePointerMap(),
		0.0
	};

	// Evalute the field
	BoxFalloffField.Evaluate(FieldContext, OutResultsView);
}

static void BoxFalloffFieldProcess(const TArray<FVector3f>& InSamplePositions,
	const FBox& InBox,
	const FTransform& InTransform,
	const float InMagnitude,
	const float InMinRange,
	const float InMaxRange,
	const float InDefault,
	const EDataflowFieldFalloffType InFalloffType,
	FDataflowVertexSelection& OutFieldSelectionMask,
	TArray<float>& OutFieldFloatOutput)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	TArray<float> ResultsArray;
	ResultsArray.Init(false, NumInSamplePositions);
	TFieldArrayView<float> ResultsViewTest(ResultsArray, 0, ResultsArray.Num());

	//
	// Get the samples which effected by the field by evaluating the field with a default 
	// value outside of the (MinRange, MaxRange)
	//
	const float NewDefault = InMinRange - 1000.f;

	BoxFalloffFieldEvaluate(InSamplePositions,
		InBox,
		InTransform,
		InMagnitude,
		InMinRange,
		InMaxRange,
		NewDefault,
		InFalloffType,
		ResultsViewTest);

	for (int32 Idx = 0; Idx < NumInSamplePositions; ++Idx)
	{
		if (ResultsViewTest[Idx] != NewDefault)
		{
			OutFieldSelectionMask.SetSelected(Idx);
		}
	}

	// Compute the field
	ResultsArray.Init(false, NumInSamplePositions);
	TFieldArrayView<float> ResultsView(ResultsArray, 0, ResultsArray.Num());

	BoxFalloffFieldEvaluate(InSamplePositions,
		InBox,
		InTransform,
		InMagnitude,
		InMinRange,
		InMaxRange,
		InDefault,
		InFalloffType,
		ResultsView);

	for (int32 Idx = 0; Idx < NumInSamplePositions; ++Idx)
	{
		OutFieldFloatOutput[Idx] = ResultsView[Idx];
	}
}

void FBoxFalloffFieldDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<float>>(&FieldFloatResult) ||
		Out->IsA<TArray<int32>>(&FieldRemap) ||
		Out->IsA<FDataflowVertexSelection>(&FieldSelectionMask) ||
		Out->IsA<int32>(&NumSamplePositions))
	{
		const TArray<FVector3f>& InSamplePositions = GetValue<TArray<FVector3f>>(Context, &SamplePositions);
		const FDataflowVertexSelection& InSampleIndices = GetValue<FDataflowVertexSelection>(Context, &SampleIndices);
		const FBox& InBox = GetValue<FBox>(Context, &Box);
		const FTransform& InTransform = GetValue<FTransform>(Context, &Transform);
		const float InMagnitude = GetValue<float>(Context, &Magnitude);
		const float InMinRange = GetValue<float>(Context, &MinRange);
		const float InMaxRange = GetValue<float>(Context, &MaxRange);
		const float InDefault = GetValue<float>(Context, &Default);

		//
		// SampleIndices input not connected, all the elements from SamplePositions array
		// will be processed and FieldRemap output will be set empty
		//
		if (!IsConnected<FDataflowVertexSelection>(&SampleIndices))
		{
			const int32 NumInSamplePositions = InSamplePositions.Num();

			//
			// Process the field
			//
			FDataflowVertexSelection NewFieldSelectionMask;
			NewFieldSelectionMask.Initialize(NumInSamplePositions, false);

			TArray<float> NewFieldFloatResult;
			NewFieldFloatResult.Init(false, NumInSamplePositions);

			BoxFalloffFieldProcess(InSamplePositions,
				InBox,
				InTransform,
				InMagnitude,
				InMinRange,
				InMaxRange,
				InDefault,
				FalloffType,
				NewFieldSelectionMask,
				NewFieldFloatResult);

			// Set the outputs
			SetValue<TArray<float>>(Context, MoveTemp(NewFieldFloatResult), &FieldFloatResult);
			SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
			SetValue<FDataflowVertexSelection>(Context, MoveTemp(NewFieldSelectionMask), &FieldSelectionMask);
			SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

			return;
		}
		//
		// SampleIndices input connected, the selected elements from SamplePositions array
		// will be processed and FieldRemap output will contain the remap info
		// IMPORTANT: Number of elements in SamplePositions and SampleIndices must be the same
		//
		else
		{
			if (InSamplePositions.Num() == InSampleIndices.Num())
			{
				TArray<FVector3f> NewSamplePositions;
				TArray<int32> NewRemapArray;

				FieldComputeRemap(InSamplePositions, InSampleIndices, NewSamplePositions, NewRemapArray);

				const int32 NumNewSamplePositions = NewSamplePositions.Num();

				//
				// Process the field
				//
				FDataflowVertexSelection NewFieldSelectionMask;
				NewFieldSelectionMask.Initialize(NumNewSamplePositions, false);

				TArray<float> NewFieldFloatResult;
				NewFieldFloatResult.Init(false, NumNewSamplePositions);

				BoxFalloffFieldProcess(NewSamplePositions,
					InBox,
					InTransform,
					InMagnitude,
					InMinRange,
					InMaxRange,
					InDefault,
					FalloffType,
					NewFieldSelectionMask,
					NewFieldFloatResult);

				// Set the outputs
				SetValue<TArray<float>>(Context, MoveTemp(NewFieldFloatResult), &FieldFloatResult);
				SetValue<TArray<int32>>(Context, MoveTemp(NewRemapArray), &FieldRemap);
				SetValue<FDataflowVertexSelection>(Context, MoveTemp(NewFieldSelectionMask), &FieldSelectionMask);
				SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

				return;
			}
		}

		// Set the outputs
		SetValue<TArray<float>>(Context, TArray<float>(), &FieldFloatResult);
		SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
		SetValue<FDataflowVertexSelection>(Context, FDataflowVertexSelection(), &FieldSelectionMask);
		SetValue<int32>(Context, 0, &NumSamplePositions);
	}
}

// --------------------------------------------------------------------------------------------------------------------

static void PlaneFalloffFieldEvaluate(const TArray<FVector3f>& InSamplePositions,
	const float InMagnitude,
	const float InMinRange,
	const float InMaxRange,
	const float InDefault,
	const float InDistance,
	const FVector& InPosition,
	const FVector& InNormal,
	const EDataflowFieldFalloffType InFalloffType,
	TFieldArrayView<float>& OutResultsView)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Setup the data for the field 
	FFieldExecutionDatas ExecutionDatas;

	FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, NumInSamplePositions);
	ExecutionDatas.SamplePositions.Init(FVector(0.f), NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; Idx++)
	{
		ExecutionDatas.SamplePositions[Idx] = (FVector)InSamplePositions[Idx];
	}

	FPlaneFalloff PlaneFalloffField(InMagnitude,
		InMinRange,
		InMaxRange,
		InDefault,
		InDistance,
		InPosition,
		InNormal,
		(EFieldFalloffType)InFalloffType);

	FFieldContext FieldContext{
		ExecutionDatas,
		FFieldContext::UniquePointerMap(),
		0.0
	};

	// Evalute the field
	PlaneFalloffField.Evaluate(FieldContext, OutResultsView);
}

static void PlaneFalloffFieldProcess(const TArray<FVector3f>& InSamplePositions,
	const float InMagnitude,
	const float InMinRange,
	const float InMaxRange,
	const float InDefault,
	const float InDistance,
	const FVector& InPosition,
	const FVector& InTranslation,
	const FVector& InNormal,
	const EDataflowFieldFalloffType InFalloffType,
	FDataflowVertexSelection& OutFieldSelectionMask,
	TArray<float>& OutFieldFloatOutput)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	TArray<float> ResultsArray;
	ResultsArray.Init(false, NumInSamplePositions);
	TFieldArrayView<float> ResultsViewTest(ResultsArray, 0, ResultsArray.Num());

	//
	// Get the samples which effected by the field by evaluating the field with a default 
	// value outside of the (MinRange, MaxRange)
	//
	const float NewDefault = InMinRange - 1000.f;

	PlaneFalloffFieldEvaluate(InSamplePositions,
		InMagnitude,
		InMinRange,
		InMaxRange,
		NewDefault,
		InDistance,
		InPosition + InTranslation,
		InNormal,
		InFalloffType,
		ResultsViewTest);

	for (int32 Idx = 0; Idx < NumInSamplePositions; ++Idx)
	{
		if (ResultsViewTest[Idx] != NewDefault)
		{
			OutFieldSelectionMask.SetSelected(Idx);
		}
	}

	// Compute the field
	ResultsArray.Init(false, NumInSamplePositions);
	TFieldArrayView<float> ResultsView(ResultsArray, 0, ResultsArray.Num());

	PlaneFalloffFieldEvaluate(InSamplePositions,
		InMagnitude,
		InMinRange,
		InMaxRange,
		InDefault,
		InDistance,
		InPosition + InTranslation,
		InNormal,
		InFalloffType,
		ResultsViewTest);

	for (int32 Idx = 0; Idx < NumInSamplePositions; ++Idx)
	{
		OutFieldFloatOutput[Idx] = ResultsView[Idx];
	}
}

void FPlaneFalloffFieldDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<float>>(&FieldFloatResult) ||
		Out->IsA<TArray<int32>>(&FieldRemap) ||
		Out->IsA<FDataflowVertexSelection>(&FieldSelectionMask) ||
		Out->IsA<int32>(&NumSamplePositions))
	{
		const TArray<FVector3f>& InSamplePositions = GetValue<TArray<FVector3f>>(Context, &SamplePositions);
		const FDataflowVertexSelection& InSampleIndices = GetValue<FDataflowVertexSelection>(Context, &SampleIndices);
		const FVector& InPosition = GetValue<FVector>(Context, &Position);
		const FVector& InNormal = GetValue<FVector>(Context, &Normal);
		const float InDistance = GetValue<float>(Context, &Distance);
		const FVector& InTranslation = GetValue<FVector>(Context, &Translation);
		const float InMagnitude = GetValue<float>(Context, &Magnitude);
		const float InMinRange = GetValue<float>(Context, &MinRange);
		const float InMaxRange = GetValue<float>(Context, &MaxRange);
		const float InDefault = GetValue<float>(Context, &Default);

		//
		// SampleIndices input not connected, all the elements from SamplePositions array
		// will be processed and FieldRemap output will be set empty
		//
		if (!IsConnected<FDataflowVertexSelection>(&SampleIndices))
		{
			const int32 NumInSamplePositions = InSamplePositions.Num();

			//
			// Process the field
			//
			FDataflowVertexSelection NewFieldSelectionMask;
			NewFieldSelectionMask.Initialize(NumInSamplePositions, false);

			TArray<float> NewFieldFloatResult;
			NewFieldFloatResult.Init(false, NumInSamplePositions);

			PlaneFalloffFieldProcess(InSamplePositions,
				InMagnitude,
				InMinRange,
				InMaxRange,
				InDefault,
				InDistance,
				InPosition,
				InTranslation,
				InNormal,
				FalloffType,
				NewFieldSelectionMask,
				NewFieldFloatResult);

			// Set the outputs
			SetValue<TArray<float>>(Context, MoveTemp(NewFieldFloatResult), &FieldFloatResult);
			SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
			SetValue<FDataflowVertexSelection>(Context, MoveTemp(NewFieldSelectionMask), &FieldSelectionMask);
			SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

			return;
		}
		//
		// SampleIndices input connected, the selected elements from SamplePositions array
		// will be processed and FieldRemap output will contain the remap info
		// IMPORTANT: Number of elements in SamplePositions and SampleIndices must be the same
		//
		else
		{
			if (InSamplePositions.Num() == InSampleIndices.Num())
			{
				TArray<FVector3f> NewSamplePositions;
				TArray<int32> NewRemapArray;

				FieldComputeRemap(InSamplePositions, InSampleIndices, NewSamplePositions, NewRemapArray);

				const int32 NumNewSamplePositions = NewSamplePositions.Num();

				//
				// Process the field
				//
				FDataflowVertexSelection NewFieldSelectionMask;
				NewFieldSelectionMask.Initialize(NumNewSamplePositions, false);

				TArray<float> NewFieldFloatResult;
				NewFieldFloatResult.Init(false, NumNewSamplePositions);

				PlaneFalloffFieldProcess(NewSamplePositions,
					InMagnitude,
					InMinRange,
					InMaxRange,
					InDefault,
					InDistance,
					InPosition,
					InTranslation,
					InNormal,
					FalloffType,
					NewFieldSelectionMask,
					NewFieldFloatResult);

				// Set the outputs
				SetValue<TArray<float>>(Context, MoveTemp(NewFieldFloatResult), &FieldFloatResult);
				SetValue<TArray<int32>>(Context, MoveTemp(NewRemapArray), &FieldRemap);
				SetValue<FDataflowVertexSelection>(Context, MoveTemp(NewFieldSelectionMask), &FieldSelectionMask);
				SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

				return;
			}
		}

		// Set the outputs
		SetValue<TArray<float>>(Context, TArray<float>(), &FieldFloatResult);
		SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
		SetValue<FDataflowVertexSelection>(Context, FDataflowVertexSelection(), &FieldSelectionMask);
		SetValue<int32>(Context, 0, &NumSamplePositions);
	}
}

// --------------------------------------------------------------------------------------------------------------------

static void RadialIntMaskFieldEvaluate(const TArray<FVector3f>& InSamplePositions,
	float InRadius,
	const FVector InPosition,
	int32 InInteriorValue,
	int32 InExteriorValue,
	EDataflowSetMaskConditionType InSetMaskConditionType,
	TFieldArrayView<int32>& OutResultsView)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Setup the data for the field 
	FFieldExecutionDatas ExecutionDatas;

	FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, NumInSamplePositions);
	ExecutionDatas.SamplePositions.Init(FVector(0.f), NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; Idx++)
	{
		ExecutionDatas.SamplePositions[Idx] = (FVector)InSamplePositions[Idx];
	}

	FRadialIntMask RadialIntMaskField(InRadius, InPosition, InInteriorValue, InExteriorValue, (ESetMaskConditionType)InSetMaskConditionType);

	FFieldContext FieldContext{
		ExecutionDatas,
		FFieldContext::UniquePointerMap(),
		0.0
	};

	// Evalute the field
	RadialIntMaskField.Evaluate(FieldContext, OutResultsView);
}

static void RadialIntMaskFieldProcess(const TArray<FVector3f>& InSamplePositions,
	float InRadius,
	const FVector InPosition,
	const FVector InTranslation,
	int32 InInteriorValue,
	int32 InExteriorValue,
	EDataflowSetMaskConditionType InSetMaskConditionType,
	TArray<int32>& OutFieldFloatResult)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Compute the field
	TArray<int32> ResultsArray;
	ResultsArray.Init(0, NumInSamplePositions);
	TFieldArrayView<int32> ResultsView(ResultsArray, 0, ResultsArray.Num());

	RadialIntMaskFieldEvaluate(InSamplePositions, 
		InRadius, 
		InPosition + InTranslation,
		InInteriorValue, 
		InExteriorValue, 
		InSetMaskConditionType, 
		ResultsView);

	// Set the outputs
	OutFieldFloatResult.Init(0, NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; ++Idx)
	{
		OutFieldFloatResult[Idx] = ResultsView[Idx];
	}
}

void FRadialIntMaskFieldDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<int32>>(&FieldIntResult) ||
		Out->IsA<TArray<int32>>(&FieldRemap) ||
		Out->IsA<int32>(&NumSamplePositions))
	{
		const TArray<FVector3f>& InSamplePositions = GetValue<TArray<FVector3f>>(Context, &SamplePositions);
		const FDataflowVertexSelection& InSampleIndices = GetValue<FDataflowVertexSelection>(Context, &SampleIndices);
		const FSphere& InSphere = GetValue<FSphere>(Context, &Sphere);
		const FVector InTranslation = GetValue<FVector>(Context, &Translation);
		const int32 InInteriorValue = GetValue<int32>(Context, &InteriorValue);
		const int32 InExteriorValue = GetValue<int32>(Context, &ExteriorValue);

		//
		// SampleIndices input not connected, all the elements from SamplePositions array
		// will be processed and FieldRemapOutput output will be set empty
		//
		if (!IsConnected<FDataflowVertexSelection>(&SampleIndices))
		{
			// Compute the field
			TArray<int32> NewFieldIntResult;
			RadialIntMaskFieldProcess(InSamplePositions,
				InSphere.W,
				InSphere.Center,
				InTranslation,
				InInteriorValue,
				InExteriorValue,
				SetMaskConditionType,
				NewFieldIntResult);

			// Set the outputs
			SetValue<TArray<int32>>(Context, MoveTemp(NewFieldIntResult), &FieldIntResult);
			SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
			SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

			return;
		}
		//
		// SampleIndices input connected, the selected elements from SamplePositions array
		// will be processed and FieldRemapOutput output will contain the remap info
		// IMPORTANT: Number of elements in SamplePositions and SampleIndices must be the same
		//
		else
		{
			if (InSamplePositions.Num() == InSampleIndices.Num())
			{
				TArray<FVector3f> NewSamplePositions;
				TArray<int32> NewRemapArray;

				FieldComputeRemap(InSamplePositions, InSampleIndices, NewSamplePositions, NewRemapArray);

				// Compute the field
				TArray<int32> NewFieldIntResult;
				RadialIntMaskFieldProcess(NewSamplePositions,
					InSphere.W,
					InSphere.Center,
					InTranslation,
					InInteriorValue,
					InExteriorValue,
					SetMaskConditionType,
					NewFieldIntResult);

				// Set the outputs
				SetValue<TArray<int32>>(Context, MoveTemp(NewFieldIntResult), &FieldIntResult);
				SetValue<TArray<int32>>(Context, MoveTemp(NewRemapArray), &FieldRemap);
				SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

				return;
			}
		}

		SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldIntResult);
		SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
		SetValue<int32>(Context, 0, &NumSamplePositions);
	}
}

// --------------------------------------------------------------------------------------------------------------------

static void UniformScalarFieldEvaluate(const TArray<FVector3f>& InSamplePositions,
	float InMagnitude,
	TFieldArrayView<float>& OutResultsView)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Setup the data for the field 
	FFieldExecutionDatas ExecutionDatas;

	FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, NumInSamplePositions);
	ExecutionDatas.SamplePositions.Init(FVector(0.f), NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; Idx++)
	{
		ExecutionDatas.SamplePositions[Idx] = (FVector)InSamplePositions[Idx];
	}

	FUniformScalar UniformScalarField(InMagnitude);

	FFieldContext FieldContext{
		ExecutionDatas,
		FFieldContext::UniquePointerMap(),
		0.0
	};

	// Evalute the field
	UniformScalarField.Evaluate(FieldContext, OutResultsView);
}

static void UniformScalarFieldProcess(const TArray<FVector3f>& InSamplePositions,
	float InMagnitude,
	TArray<float>& OutFieldFloatResult)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Compute the field
	TArray<float> ResultsArray;
	ResultsArray.Init(0.f, NumInSamplePositions);
	TFieldArrayView<float> ResultsView(ResultsArray, 0, ResultsArray.Num());

	UniformScalarFieldEvaluate(InSamplePositions, InMagnitude, ResultsView);

	// Set the outputs
	OutFieldFloatResult.Init(0.f, NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; ++Idx)
	{
		OutFieldFloatResult[Idx] = ResultsView[Idx];
	}
}

void FUniformScalarFieldDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<float>>(&FieldFloatResult) ||
		Out->IsA<TArray<int32>>(&FieldRemap) ||
		Out->IsA<int32>(&NumSamplePositions))
	{
		const TArray<FVector3f>& InSamplePositions = GetValue<TArray<FVector3f>>(Context, &SamplePositions);
		const FDataflowVertexSelection& InSampleIndices = GetValue<FDataflowVertexSelection>(Context, &SampleIndices);
		const float InMagnitude = GetValue<float>(Context, &Magnitude);

		//
		// SampleIndices input not connected, all the elements from SamplePositions array
		// will be processed and FieldRemapOutput output will be set empty
		//
		if (!IsConnected<FDataflowVertexSelection>(&SampleIndices))
		{
			// Compute the field
			TArray<float> NewFieldFloatResult;
			UniformScalarFieldProcess(InSamplePositions, InMagnitude, NewFieldFloatResult);

			// Set the outputs
			SetValue<TArray<float>>(Context, MoveTemp(NewFieldFloatResult), &FieldFloatResult);
			SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
			SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

			return;
		}
		//
		// SampleIndices input connected, the selected elements from SamplePositions array
		// will be processed and FieldRemapOutput output will contain the remap info
		// IMPORTANT: Number of elements in SamplePositions and SampleIndices must be the same
		//
		else
		{
			if (InSamplePositions.Num() == InSampleIndices.Num())
			{
				TArray<FVector3f> NewSamplePositions;
				TArray<int32> NewRemapArray;

				FieldComputeRemap(InSamplePositions, InSampleIndices, NewSamplePositions, NewRemapArray);

				// Compute the field
				TArray<float> NewFieldFloatResult;
				UniformScalarFieldProcess(NewSamplePositions, InMagnitude, NewFieldFloatResult);

				// Set the outputs
				SetValue<TArray<float>>(Context, MoveTemp(NewFieldFloatResult), &FieldFloatResult);
				SetValue<TArray<int32>>(Context, MoveTemp(NewRemapArray), &FieldRemap);
				SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

				return;
			}
		}

		SetValue<TArray<float>>(Context, TArray<float>(), &FieldFloatResult);
		SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
		SetValue<int32>(Context, 0, &NumSamplePositions);
	}
}

// --------------------------------------------------------------------------------------------------------------------

static void UniformVectorFieldEvaluate(const TArray<FVector3f>& InSamplePositions,
	const float InMagnitude,
	const FVector InDirection,
	TFieldArrayView<FVector>& OutResultsView)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Setup the data for the field 
	FFieldExecutionDatas ExecutionDatas;

	FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, NumInSamplePositions);
	ExecutionDatas.SamplePositions.Init(FVector(0.f), NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; Idx++)
	{
		ExecutionDatas.SamplePositions[Idx] = (FVector)InSamplePositions[Idx];
	}

	FUniformVector UniformVectorField(InMagnitude, InDirection);

	FFieldContext FieldContext{
		ExecutionDatas,
		FFieldContext::UniquePointerMap(),
		0.0
	};

	// Evalute the field
	UniformVectorField.Evaluate(FieldContext, OutResultsView);
}

static void UniformVectorFieldProcess(const TArray<FVector3f>& InSamplePositions,
	float InMagnitude,
	const FVector InDirection,
	TArray<FVector>& OutFieldFloatResult)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Compute the field
	TArray<FVector> ResultsArray;
	ResultsArray.Init(FVector(0.f), NumInSamplePositions);
	TFieldArrayView<FVector> ResultsView(ResultsArray, 0, ResultsArray.Num());

	UniformVectorFieldEvaluate(InSamplePositions, InMagnitude, InDirection, ResultsView);

	// Set the outputs
	OutFieldFloatResult.Init(FVector(0.f), NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; ++Idx)
	{
		OutFieldFloatResult[Idx] = ResultsView[Idx];
	}
}

void FUniformVectorFieldDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<FVector>>(&FieldVectorResult) ||
		Out->IsA<TArray<int32>>(&FieldRemap) ||
		Out->IsA<int32>(&NumSamplePositions))
	{
		const TArray<FVector3f>& InSamplePositions = GetValue<TArray<FVector3f>>(Context, &SamplePositions);
		const FDataflowVertexSelection& InSampleIndices = GetValue<FDataflowVertexSelection>(Context, &SampleIndices);
		const float InMagnitude = GetValue<float>(Context, &Magnitude);
		const FVector InDirection = GetValue<FVector>(Context, &Direction);

		//
		// SampleIndices input not connected, all the elements from SamplePositions array
		// will be processed and FieldRemapOutput output will be set empty
		//
		if (!IsConnected<FDataflowVertexSelection>(&SampleIndices))
		{
			// Compute the field
			TArray<FVector> NewFieldVectorResult;
			UniformVectorFieldProcess(InSamplePositions, InMagnitude, InDirection, NewFieldVectorResult);

			// Set the outputs
			SetValue<TArray<FVector>>(Context, MoveTemp(NewFieldVectorResult), &FieldVectorResult);
			SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
			SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

			return;
		}
		//
		// SampleIndices input connected, the selected elements from SamplePositions array
		// will be processed and FieldRemapOutput output will contain the remap info
		// IMPORTANT: Number of elements in SamplePositions and SampleIndices must be the same
		//
		else
		{
			if (InSamplePositions.Num() == InSampleIndices.Num())
			{
				TArray<FVector3f> NewSamplePositions;
				TArray<int32> NewRemapArray;

				FieldComputeRemap(InSamplePositions, InSampleIndices, NewSamplePositions, NewRemapArray);

				// Compute the field
				TArray<FVector> NewFieldVectorResult;
				UniformVectorFieldProcess(NewSamplePositions, InMagnitude, InDirection, NewFieldVectorResult);

				// Set the outputs
				SetValue<TArray<FVector>>(Context, MoveTemp(NewFieldVectorResult), &FieldVectorResult);
				SetValue<TArray<int32>>(Context, MoveTemp(NewRemapArray), &FieldRemap);
				SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

				return;
			}
		}

		SetValue<TArray<FVector>>(Context, TArray<FVector>(), &FieldVectorResult);
		SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
		SetValue<int32>(Context, 0, &NumSamplePositions);
	}

}

// --------------------------------------------------------------------------------------------------------------------

static void RadialVectorFieldEvaluate(const TArray<FVector3f>& InSamplePositions,
	const float InMagnitude,
	const FVector InPosition,
	TFieldArrayView<FVector>& OutResultsView)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Setup the data for the field 
	FFieldExecutionDatas ExecutionDatas;

	FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, NumInSamplePositions);
	ExecutionDatas.SamplePositions.Init(FVector(0.f), NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; Idx++)
	{
		ExecutionDatas.SamplePositions[Idx] = (FVector)InSamplePositions[Idx];
	}

	FRadialVector RadialVectorField(InMagnitude, InPosition);

	FFieldContext FieldContext{
		ExecutionDatas,
		FFieldContext::UniquePointerMap(),
		0.0
	};

	// Evalute the field
	RadialVectorField.Evaluate(FieldContext, OutResultsView);
}

static void RadialVectorFieldProcess(const TArray<FVector3f>& InSamplePositions,
	float InMagnitude,
	const FVector InPosition,
	TArray<FVector>& OutFieldFloatResult)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Compute the field
	TArray<FVector> ResultsArray;
	ResultsArray.Init(FVector(0.f), NumInSamplePositions);
	TFieldArrayView<FVector> ResultsView(ResultsArray, 0, ResultsArray.Num());

	RadialVectorFieldEvaluate(InSamplePositions, InMagnitude, InPosition, ResultsView);

	// Set the outputs
	OutFieldFloatResult.Init(FVector(0.f), NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; ++Idx)
	{
		OutFieldFloatResult[Idx] = ResultsView[Idx];
	}
}
void FRadialVectorFieldDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<FVector>>(&FieldVectorResult) ||
		Out->IsA<TArray<int32>>(&FieldRemap) ||
		Out->IsA<int32>(&NumSamplePositions))
	{
		const TArray<FVector3f>& InSamplePositions = GetValue<TArray<FVector3f>>(Context, &SamplePositions);
		const FDataflowVertexSelection& InSampleIndices = GetValue<FDataflowVertexSelection>(Context, &SampleIndices);
		const float InMagnitude = GetValue<float>(Context, &Magnitude);
		const FVector InPosition = GetValue<FVector>(Context, &Position);

		//
		// SampleIndices input not connected, all the elements from SamplePositions array
		// will be processed and FieldRemapOutput output will be set empty
		//
		if (!IsConnected<FDataflowVertexSelection>(&SampleIndices))
		{
			// Compute the field
			TArray<FVector> NewFieldVectorResult;
			RadialVectorFieldProcess(InSamplePositions, InMagnitude, InPosition, NewFieldVectorResult);

			// Set the outputs
			SetValue<TArray<FVector>>(Context, MoveTemp(NewFieldVectorResult), &FieldVectorResult);
			SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
			SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

			return;
		}
		//
		// SampleIndices input connected, the selected elements from SamplePositions array
		// will be processed and FieldRemapOutput output will contain the remap info
		// IMPORTANT: Number of elements in SamplePositions and SampleIndices must be the same
		//
		else
		{
			if (InSamplePositions.Num() == InSampleIndices.Num())
			{
				TArray<FVector3f> NewSamplePositions;
				TArray<int32> NewRemapArray;

				FieldComputeRemap(InSamplePositions, InSampleIndices, NewSamplePositions, NewRemapArray);

				// Compute the field
				TArray<FVector> NewFieldVectorResult;
				RadialVectorFieldProcess(NewSamplePositions, InMagnitude, InPosition, NewFieldVectorResult);

				// Set the outputs
				SetValue<TArray<FVector>>(Context, MoveTemp(NewFieldVectorResult), &FieldVectorResult);
				SetValue<TArray<int32>>(Context, MoveTemp(NewRemapArray), &FieldRemap);
				SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

				return;
			}
		}

		SetValue<TArray<FVector>>(Context, TArray<FVector>(), &FieldVectorResult);
		SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
		SetValue<int32>(Context, 0, &NumSamplePositions);
	}
}

// --------------------------------------------------------------------------------------------------------------------

static void RandomVectorFieldEvaluate(const TArray<FVector3f>& InSamplePositions,
	const float InMagnitude,
	TFieldArrayView<FVector>& OutResultsView)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Setup the data for the field 
	FFieldExecutionDatas ExecutionDatas;

	FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, NumInSamplePositions);
	ExecutionDatas.SamplePositions.Init(FVector(0.f), NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; Idx++)
	{
		ExecutionDatas.SamplePositions[Idx] = (FVector)InSamplePositions[Idx];
	}

	FRandomVector RandomVectorField(InMagnitude);

	FFieldContext FieldContext{
		ExecutionDatas,
		FFieldContext::UniquePointerMap(),
		0.0
	};

	// Evalute the field
	RandomVectorField.Evaluate(FieldContext, OutResultsView);
}

static void RandomVectorFieldProcess(const TArray<FVector3f>& InSamplePositions,
	float InMagnitude,
	TArray<FVector>& OutFieldFloatResult)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Compute the field
	TArray<FVector> ResultsArray;
	ResultsArray.Init(FVector(0.f), NumInSamplePositions);
	TFieldArrayView<FVector> ResultsView(ResultsArray, 0, ResultsArray.Num());

	RandomVectorFieldEvaluate(InSamplePositions, InMagnitude, ResultsView);

	// Set the outputs
	OutFieldFloatResult.Init(FVector(0.f), NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; ++Idx)
	{
		OutFieldFloatResult[Idx] = ResultsView[Idx];
	}
}

void FRandomVectorFieldDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<FVector>>(&FieldVectorResult) ||
		Out->IsA<TArray<int32>>(&FieldRemap) ||
		Out->IsA<int32>(&NumSamplePositions))
	{
		const TArray<FVector3f>& InSamplePositions = GetValue<TArray<FVector3f>>(Context, &SamplePositions);
		const FDataflowVertexSelection& InSampleIndices = GetValue<FDataflowVertexSelection>(Context, &SampleIndices);
		const float InMagnitude = GetValue<float>(Context, &Magnitude);

		//
		// SampleIndices input not connected, all the elements from SamplePositions array
		// will be processed and FieldRemapOutput output will be set empty
		//
		if (!IsConnected<FDataflowVertexSelection>(&SampleIndices))
		{
			// Compute the field
			TArray<FVector> NewFieldVectorResult;
			RandomVectorFieldProcess(InSamplePositions, InMagnitude, NewFieldVectorResult);

			// Set the outputs
			SetValue<TArray<FVector>>(Context, MoveTemp(NewFieldVectorResult), &FieldVectorResult);
			SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
			SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

			return;
		}
		//
		// SampleIndices input connected, the selected elements from SamplePositions array
		// will be processed and FieldRemapOutput output will contain the remap info
		// IMPORTANT: Number of elements in SamplePositions and SampleIndices must be the same
		//
		else
		{
			if (InSamplePositions.Num() == InSampleIndices.Num())
			{
				TArray<FVector3f> NewSamplePositions;
				TArray<int32> NewRemapArray;

				FieldComputeRemap(InSamplePositions, InSampleIndices, NewSamplePositions, NewRemapArray);

				// Compute the field
				TArray<FVector> NewFieldVectorResult;
				RandomVectorFieldProcess(NewSamplePositions, InMagnitude, NewFieldVectorResult);

				// Set the outputs
				SetValue<TArray<FVector>>(Context, MoveTemp(NewFieldVectorResult), &FieldVectorResult);
				SetValue<TArray<int32>>(Context, MoveTemp(NewRemapArray), &FieldRemap);
				SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

				return;
			}
		}

		SetValue<TArray<FVector>>(Context, TArray<FVector>(), &FieldVectorResult);
		SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
		SetValue<int32>(Context, 0, &NumSamplePositions);
	}
}

// --------------------------------------------------------------------------------------------------------------------

static void NoiseFieldEvaluate(const TArray<FVector3f>& InSamplePositions,
	float InMinRange,
	float InMaxRange,
	const FTransform& InTransform,
	TFieldArrayView<float>& OutResultsView)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Setup the data for the field 
	FFieldExecutionDatas ExecutionDatas;

	FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, NumInSamplePositions);
	ExecutionDatas.SamplePositions.Init(FVector(0.f), NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; Idx++)
	{
		ExecutionDatas.SamplePositions[Idx] = (FVector)InSamplePositions[Idx];
	}

	FNoiseField NoiseField(InMinRange, InMaxRange, InTransform);

	FFieldContext FieldContext{
		ExecutionDatas,
		FFieldContext::UniquePointerMap(),
		0.0
	};

	// Evalute the field
	NoiseField.Evaluate(FieldContext, OutResultsView);
}

static void NoiseFieldProcess(const TArray<FVector3f>& InSamplePositions,
	float InMinRange,
	float InMaxRange,
	const FTransform& InTransform, 
	TArray<float>& OutFieldFloatResult)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Compute the field
	TArray<float> ResultsArray;
	ResultsArray.Init(0.f, NumInSamplePositions);
	TFieldArrayView<float> ResultsView(ResultsArray, 0, ResultsArray.Num());

	NoiseFieldEvaluate(InSamplePositions, InMinRange, InMaxRange, InTransform, ResultsView);

	// Set the outputs
	OutFieldFloatResult.Init(0.f, NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; ++Idx)
	{
		OutFieldFloatResult[Idx] = ResultsView[Idx];
	}
}

void FNoiseFieldDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<float>>(&FieldFloatResult) ||
		Out->IsA<TArray<int32>>(&FieldRemap) ||
		Out->IsA<int32>(&NumSamplePositions))
	{
		const TArray<FVector3f>& InSamplePositions = GetValue<TArray<FVector3f>>(Context, &SamplePositions);
		const FDataflowVertexSelection& InSampleIndices = GetValue<FDataflowVertexSelection>(Context, &SampleIndices);
		const float InMinRange = GetValue<float>(Context, &MinRange);
		const float InMaxRange = GetValue<float>(Context, &MaxRange);
		const FTransform InTransform = GetValue<FTransform>(Context, &Transform);

		//
		// SampleIndices input not connected, all the elements from SamplePositions array
		// will be processed and FieldRemapOutput output will be set empty
		//
		if (!IsConnected<FDataflowVertexSelection>(&SampleIndices))
		{
			// Compute the field
			TArray<float> NewFieldFloatResult;
			NoiseFieldProcess(InSamplePositions, InMinRange, InMaxRange, InTransform, NewFieldFloatResult);

			// Set the outputs
			SetValue<TArray<float>>(Context, MoveTemp(NewFieldFloatResult), &FieldFloatResult);
			SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
			SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

			return;
		}
		//
		// SampleIndices input connected, the selected elements from SamplePositions array
		// will be processed and FieldRemapOutput output will contain the remap info
		// IMPORTANT: Number of elements in SamplePositions and SampleIndices must be the same
		//
		else
		{
			if (InSamplePositions.Num() == InSampleIndices.Num())
			{
				TArray<FVector3f> NewSamplePositions;
				TArray<int32> NewRemapArray;

				FieldComputeRemap(InSamplePositions, InSampleIndices, NewSamplePositions, NewRemapArray);

				// Compute the field
				TArray<float> NewFieldFloatResult;
				NoiseFieldProcess(NewSamplePositions, InMinRange, InMaxRange, InTransform, NewFieldFloatResult);

				// Set the outputs
				SetValue<TArray<float>>(Context, MoveTemp(NewFieldFloatResult), &FieldFloatResult);
				SetValue<TArray<int32>>(Context, MoveTemp(NewRemapArray), &FieldRemap);
				SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

				return;
			}
		}

		SetValue<TArray<float>>(Context, TArray<float>(), &FieldFloatResult);
		SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
		SetValue<int32>(Context, 0, &NumSamplePositions);
	}
}

// --------------------------------------------------------------------------------------------------------------------

static void UniformIntegerFieldEvaluate(const TArray<FVector3f>& InSamplePositions,
	const int32 InMagnitude,
	TFieldArrayView<int32>& OutResultsView)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Setup the data for the field 
	FFieldExecutionDatas ExecutionDatas;

	FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, NumInSamplePositions);
	ExecutionDatas.SamplePositions.Init(FVector(0.f), NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; Idx++)
	{
		ExecutionDatas.SamplePositions[Idx] = (FVector)InSamplePositions[Idx];
	}

	FUniformInteger UniformIntegerField(InMagnitude);

	FFieldContext FieldContext{
		ExecutionDatas,
		FFieldContext::UniquePointerMap(),
		0.0
	};

	// Evalute the field
	UniformIntegerField.Evaluate(FieldContext, OutResultsView);
}

static void UniformIntegerFieldProcess(const TArray<FVector3f>& InSamplePositions,
	int32 InMagnitude,
	TArray<int32>& OutFieldFloatResult)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Compute the field
	TArray<int32> ResultsArray;
	ResultsArray.Init(0, NumInSamplePositions);
	TFieldArrayView<int32> ResultsView(ResultsArray, 0, ResultsArray.Num());

	UniformIntegerFieldEvaluate(InSamplePositions, InMagnitude, ResultsView);

	// Set the outputs
	OutFieldFloatResult.Init(0, NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; ++Idx)
	{
		OutFieldFloatResult[Idx] = ResultsView[Idx];
	}
}

void FUniformIntegerFieldDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<int32>>(&FieldIntResult) ||
		Out->IsA<TArray<int32>>(&FieldRemap) ||
		Out->IsA<int32>(&NumSamplePositions))
	{
		const TArray<FVector3f>& InSamplePositions = GetValue<TArray<FVector3f>>(Context, &SamplePositions);
		const FDataflowVertexSelection& InSampleIndices = GetValue<FDataflowVertexSelection>(Context, &SampleIndices);
		const int32 InMagnitude = GetValue<int32>(Context, &Magnitude);

		//
		// SampleIndices input not connected, all the elements from SamplePositions array
		// will be processed and FieldRemapOutput output will be set empty
		//
		if (!IsConnected<FDataflowVertexSelection>(&SampleIndices))
		{
			// Compute the field
			TArray<int32> NewFieldIntResult;
			UniformIntegerFieldProcess(InSamplePositions, InMagnitude, NewFieldIntResult);

			// Set the outputs
			SetValue<TArray<int32>>(Context, MoveTemp(NewFieldIntResult), &FieldIntResult);
			SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
			SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

			return;
		}
		//
		// SampleIndices input connected, the selected elements from SamplePositions array
		// will be processed and FieldRemapOutput output will contain the remap info
		// IMPORTANT: Number of elements in SamplePositions and SampleIndices must be the same
		//
		else
		{
			if (InSamplePositions.Num() == InSampleIndices.Num())
			{
				TArray<FVector3f> NewSamplePositions;
				TArray<int32> NewRemapArray;

				FieldComputeRemap(InSamplePositions, InSampleIndices, NewSamplePositions, NewRemapArray);

				// Compute the field
				TArray<int32> NewFieldIntResult;
				UniformIntegerFieldProcess(NewSamplePositions, InMagnitude, NewFieldIntResult);

				// Set the outputs
				SetValue<TArray<int32>>(Context, MoveTemp(NewFieldIntResult), &FieldIntResult);
				SetValue<TArray<int32>>(Context, MoveTemp(NewRemapArray), &FieldRemap);
				SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

				return;
			}
		}

		SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldIntResult);
		SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
		SetValue<int32>(Context, 0, &NumSamplePositions);
	}
}

// --------------------------------------------------------------------------------------------------------------------

static void WaveScalarFieldEvaluate(const TArray<FVector3f>& InSamplePositions,
	float InMagnitude,
	const FVector InPosition,
	float InWavelength,
	float InPeriod,
	const EDataflowWaveFunctionType InFunctionType,
	const EDataflowFieldFalloffType InFalloffType,
	TFieldArrayView<float>& OutResultsView)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Setup the data for the field 
	FFieldExecutionDatas ExecutionDatas;

	FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, NumInSamplePositions);
	ExecutionDatas.SamplePositions.Init(FVector(0.f), NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; Idx++)
	{
		ExecutionDatas.SamplePositions[Idx] = (FVector)InSamplePositions[Idx];
	}

	FWaveScalar WaveScalarField(InMagnitude, InPosition, InWavelength, InPeriod, (EWaveFunctionType)InFunctionType, (EFieldFalloffType)InFalloffType);

	FFieldContext FieldContext{
		ExecutionDatas,
		FFieldContext::UniquePointerMap(),
		0.0
	};

	// Evalute the field
	WaveScalarField.Evaluate(FieldContext, OutResultsView);
}

static void WaveScalarFieldProcess(const TArray<FVector3f>& InSamplePositions,
	float InMagnitude,
	const FVector InPosition,
	const FVector InTranslation,
	float InWavelength,
	float InPeriod,
	const EDataflowWaveFunctionType InFunctionType,
	const EDataflowFieldFalloffType InFalloffType,
	TArray<float>& OutFieldFloatResult)
{
	const int32 NumInSamplePositions = InSamplePositions.Num();

	// Compute the field
	TArray<float> ResultsArray;
	ResultsArray.Init(0.f, NumInSamplePositions);
	TFieldArrayView<float> ResultsView(ResultsArray, 0, ResultsArray.Num());

	WaveScalarFieldEvaluate(InSamplePositions,
		InMagnitude,
		InPosition + InTranslation,
		InWavelength,
		InPeriod,
		InFunctionType,
		InFalloffType,
		ResultsView);

	// Set the outputs
	OutFieldFloatResult.Init(0.f, NumInSamplePositions);

	for (int32 Idx = 0; Idx < NumInSamplePositions; ++Idx)
	{
		OutFieldFloatResult[Idx] = ResultsView[Idx];
	}
}

void FWaveScalarFieldDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<float>>(&FieldFloatResult) ||
		Out->IsA<TArray<int32>>(&FieldRemap) ||
		Out->IsA<int32>(&NumSamplePositions))
	{
		const TArray<FVector3f>& InSamplePositions = GetValue<TArray<FVector3f>>(Context, &SamplePositions);
		const FDataflowVertexSelection& InSampleIndices = GetValue<FDataflowVertexSelection>(Context, &SampleIndices);
		const float InMagnitude = GetValue<float>(Context, &Magnitude);
		const FVector InPosition = GetValue<FVector>(Context, &Position);
		const FVector InTranslation = GetValue<FVector>(Context, &Translation);
		const float InWavelength = GetValue<float>(Context, &Wavelength);
		const float InPeriod = GetValue<float>(Context, &Period);

		//
		// SampleIndices input not connected, all the elements from SamplePositions array
		// will be processed and FieldRemapOutput output will be set empty
		//
		if (!IsConnected<FDataflowVertexSelection>(&SampleIndices))
		{
			// Compute the field
			TArray<float> NewFieldFloatResult;
			WaveScalarFieldProcess(InSamplePositions, 
				InMagnitude,
				InPosition,
				InTranslation, 
				InWavelength,
				InPeriod,
				FunctionType, 
				FalloffType, 
				NewFieldFloatResult);

			// Set the outputs
			SetValue<TArray<float>>(Context, MoveTemp(NewFieldFloatResult), &FieldFloatResult);
			SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
			SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

			return;
		}
		//
		// SampleIndices input connected, the selected elements from SamplePositions array
		// will be processed and FieldRemapOutput output will contain the remap info
		// IMPORTANT: Number of elements in SamplePositions and SampleIndices must be the same
		//
		else
		{
			if (InSamplePositions.Num() == InSampleIndices.Num())
			{
				TArray<FVector3f> NewSamplePositions;
				TArray<int32> NewRemapArray;

				FieldComputeRemap(InSamplePositions, InSampleIndices, NewSamplePositions, NewRemapArray);

				// Compute the field
				TArray<float> NewFieldFloatResult;
				WaveScalarFieldProcess(NewSamplePositions,
					InMagnitude,
					InPosition,
					InTranslation,
					InWavelength,
					InPeriod,
					FunctionType,
					FalloffType,
					NewFieldFloatResult);

				// Set the outputs
				SetValue<TArray<float>>(Context, MoveTemp(NewFieldFloatResult), &FieldFloatResult);
				SetValue<TArray<int32>>(Context, MoveTemp(NewRemapArray), &FieldRemap);
				SetValue<int32>(Context, InSamplePositions.Num(), &NumSamplePositions);

				return;
			}
		}

		SetValue<TArray<float>>(Context, TArray<float>(), &FieldFloatResult);
		SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
		SetValue<int32>(Context, 0, &NumSamplePositions);
	}
}

// --------------------------------------------------------------------------------------------------------------------

static void SumScalarEvaluate(const TArray<float>& InFieldFloatLeft,
	const TArray<float>& InFieldFloatRight,
	TArray<float>& OutNewFieldFloatOutput,
	const int32 InNumSamplePositions,
	const EDataflowFloatFieldOperationType InOperation,
	const float InMagnitude,
	const bool bInSwapInputs)
{
	switch (InOperation)
	{
		case EDataflowFloatFieldOperationType::Dataflow_FloatFieldOperationType_Multiply:
			for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
			{
				OutNewFieldFloatOutput[Idx] = InFieldFloatLeft[Idx] * InFieldFloatRight[Idx];
			}
			break;
		case EDataflowFloatFieldOperationType::Dataflow_FloatFieldFalloffType_Divide:
			if (!bInSwapInputs)
			{
				for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
				{
					if (!FMath::IsNearlyZero(InFieldFloatRight[Idx]))
					{
						OutNewFieldFloatOutput[Idx] = InFieldFloatLeft[Idx] / InFieldFloatRight[Idx];
					}
				}
			}
			else
			{
				for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
				{
					if (!FMath::IsNearlyZero(InFieldFloatLeft[Idx]))
					{
						OutNewFieldFloatOutput[Idx] = InFieldFloatRight[Idx] / InFieldFloatLeft[Idx];
					}
				}
			}
			break;
		case EDataflowFloatFieldOperationType::Dataflow_FloatFieldFalloffType_Add:
			for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
			{
				OutNewFieldFloatOutput[Idx] = InFieldFloatLeft[Idx] + InFieldFloatRight[Idx];
			}
			break;
		case EDataflowFloatFieldOperationType::Dataflow_FloatFieldFalloffType_Substract:
			if (!bInSwapInputs)
			{
				for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
				{
					OutNewFieldFloatOutput[Idx] = InFieldFloatLeft[Idx] - InFieldFloatRight[Idx];
				}
			}
			else
			{
				for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
				{
					OutNewFieldFloatOutput[Idx] = InFieldFloatRight[Idx] - InFieldFloatLeft[Idx];
				}
			}
			break;
		case EDataflowFloatFieldOperationType::Dataflow_FloatFieldFalloffType_Min:
			for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
			{
				OutNewFieldFloatOutput[Idx] = FMath::Min(InFieldFloatLeft[Idx], InFieldFloatRight[Idx]);
			}
			break;
		case EDataflowFloatFieldOperationType::Dataflow_FloatFieldFalloffType_Max:

			for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
			{
				OutNewFieldFloatOutput[Idx] = FMath::Max(InFieldFloatLeft[Idx], InFieldFloatRight[Idx]);
			}
			break;
	}

	if (InMagnitude != 1.0)
	{
		for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
		{
			OutNewFieldFloatOutput[Idx] *= InMagnitude;
		}
	}
}


void FSumScalarFieldDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<float>>(&FieldFloatResult) ||
		Out->IsA<TArray<int32>>(&FieldRemap))
	{
		const TArray<float>& InFieldFloatLeft = GetValue<TArray<float>>(Context, &FieldFloatLeft);
		const TArray<int32>& InFieldRemapLeft = GetValue<TArray<int32>>(Context, &FieldRemapLeft);
		const TArray<float>& InFieldFloatRight = GetValue<TArray<float>>(Context, &FieldFloatRight);
		const TArray<int32>& InFieldRemapRight = GetValue<TArray<int32>>(Context, &FieldRemapRight);

		const int32 NumFieldFloatLeft = InFieldFloatLeft.Num();
		const int32 NumFieldRemapLeft = InFieldRemapLeft.Num();
		const int32 NumFieldFloatRight = InFieldFloatRight.Num();
		const int32 NumFieldRemapRight = InFieldRemapRight.Num();

		const bool IsFieldRemapLeftConnected = IsConnected<TArray<int32>>(&FieldRemapLeft);
		const bool IsFieldRemapRightConnected = IsConnected<TArray<int32>>(&FieldRemapRight);

		TArray<float> NewFieldFloatOutput;

		// Both Remap inputs are connected but they can be different sizes
		// Process the two FloatArray inputs and generate a new output remap
		if (IsFieldRemapLeftConnected && IsFieldRemapRightConnected)
		{
			if (NumFieldFloatLeft == NumFieldRemapLeft &&
				NumFieldFloatRight == NumFieldRemapRight)
			{
				int32 NewArraySize = FMath::Max(NumFieldFloatLeft, NumFieldFloatRight);

				NewFieldFloatOutput.Init(false, NewArraySize);

				TArray<int32> NewRemapArray;
				NewRemapArray.Init(false, NewArraySize);

				TArray<float> NewFloatArrayLeft, NewFloatArrayRight;
				NewFloatArrayLeft.Init(false, NewArraySize);
				NewFloatArrayRight.Init(false, NewArraySize);

				int32 NewSampleCount = 0;

				for (int32 IndexLeft = 0; IndexLeft < NumFieldFloatLeft; ++IndexLeft)
				{
					int32 SampleIndex = InFieldRemapLeft[IndexLeft];

					if (InFieldRemapRight.Contains(SampleIndex))
					{
						int32 IndexRight = InFieldRemapRight.Find(SampleIndex);

						NewRemapArray[NewSampleCount] = SampleIndex;

						NewFloatArrayLeft[NewSampleCount] = InFieldFloatLeft[IndexLeft];
						NewFloatArrayRight[NewSampleCount] = InFieldFloatRight[IndexRight];

						NewSampleCount++;
					}
				}

				NewRemapArray.SetNum(NewSampleCount, EAllowShrinking::Yes);
				NewFloatArrayLeft.SetNum(NewSampleCount, EAllowShrinking::Yes);
				NewFloatArrayRight.SetNum(NewSampleCount, EAllowShrinking::Yes);
				NewFieldFloatOutput.SetNum(NewSampleCount, EAllowShrinking::Yes);

				SumScalarEvaluate(NewFloatArrayLeft,
					NewFloatArrayRight,
					NewFieldFloatOutput,
					NewSampleCount,
					Operation,
					Magnitude,
					bSwapInputs);

				SetValue<TArray<float>>(Context, MoveTemp(NewFieldFloatOutput), &FieldFloatResult);
				SetValue<TArray<int32>>(Context, MoveTemp(NewRemapArray), &FieldRemap);

				return;
			}
		}
		// At least one of the Remap inputs are not connected
		// There is no remap information, just process the two FloatArray inputs
		// but they have to be the same size
		else
		{
			if (NumFieldFloatLeft == NumFieldFloatRight)
			{
				const int32 NumSamplePositions = NumFieldFloatLeft;

				NewFieldFloatOutput.Init(false, NumSamplePositions);

				SumScalarEvaluate(InFieldFloatLeft,
					InFieldFloatRight,
					NewFieldFloatOutput,
					NumSamplePositions,
					Operation,
					Magnitude,
					bSwapInputs);

				SetValue<TArray<float>>(Context, MoveTemp(NewFieldFloatOutput), &FieldFloatResult);
				SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);

				return;
			}
		}

		SetValue<TArray<float>>(Context, TArray<float>(), &FieldFloatResult);
		SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
	}
}

// --------------------------------------------------------------------------------------------------------------------

static void SumVectorEvaluate(const TArray<float>* InFieldFloat,
	const TArray<FVector>* InFieldVectorLeft,
	const TArray<FVector>* InFieldVectorRight,
	TArray<FVector>& OutNewFieldVectorResult,
	const int32 InNumSamplePositions,
	const EDataflowVectorFieldOperationType InOperation,
	const float InMagnitude,
	const bool bSwapVectorInputs)
{
	if (InFieldVectorLeft != nullptr && InFieldVectorRight != nullptr)
	{
		switch (InOperation)
		{
		case EDataflowVectorFieldOperationType::Dataflow_VectorFieldOperationType_Multiply:
			for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
			{
				OutNewFieldVectorResult[Idx] = (*InFieldVectorLeft)[Idx] * (*InFieldVectorRight)[Idx];
			}
			break;
		case EDataflowVectorFieldOperationType::Dataflow_VectorFieldFalloffType_Divide:
			if (!bSwapVectorInputs)
			{
				for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
				{
					if (!FMath::IsNearlyZero((*InFieldVectorRight)[Idx].X) &&
						!FMath::IsNearlyZero((*InFieldVectorRight)[Idx].Y) &&
						!FMath::IsNearlyZero((*InFieldVectorRight)[Idx].Z))
					{
						OutNewFieldVectorResult[Idx] = (*InFieldVectorLeft)[Idx] / (*InFieldVectorRight)[Idx];
					}
				}
			}
			else
			{
				for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
				{
					if (!FMath::IsNearlyZero((*InFieldVectorLeft)[Idx].X) &&
						!FMath::IsNearlyZero((*InFieldVectorLeft)[Idx].Y) &&
						!FMath::IsNearlyZero((*InFieldVectorLeft)[Idx].Z))
					{
						OutNewFieldVectorResult[Idx] = (*InFieldVectorRight)[Idx] / (*InFieldVectorLeft)[Idx];
					}
				}

			}
			break;
		case EDataflowVectorFieldOperationType::Dataflow_VectorFieldFalloffType_Add:
			for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
			{
				OutNewFieldVectorResult[Idx] = (*InFieldVectorLeft)[Idx] + (*InFieldVectorRight)[Idx];
			}
			break;
		case EDataflowVectorFieldOperationType::Dataflow_VectorFieldFalloffType_Substract:
			if (!bSwapVectorInputs)
			{
				for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
				{
					OutNewFieldVectorResult[Idx] = (*InFieldVectorLeft)[Idx] - (*InFieldVectorRight)[Idx];
				}
			}
			else
			{
				for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
				{
					OutNewFieldVectorResult[Idx] = (*InFieldVectorRight)[Idx] - (*InFieldVectorLeft)[Idx];
				}

			}
			break;
		case EDataflowVectorFieldOperationType::Dataflow_VectorFieldFalloffType_CrossProduct:
			if (!bSwapVectorInputs)
			{
				for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
				{
					OutNewFieldVectorResult[Idx] = (*InFieldVectorLeft)[Idx] ^ (*InFieldVectorRight)[Idx];
				}
			}
			else
			{
				for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
				{
					OutNewFieldVectorResult[Idx] = (*InFieldVectorRight)[Idx] ^ (*InFieldVectorLeft)[Idx];
				}

			}
			break;
		}
	}
	else if (InFieldVectorLeft != nullptr)
	{
		for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
		{
			OutNewFieldVectorResult[Idx] = (*InFieldVectorLeft)[Idx];
		}
	}
	else if (InFieldVectorRight != nullptr)
	{
		for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
		{
			OutNewFieldVectorResult[Idx] = (*InFieldVectorRight)[Idx];
		}
	}

	if (InFieldFloat != nullptr)
	{
		for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
		{
			OutNewFieldVectorResult[Idx] *= (*InFieldFloat)[Idx];
		}
	}

	if (InMagnitude != 1.0)
	{
		for (int32 Idx = 0; Idx < InNumSamplePositions; ++Idx)
		{
			OutNewFieldVectorResult[Idx] *= InMagnitude;
		}
	}
}

void FSumVectorFieldDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<FVector>>(&FieldVectorResult) ||
		Out->IsA<TArray<int32>>(&FieldRemap))
	{
		const TArray<float>& InFieldFloat = GetValue<TArray<float>>(Context, &FieldFloat);
		const TArray<int32>& InFieldFloatRemap = GetValue<TArray<int32>>(Context, &FieldFloatRemap);
		const TArray<FVector>& InFieldVectorLeft = GetValue<TArray<FVector>>(Context, &FieldVectorLeft);
		const TArray<int32>& InFieldRemapLeft = GetValue<TArray<int32>>(Context, &FieldRemapLeft);
		const TArray<FVector>& InFieldVectorRight = GetValue<TArray<FVector>>(Context, &FieldVectorRight);
		const TArray<int32>& InFieldRemapRight = GetValue<TArray<int32>>(Context, &FieldRemapRight);

		const int32 NumFieldFloat = InFieldFloat.Num();
		const int32 NumFieldFloatRemap = InFieldFloatRemap.Num();
		const int32 NumFieldVectorLeft = InFieldVectorLeft.Num();
		const int32 NumFieldRemapLeft = InFieldRemapLeft.Num();
		const int32 NumFieldVectorRight = InFieldVectorRight.Num();
		const int32 NumFieldRemapRight = InFieldRemapRight.Num();

		const bool IsFieldFloatConnected = IsConnected<TArray<float>>(&FieldFloat);
		const bool IsFieldFloatRemapConnected = IsConnected<TArray<int32>>(&FieldFloatRemap);
		const bool IsFieldVectorLeftConnected = IsConnected<TArray<FVector>>(&FieldVectorLeft);
		const bool IsFieldRemapLeftConnected = IsConnected<TArray<int32>>(&FieldRemapLeft);
		const bool IsFieldVectorRightConnected = IsConnected<TArray<FVector>>(&FieldVectorRight);
		const bool IsFieldRemapRightConnected = IsConnected<TArray<int32>>(&FieldRemapRight);

		TArray<FVector> NewFieldVectorResult;

		if (IsFieldFloatConnected && IsFieldVectorLeftConnected && !IsFieldVectorRightConnected)
		{
			// Both Remap inputs are connected but they can be different sizes
			// Process the two array inputs and generate a new output remap
			if (IsFieldFloatRemapConnected && IsFieldRemapLeftConnected)
			{
				if (NumFieldFloat == NumFieldFloatRemap &&
					NumFieldVectorLeft == NumFieldRemapLeft)
				{
					int32 NewArraySize = FMath::Max(NumFieldFloat, NumFieldVectorLeft);

					NewFieldVectorResult.Init(FVector(0.f), NewArraySize);

					TArray<int32> NewRemapArray;
					NewRemapArray.Init(false, NewArraySize);

					TArray<float> NewFloatArray;
					TArray<FVector> NewVectorArrayLeft;
					NewFloatArray.Init(false, NewArraySize);
					NewVectorArrayLeft.Init(FVector(0.f), NewArraySize);

					int32 NewSampleCount = 0;

					for (int32 IndexLeft = 0; IndexLeft < NumFieldFloat; ++IndexLeft)
					{
						int32 SampleIndex = InFieldFloatRemap[IndexLeft];

						if (InFieldRemapLeft.Contains(SampleIndex))
						{
							int32 IndexRight = InFieldRemapLeft.Find(SampleIndex);

							NewRemapArray[NewSampleCount] = SampleIndex;

							NewFloatArray[NewSampleCount] = InFieldFloat[IndexLeft];
							NewVectorArrayLeft[NewSampleCount] = InFieldVectorLeft[IndexRight];

							NewSampleCount++;
						}
					}

					NewRemapArray.SetNum(NewSampleCount, EAllowShrinking::Yes);
					NewFloatArray.SetNum(NewSampleCount, EAllowShrinking::Yes);
					NewVectorArrayLeft.SetNum(NewSampleCount, EAllowShrinking::Yes);
					NewFieldVectorResult.SetNum(NewSampleCount, EAllowShrinking::Yes);

					SumVectorEvaluate(&NewFloatArray,
						&NewVectorArrayLeft,
						nullptr,
						NewFieldVectorResult,
						NewSampleCount,
						Operation,
						Magnitude,
						bSwapVectorInputs);

					SetValue<TArray<FVector>>(Context, MoveTemp(NewFieldVectorResult), &FieldVectorResult);
					SetValue<TArray<int32>>(Context, MoveTemp(NewRemapArray), &FieldRemap);

					return;
				}
			}
			// At least one of the Remap inputs are not connected
			// There is no remap information, just process the two FloatArray inputs
			// but they have to be the same size
			else
			{
				if (NumFieldFloat == NumFieldVectorLeft)
				{
					const int32 NumSamplePositions = NumFieldFloat;

					NewFieldVectorResult.Init(FVector(0.f), NumSamplePositions);

					SumVectorEvaluate(&InFieldFloat,
						&InFieldVectorLeft,
						nullptr,
						NewFieldVectorResult,
						NumSamplePositions,
						Operation,
						Magnitude,
						bSwapVectorInputs);

					SetValue<TArray<FVector>>(Context, MoveTemp(NewFieldVectorResult), &FieldVectorResult);
					SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);

					return;
				}
			}
		}
		else if (IsFieldFloatConnected && !IsFieldVectorLeftConnected && IsFieldVectorRightConnected)
		{
			// Both Remap inputs are connected but they can be different sizes
			// Process the two array inputs and generate a new output remap
			if (IsFieldFloatRemapConnected && IsFieldRemapRightConnected)
			{
				if (NumFieldFloat == NumFieldFloatRemap &&
					NumFieldVectorRight == NumFieldRemapRight)
				{
					int32 NewArraySize = FMath::Max(NumFieldFloat, NumFieldVectorRight);

					NewFieldVectorResult.Init(FVector(0.f), NewArraySize);

					TArray<int32> NewRemapArray;
					NewRemapArray.Init(false, NewArraySize);

					TArray<float> NewFloatArray;
					TArray<FVector> NewVectorArrayRight;
					NewFloatArray.Init(false, NewArraySize);
					NewVectorArrayRight.Init(FVector(0.f), NewArraySize);

					int32 NewSampleCount = 0;

					for (int32 IndexLeft = 0; IndexLeft < NumFieldFloat; ++IndexLeft)
					{
						int32 SampleIndex = InFieldFloatRemap[IndexLeft];

						if (InFieldRemapRight.Contains(SampleIndex))
						{
							int32 IndexRight = InFieldRemapRight.Find(SampleIndex);

							NewRemapArray[NewSampleCount] = SampleIndex;

							NewFloatArray[NewSampleCount] = InFieldFloat[IndexLeft];
							NewVectorArrayRight[NewSampleCount] = InFieldVectorRight[IndexRight];

							NewSampleCount++;
						}
					}

					NewRemapArray.SetNum(NewSampleCount, EAllowShrinking::Yes);
					NewFloatArray.SetNum(NewSampleCount, EAllowShrinking::Yes);
					NewVectorArrayRight.SetNum(NewSampleCount, EAllowShrinking::Yes);
					NewFieldVectorResult.SetNum(NewSampleCount, EAllowShrinking::Yes);

					SumVectorEvaluate(&NewFloatArray,
						nullptr,
						&NewVectorArrayRight,
						NewFieldVectorResult,
						NewSampleCount,
						Operation,
						Magnitude,
						bSwapVectorInputs);

					SetValue<TArray<FVector>>(Context, MoveTemp(NewFieldVectorResult), &FieldVectorResult);
					SetValue<TArray<int32>>(Context, MoveTemp(NewRemapArray), &FieldRemap);

					return;
				}
			}
			// At least one of the Remap inputs are not connected
			// There is no remap information, just process the two FloatArray inputs
			// but they have to be the same size
			else
			{
				if (NumFieldFloat == NumFieldVectorRight)
				{
					const int32 NumSamplePositions = NumFieldFloat;

					NewFieldVectorResult.Init(FVector(0.f), NumSamplePositions);

					SumVectorEvaluate(&InFieldFloat,
						nullptr,
						&InFieldVectorRight,
						NewFieldVectorResult,
						NumSamplePositions,
						Operation,
						Magnitude,
						bSwapVectorInputs);

					SetValue<TArray<FVector>>(Context, MoveTemp(NewFieldVectorResult), &FieldVectorResult);
					SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);

					return;
				}
			}
		}
		else if (!IsFieldFloatConnected && IsFieldVectorLeftConnected && IsFieldVectorRightConnected)
		{
			// Both Remap inputs are connected but they can be different sizes
			// Process the two array inputs and generate a new output remap
			if (IsFieldRemapLeftConnected && IsFieldRemapRightConnected)
			{
				if (NumFieldVectorLeft == NumFieldRemapLeft &&
					NumFieldVectorRight == NumFieldRemapRight)
				{
					int32 NewArraySize = FMath::Max(NumFieldVectorLeft, NumFieldVectorRight);

					NewFieldVectorResult.Init(FVector(0.f), NewArraySize);

					TArray<int32> NewRemapArray;
					NewRemapArray.Init(false, NewArraySize);

					TArray<FVector> NewVectorArrayLeft, NewVectorArrayRight;
					NewVectorArrayLeft.Init(FVector(0.f), NewArraySize);
					NewVectorArrayLeft.Init(FVector(0.f), NewArraySize);

					int32 NewSampleCount = 0;

					for (int32 IndexLeft = 0; IndexLeft < NumFieldVectorLeft; ++IndexLeft)
					{
						int32 SampleIndex = InFieldRemapLeft[IndexLeft];

						if (InFieldRemapRight.Contains(SampleIndex))
						{
							int32 IndexRight = InFieldRemapRight.Find(SampleIndex);

							NewRemapArray[NewSampleCount] = SampleIndex;

							NewVectorArrayLeft[NewSampleCount] = InFieldVectorLeft[IndexLeft];
							NewVectorArrayRight[NewSampleCount] = InFieldVectorRight[IndexRight];

							NewSampleCount++;
						}
					}

					NewRemapArray.SetNum(NewSampleCount, EAllowShrinking::Yes);
					NewVectorArrayLeft.SetNum(NewSampleCount, EAllowShrinking::Yes);
					NewVectorArrayRight.SetNum(NewSampleCount, EAllowShrinking::Yes);
					NewFieldVectorResult.SetNum(NewSampleCount, EAllowShrinking::Yes);

					SumVectorEvaluate(nullptr,
						&NewVectorArrayLeft,
						&NewVectorArrayRight,
						NewFieldVectorResult,
						NewSampleCount,
						Operation,
						Magnitude,
						bSwapVectorInputs);

					SetValue<TArray<FVector>>(Context, MoveTemp(NewFieldVectorResult), &FieldVectorResult);
					SetValue<TArray<int32>>(Context, MoveTemp(NewRemapArray), &FieldRemap);

					return;
				}
			}
			// At least one of the Remap inputs are not connected
			// There is no remap information, just process the two FloatArray inputs
			// but they have to be the same size
			else
			{
				if (NumFieldVectorLeft == NumFieldVectorRight)
				{
					const int32 NumSamplePositions = NumFieldVectorLeft;

					NewFieldVectorResult.Init(FVector(0.f), NumSamplePositions);

					SumVectorEvaluate(nullptr,
						&InFieldVectorLeft,
						&InFieldVectorRight,
						NewFieldVectorResult,
						NumSamplePositions,
						Operation,
						Magnitude,
						bSwapVectorInputs);

					SetValue<TArray<FVector>>(Context, MoveTemp(NewFieldVectorResult), &FieldVectorResult);
					SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);

					return;
				}
			}
		}
		else if (IsFieldFloatConnected && IsFieldVectorLeftConnected && IsFieldVectorRightConnected)
		{
			// All 3 Remap inputs are connected but they can be different sizes
			// Process the 3 array inputs and generate a new output remap
			if (IsFieldFloatRemapConnected && IsFieldRemapLeftConnected && IsFieldRemapRightConnected)
			{
				if (NumFieldFloat == NumFieldFloatRemap && 
					NumFieldVectorLeft == NumFieldRemapLeft &&
					NumFieldVectorRight == NumFieldRemapRight)
				{
					int32 NewArraySize = FMath::Max3(NumFieldFloat, NumFieldVectorLeft, NumFieldVectorRight);

					NewFieldVectorResult.Init(FVector(0.f), NewArraySize);

					TArray<int32> NewRemapArray;
					NewRemapArray.Init(false, NewArraySize);

					TArray<float> NewFloatArray;
					TArray<FVector> NewVectorArrayLeft, NewVectorArrayRight;
					NewFloatArray.Init(false, NewArraySize);
					NewVectorArrayLeft.Init(FVector(0.f), NewArraySize);
					NewVectorArrayRight.Init(FVector(0.f), NewArraySize);

					int32 NewSampleCount = 0;

					for (int32 IndexLeft = 0; IndexLeft < NumFieldFloat; ++IndexLeft)
					{
						int32 SampleIndex = InFieldFloatRemap[IndexLeft];

						if (InFieldRemapLeft.Contains(SampleIndex) &&
							InFieldRemapRight.Contains(SampleIndex))
						{
							int32 IndexRight1 = InFieldRemapLeft.Find(SampleIndex);
							int32 IndexRight2 = InFieldRemapRight.Find(SampleIndex);

							NewRemapArray[NewSampleCount] = SampleIndex;

							NewFloatArray[NewSampleCount] = InFieldFloat[IndexLeft];
							NewVectorArrayLeft[NewSampleCount] = InFieldVectorLeft[IndexRight1];
							NewVectorArrayRight[NewSampleCount] = InFieldVectorRight[IndexRight2];

							NewSampleCount++;
						}
					}

					NewRemapArray.SetNum(NewSampleCount, EAllowShrinking::Yes);
					NewFloatArray.SetNum(NewSampleCount, EAllowShrinking::Yes);
					NewVectorArrayLeft.SetNum(NewSampleCount, EAllowShrinking::Yes);
					NewVectorArrayRight.SetNum(NewSampleCount, EAllowShrinking::Yes);
					NewFieldVectorResult.SetNum(NewSampleCount, EAllowShrinking::Yes);

					SumVectorEvaluate(&NewFloatArray,
						&NewVectorArrayLeft,
						&NewVectorArrayRight,
						NewFieldVectorResult,
						NewSampleCount,
						Operation,
						Magnitude,
						bSwapVectorInputs);

					SetValue<TArray<FVector>>(Context, MoveTemp(NewFieldVectorResult), &FieldVectorResult);
					SetValue<TArray<int32>>(Context, MoveTemp(NewRemapArray), &FieldRemap);

					return;
				}
			}
			// At least one of the Remap inputs are not connected
			// There is no remap information, just process the two FloatArray inputs
			// but they have to be the same size
			else
			{
				if (NumFieldFloat == NumFieldVectorLeft &&
					NumFieldFloat == NumFieldVectorRight)
				{
					const int32 NumSamplePositions = NumFieldFloat;

					NewFieldVectorResult.Init(FVector(0.f), NumSamplePositions);

					SumVectorEvaluate(&InFieldFloat,
						&InFieldVectorLeft,
						&InFieldVectorRight,
						NewFieldVectorResult,
						NumSamplePositions,
						Operation,
						Magnitude,
						bSwapVectorInputs);

					SetValue<TArray<FVector>>(Context, MoveTemp(NewFieldVectorResult), &FieldVectorResult);
					SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);

					return;
				}
			}
		}

		SetValue<TArray<FVector>>(Context, TArray<FVector>(), &FieldVectorResult);
		SetValue<TArray<int32>>(Context, TArray<int32>(), &FieldRemap);
	}
}

// ----------------------------------------------------------------------------

void FFieldMakeDenseFloatArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<float>>(&FieldFloatResult))
	{
		const TArray<float>& InFieldFloatInput = GetValue<TArray<float>>(Context, &FieldFloatInput);
		const TArray<int32>& InFieldRemap = GetValue<TArray<int32>>(Context, &FieldRemap);
		const int32 InNumSamplePositions = GetValue<int32>(Context, &NumSamplePositions);

		if (InFieldFloatInput.Num() == InFieldRemap.Num() && InNumSamplePositions > 0)
		{
			TArray<float> NewFieldFloatOutput;
			NewFieldFloatOutput.Init(Default, InNumSamplePositions);

			for (int32 Idx = 0; Idx < InFieldRemap.Num(); ++Idx)
			{
				int32 RemapIdx = InFieldRemap[Idx];
				if (RemapIdx < InNumSamplePositions)
				{
					NewFieldFloatOutput[InFieldRemap[Idx]] = InFieldFloatInput[Idx];
				}
			}

			SetValue<TArray<float>>(Context, MoveTemp(NewFieldFloatOutput), &FieldFloatResult);

			return;
		}

		SetValue<TArray<float>>(Context, TArray<float>(), &FieldFloatResult);
	}
}



