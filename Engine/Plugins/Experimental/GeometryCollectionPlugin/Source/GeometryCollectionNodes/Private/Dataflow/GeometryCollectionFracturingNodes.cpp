// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionFracturingNodes.h"
#include "Dataflow/DataflowCore.h"
#if WITH_EDITOR
#include "Dataflow/DataflowRenderingViewMode.h"
#endif

#include "Engine/StaticMesh.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionEngineUtility.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealTypePrivate.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "StaticMeshAttributes.h"
#include "DynamicMeshEditor.h"
#include "Operations/MeshBoolean.h"

#include "EngineGlobals.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "Voronoi/Voronoi.h"
#include "PlanarCut.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "FractureEngineClustering.h"
#include "FractureEngineSelection.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"
#include "GeometryCollection/Facades/CollectionBoundsFacade.h"
#include "Dataflow/DataflowSelection.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "UDynamicMesh.h"
#include "MeshDescription.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "StaticMeshAttributes.h"
#include "Dataflow/DataflowSimpleDebugDrawMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionFracturingNodes)

namespace UE::Dataflow
{

	void GeometryCollectionFracturingNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FUniformScatterPointsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FUniformScatterPointsDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRadialScatterPointsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRadialScatterPointsDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGridScatterPointsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FClusterScatterPointsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVoronoiFractureDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVoronoiFractureDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FPlaneCutterDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FPlaneCutterDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FExplodedViewDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSliceCutterDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBrickCutterDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMeshCutterDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FUniformFractureDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVisualizeFractureDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FTransformPointsDataflowNode)
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAppendPointsDataflowNode)

//		Commented out until we decide how to make generic data setter nodes
//		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetFloatAttributeDataflowNode);
	}
}

FClusterScatterPointsDataflowNode::FClusterScatterPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&BoundingBox);

	RegisterInputConnection(&NumberClustersMin).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&NumberClustersMax).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&PointsPerClusterMin).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&PointsPerClusterMax).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ClusterRadiusFractionMin).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ClusterRadiusFractionMax).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ClusterRadiusOffset).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RandomSeed).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Points);
}

void FClusterScatterPointsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Points))
	{
		const FBox& Bounds = GetValue<FBox>(Context, &BoundingBox);
		if (Bounds.GetVolume() > 0.f)
		{
			FRandomStream RandStream(GetValue(Context, &RandomSeed));

			const int32 InNumberClustersMin = FMath::Max(1, GetValue(Context, &NumberClustersMin));
			const int32 InNumberClustersMax = FMath::Max(InNumberClustersMin, GetValue(Context, &NumberClustersMax));
			const int32 ClusterCount = RandStream.RandRange(InNumberClustersMin, InNumberClustersMax);
			const FVector Extent(Bounds.Max - Bounds.Min);

			TArray<FVector> ClusterCenters;
			ClusterCenters.Reserve(ClusterCount);
			for (int32 ClusterIdx = 0; ClusterIdx < ClusterCount; ++ClusterIdx)
			{
				ClusterCenters.Emplace(Bounds.Min + FVector(RandStream.FRand(), RandStream.FRand(), RandStream.FRand()) * Extent);
			}



			const int32 InPointsPerClusterMin = FMath::Max(0, GetValue(Context, &PointsPerClusterMin));
			const int32 InPointsPerClusterMax = FMath::Max(InPointsPerClusterMin, GetValue(Context, &PointsPerClusterMax));
			const double InClusterRadiusOffset = (double)GetValue(Context, &ClusterRadiusOffset);
			const double InClusterRadiusFractionMin = FMath::Max(0, (double)GetValue(Context, &ClusterRadiusFractionMin));
			const double InClusterRadiusFractionMax = FMath::Max(InClusterRadiusFractionMin, (double)GetValue(Context, &ClusterRadiusFractionMax));
			const double BoundsSize = Bounds.GetExtent().GetAbsMax();

			TArray<FVector> NewPoints;
			NewPoints.Reserve(ClusterCount * FMath::CeilToInt32(double(InPointsPerClusterMin + InPointsPerClusterMax) * .5));
			for (int32 CenterIdx = 0; CenterIdx < ClusterCenters.Num(); ++CenterIdx)
			{
				const int32 SubPointCount = RandStream.RandRange(InPointsPerClusterMin, InPointsPerClusterMax);

				for (int32 SubPointIdx = 0; SubPointIdx < SubPointCount; ++SubPointIdx)
				{
					FVector V(RandStream.VRand());
					V.Normalize();
					V *= InClusterRadiusOffset + (RandStream.FRandRange(InClusterRadiusFractionMin, InClusterRadiusFractionMax) * BoundsSize);
					V += ClusterCenters[CenterIdx];
					NewPoints.Emplace(V);
				}
			}

			SetValue(Context, MoveTemp(NewPoints), &Points);
		}
		else
		{
			// ERROR: Invalid BoundingBox input
			SetValue(Context, TArray<FVector>(), &Points);
		}
	}
}

void FUniformScatterPointsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<FVector>>(&Points))
	{
		const FBox& BBox = GetValue<FBox>(Context, &BoundingBox);
		if (BBox.GetVolume() > 0.f)
		{
			FRandomStream RandStream(GetValue<float>(Context, &RandomSeed));

			const FVector Extent(BBox.Max - BBox.Min);
			const int32 NumPoints = RandStream.RandRange(GetValue<int32>(Context, &MinNumberOfPoints), GetValue<int32>(Context, &MaxNumberOfPoints));

			TArray<FVector> PointsArr;
			PointsArr.Reserve(NumPoints);
			for (int32 Idx = 0; Idx < NumPoints; ++Idx)
			{
				PointsArr.Emplace(BBox.Min + FVector(RandStream.FRand(), RandStream.FRand(), RandStream.FRand()) * Extent);
			}

			SetValue(Context, MoveTemp(PointsArr), &Points);
		}
		else
		{
			// ERROR: Invalid BoundingBox input
			SetValue(Context, TArray<FVector>(), &Points);
		}
	}
}

void FUniformScatterPointsDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<FVector>>(&Points))
	{
		const FBox& BBox = GetValue<FBox>(Context, &BoundingBox);
		if (BBox.GetVolume() > 0.f)
		{
			FRandomStream RandStream(GetValue<int32>(Context, &RandomSeed));

			const FVector Extent(BBox.Max - BBox.Min);
			const int32 NumPoints = RandStream.RandRange(GetValue<int32>(Context, &MinNumberOfPoints), GetValue<int32>(Context, &MaxNumberOfPoints));

			TArray<FVector> PointsArr;
			PointsArr.Reserve(NumPoints);
			for (int32 Idx = 0; Idx < NumPoints; ++Idx)
			{
				PointsArr.Emplace(BBox.Min + FVector(RandStream.FRand(), RandStream.FRand(), RandStream.FRand()) * Extent);
			}

			SetValue(Context, MoveTemp(PointsArr), &Points);
		}
		else
		{
			// ERROR: Invalid BoundingBox input
			SetValue(Context, TArray<FVector>(), &Points);
		}
	}
}

void FRadialScatterPointsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<FVector>>(&Points))
	{
		const FVector::FReal RadialStep = GetValue<float>(Context, &Radius) / GetValue<int32>(Context, &RadialSteps);
		const FVector::FReal AngularStep = 2 * PI / GetValue<int32>(Context, &AngularSteps);

		FRandomStream RandStream(GetValue<float>(Context, &RandomSeed));
		FVector UpVector(GetValue<FVector>(Context, &Normal));
		UpVector.Normalize();
		FVector BasisX, BasisY;
		UpVector.FindBestAxisVectors(BasisX, BasisY);

		TArray<FVector> PointsArr;

		FVector::FReal Len = RadialStep * .5;
		for (int32 ii = 0; ii < GetValue<int32>(Context, &RadialSteps); ++ii, Len += RadialStep)
		{
			FVector::FReal Angle = FMath::DegreesToRadians(GetValue<float>(Context, &AngleOffset));
			for (int32 kk = 0; kk < AngularSteps; ++kk, Angle += AngularStep)
			{
				FVector RotatingOffset = Len * (FMath::Cos(Angle) * BasisX + FMath::Sin(Angle) * BasisY);
				PointsArr.Emplace(GetValue<FVector>(Context, &Center) + RotatingOffset + (RandStream.VRand() * RandStream.FRand() * Variability));
			}
		}

		SetValue(Context, MoveTemp(PointsArr), &Points);
	}
}

void FRadialScatterPointsDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<FVector>>(&Points))
	{
		const FBox InBoundingBox = GetValue<FBox>(Context, &BoundingBox);
		const FVector InCenter = GetValue<FVector>(Context, &Center);
		const FVector InNormal = GetValue<FVector>(Context, &Normal);
		const int32 InRandomSeed = GetValue<int32>(Context, &RandomSeed);
		const int32 InAngularSteps = GetValue<int32>(Context, &AngularSteps);
		const float InAngleOffset = GetValue<float>(Context, &AngleOffset);
		const float InAngularNoise = GetValue<float>(Context, &AngularNoise);
		const float InRadius = GetValue<float>(Context, &Radius);
		const int32 InRadialSteps = GetValue<int32>(Context, &RadialSteps);
		const float InRadialStepExponent = GetValue<float>(Context, &RadialStepExponent);
		const float InRadialMinStep = GetValue<float>(Context, &RadialMinStep);
		const float InRadialNoise = GetValue<float>(Context, &RadialNoise);
		const float InRadialVariability = GetValue<float>(Context, &RadialVariability);
		const float InAngularVariability = GetValue<float>(Context, &AngularVariability);
		const float InAxialVariability = GetValue<float>(Context, &AxialVariability);

		TArray<FVector> PointsArr;

		const FVector::FReal AngularStep = 2 * PI / InAngularSteps;

		FVector CenterVal(InBoundingBox.GetCenter() + InCenter);

		FRandomStream RandStream(InRandomSeed);
		FVector UpVector(InNormal);
		UpVector.Normalize();
		FVector BasisX, BasisY;
		UpVector.FindBestAxisVectors(BasisX, BasisY);

		// Precompute consistent noise for each angular step
		TArray<FVector::FReal> AngleStepOffsets;
		AngleStepOffsets.SetNumUninitialized(InAngularSteps);
		for (int32 AngleIdx = 0; AngleIdx < InAngularSteps; ++AngleIdx)
		{
			AngleStepOffsets[AngleIdx] = FMath::DegreesToRadians(RandStream.FRandRange(-1, 1) * InAngularNoise);
		}

		// Compute radial positions following an (idx+1)^exp curve, and then re-normalize back to the Radius range
		TArray<FVector::FReal> RadialPositions;
		RadialPositions.SetNumUninitialized(InRadialSteps);
		FVector::FReal StepOffset = 0;
		for (int32 RadIdx = 0; RadIdx < InRadialSteps; ++RadIdx)
		{
			FVector::FReal RadialPos = FMath::Pow(RadIdx + 1, InRadialStepExponent) + StepOffset;
			if (RadIdx == 0)
			{
				// Note we bring the first point a half-step toward the center, and shift all subsequent points accordingly
				// so that for Exponent==1, the step from center to first boundary is the same distance as the step between each boundary
				// (this is only necessary because there is no Voronoi site at the center)
				RadialPos *= .5;
				StepOffset = -RadialPos;
			}

			RadialPositions[RadIdx] = RadialPos;
		}
		// Normalize positions so that the diagram fits in the target radius
		FVector::FReal RadialPosNorm = InRadius / RadialPositions.Last();
		for (FVector::FReal& RadialPos : RadialPositions)
		{
			RadialPos = RadialPos * RadialPosNorm;
		}
		// Add radial noise 
		for (int32 RadIdx = 0; RadIdx < InRadialSteps; ++RadIdx)
		{
			FVector::FReal& RadialPos = RadialPositions[RadIdx];
			// Offset by RadialNoise, but don't allow noise to take the value below 0
			RadialPos += RandStream.FRandRange(-FMath::Min(RadialPos, InRadialNoise), InRadialNoise);
		}
		// make sure the positions remain in increasing order
		RadialPositions.Sort();
		// Adjust positions so they are never closer than the RadialMinStep
		FVector::FReal LastRadialPos = 0;
		for (int32 RadIdx = 0; RadIdx < InRadialSteps; ++RadIdx)
		{
			FVector::FReal MinStep = InRadialMinStep;
			if (RadIdx == 0)
			{
				MinStep *= .5;
			}
			if (RadialPositions[RadIdx] - LastRadialPos < MinStep)
			{
				RadialPositions[RadIdx] = LastRadialPos + MinStep;
			}
			LastRadialPos = RadialPositions[RadIdx];
		}

		// Add a bit of noise to work around failure case in Voro++
		// TODO: fix the failure case in Voro++ and remove this
		float MinRadialVariability = InRadius > 1.f ? .0001f : 0.f;
		float UseRadialVariability = FMath::Max(MinRadialVariability, InRadialVariability);

		// Create the radial Voronoi sites
		for (int32 ii = 0; ii < InRadialSteps; ++ii)
		{
			FVector::FReal Len = RadialPositions[ii];
			FVector::FReal Angle = FMath::DegreesToRadians(InAngleOffset);
			for (int32 kk = 0; kk < InAngularSteps; ++kk, Angle += AngularStep)
			{
				// Add the global noise and the per-point noise into the angle
				FVector::FReal UseAngle = Angle + AngleStepOffsets[kk] + FMath::DegreesToRadians(RandStream.FRand() * InAngularVariability);
				// Add per point noise into the radial position
				FVector::FReal UseRadius = Len + FVector::FReal(RandStream.FRand() * UseRadialVariability);
				FVector RotatingOffset = UseRadius * (FMath::Cos(UseAngle) * BasisX + FMath::Sin(UseAngle) * BasisY);
				PointsArr.Emplace(CenterVal + RotatingOffset + UpVector * (RandStream.FRandRange(-1, 1) * InAxialVariability));
			}
		}

		SetValue(Context, MoveTemp(PointsArr), &Points);
	}
}

void FGridScatterPointsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<FVector>>(&Points))
	{
		const FBox& BBox = GetValue<FBox>(Context, &BoundingBox);
		if (BBox.GetVolume() > 0.f)
		{
			const FVector Extent(BBox.Max - BBox.Min);
			// Note: Should match ClampMax in the UI. Do not raise above 1290 to avoid overflowing the TArray.
			// (A smaller limit is preferrable because the rendering / later processing will likely not want to handle that many points.)
			constexpr int32 MaxPointsPerDim = 200;
			const int32 NumPointsInX = FMath::Clamp(GetValue<int32>(Context, &NumberOfPointsInX), 0, MaxPointsPerDim);
			const int32 NumPointsInY = FMath::Clamp(GetValue<int32>(Context, &NumberOfPointsInY), 0, MaxPointsPerDim);
			const int32 NumPointsInZ = FMath::Clamp(GetValue<int32>(Context, &NumberOfPointsInZ), 0, MaxPointsPerDim);
			
			if (NumPointsInX >= 1 && NumPointsInY >= 1 && NumPointsInZ >= 1)
			{
				const int32 NumPoints = NumPointsInX * NumPointsInY * NumPointsInZ;
				const float dX = Extent.X / (float)NumPointsInX;
				const float dY = Extent.Y / (float)NumPointsInY;
				const float dZ = Extent.Z / (float)NumPointsInZ;

				FRandomStream RandStream(GetValue<int32>(Context, &RandomSeed));

				TArray<FVector> PointsArr;
				PointsArr.Reserve(NumPoints);
				for (int32 Idx_X = 0; Idx_X < NumPointsInX; ++Idx_X)
				{
					for (int32 Idx_Y = 0; Idx_Y < NumPointsInY; ++Idx_Y)
					{
						for (int32 Idx_Z = 0; Idx_Z < NumPointsInZ; ++Idx_Z)
						{
							FVector RandomDisplacement = FVector(RandStream.FRandRange(-1.f, 1.f) * GetValue<float>(Context, &MaxRandomDisplacementX),
								RandStream.FRandRange(-1.f, 1.f) * GetValue<float>(Context, &MaxRandomDisplacementY),
								RandStream.FRandRange(-1.f, 1.f) * GetValue<float>(Context, &MaxRandomDisplacementZ));

							PointsArr.Emplace(BBox.Min.X + 0.5f * dX + (float)Idx_X * dX + RandomDisplacement.X,
								BBox.Min.Y + 0.5f * dY + (float)Idx_Y * dY + RandomDisplacement.Y,
								BBox.Min.Z + 0.5f * dZ + (float)Idx_Z * dZ + RandomDisplacement.Z);
						}
					}
				}

				SetValue(Context, MoveTemp(PointsArr), &Points);
			}
			else
			{
				// ERROR: Invalid number of points
				SetValue(Context, TArray<FVector>(), &Points);
			}
		}
		else
		{
			// ERROR: Invalid BoundingBox input
			SetValue(Context, TArray<FVector>(), &Points);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////

FTransformPointsDataflowNode::FTransformPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Points);
	RegisterInputConnection(&Transform);
	RegisterOutputConnection(&Points, &Points);
}

void FTransformPointsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Points))
	{
		const FTransform& InTransform = GetValue(Context, &Transform);
		TArray<FVector> OutPoints = GetValue(Context, &Points);
		for (FVector& Point : OutPoints)
		{
			Point = InTransform.TransformPosition(Point);
		}
		SetValue(Context, MoveTemp(OutPoints), &Points);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////

FAppendPointsDataflowNode::FAppendPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&PointsA);
	RegisterInputConnection(&PointsB);
	RegisterOutputConnection(&Points, &PointsA);
}

void FAppendPointsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Points))
	{
		const TArray<FVector>& InPointsA = GetValue(Context, &PointsA);
		const TArray<FVector>& InPointsB = GetValue(Context, &PointsB);
		TArray<FVector> OutPoints;
		OutPoints.Append(InPointsA);
		OutPoints.Append(InPointsB);
		SetValue(Context, MoveTemp(OutPoints), &Points);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////
void FVoronoiFractureDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		const FDataflowTransformSelection& InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);

		if (IsConnected<FDataflowTransformSelection>(&TransformSelection))
		{
			if (InTransformSelection.AnySelected())
			{
				FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

				FFractureEngineFracturing::VoronoiFracture(InCollection,
					InTransformSelection,
					GetValue<TArray<FVector>>(Context, &Points),
					FTransform::Identity,
					(int32)GetValue<float>(Context, &RandomSeed),
					GetValue<float>(Context, &ChanceToFracture),
					true,
					GetValue<float>(Context, &Grout),
					GetValue<float>(Context, &Amplitude),
					GetValue<float>(Context, &Frequency),
					GetValue<float>(Context, &Persistence),
					GetValue<float>(Context, &Lacunarity),
					GetValue<int32>(Context, &OctaveNumber),
					GetValue<float>(Context, &PointSpacing),
					AddSamplesForCollision,
					GetValue<float>(Context, &CollisionSampleSpacing));

				SetValue(Context, MoveTemp(InCollection), &Collection);

				return;
			}
		}

		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		SetValue(Context, InCollection, &Collection);
	}
}

void FVoronoiFractureDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection) ||
		Out->IsA<FDataflowTransformSelection>(&TransformSelection) ||
		Out->IsA<FDataflowTransformSelection>(&NewGeometryTransformSelection))
	{
		FDataflowTransformSelection InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);

		//
		// If not connected select everything by default
		//
		if (!IsConnected<FDataflowTransformSelection>(&TransformSelection))
		{
			const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

			GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
			const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectAll();

			FDataflowTransformSelection NewTransformSelection;
			NewTransformSelection.InitializeFromCollection(InCollection, false);
			NewTransformSelection.SetFromArray(SelectionArr);

			InTransformSelection = NewTransformSelection;
		}

		if (InTransformSelection.AnySelected())
		{
			FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

			int32 ResultGeometryIndex = FFractureEngineFracturing::VoronoiFracture(InCollection,
				InTransformSelection,
				GetValue<TArray<FVector>>(Context, &Points),
				GetValue<FTransform>(Context, &Transform),
				0, // RandomSeed is not used in Voronoi fracture, it is used in the source point generation
				GetValue<float>(Context, &ChanceToFracture),
				FIslandSplitSettings(SplitIslands, (double)CloseVertexDistance, (double)VertexToSurfaceBridgeDistance),
				GetValue<float>(Context, &Grout),
				GetValue<float>(Context, &Amplitude),
				GetValue<float>(Context, &Frequency),
				GetValue<float>(Context, &Persistence),
				GetValue<float>(Context, &Lacunarity),
				GetValue<int32>(Context, &OctaveNumber),
				GetValue<float>(Context, &PointSpacing),
				AddSamplesForCollision,
				GetValue<float>(Context, &CollisionSampleSpacing));

			FDataflowTransformSelection NewSelection;
			FDataflowTransformSelection OriginalSelection;

			if (ResultGeometryIndex != INDEX_NONE)
			{
				if (InCollection.HasAttribute("TransformIndex", FGeometryCollection::GeometryGroup))
				{
					const TManagedArray<int32>& GeometryToTransformIndices = InCollection.GetAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);

					NewSelection.InitializeFromCollection(InCollection, false);
					OriginalSelection.InitializeFromCollection(InCollection, false);

					// The newly fractured pieces are added to the end of the transform array (starting position is ResultGeometryIndex)
					for (int32 GeometryIdx = ResultGeometryIndex; GeometryIdx < GeometryToTransformIndices.Num(); ++GeometryIdx)
					{
						const int32 TransformIdx = GeometryToTransformIndices[GeometryIdx];
						NewSelection.SetSelected(TransformIdx);
					}

					for (int32 TransformIdx = 0; TransformIdx < InTransformSelection.Num(); ++TransformIdx)
					{
						if (InTransformSelection.IsSelected(TransformIdx))
						{
							OriginalSelection.SetSelected(TransformIdx);
						}
					}
				}
			}

			SetValue(Context, MoveTemp(InCollection), &Collection);
			SetValue(Context, OriginalSelection, &TransformSelection);
			SetValue(Context, NewSelection, &NewGeometryTransformSelection);

			return;
		}

		SafeForwardInput(Context, &Collection, &Collection);
		SetValue(Context, InTransformSelection, &TransformSelection);
		SetValue(Context, FDataflowTransformSelection(), &NewGeometryTransformSelection);
	}
}

void FPlaneCutterDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		const FDataflowTransformSelection& InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);

		if (IsConnected<FDataflowTransformSelection>(&TransformSelection))
		{
			if (InTransformSelection.AnySelected())
			{
				FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

				FFractureEngineFracturing::PlaneCutter(InCollection,
					InTransformSelection,
					GetValue<FBox>(Context, &BoundingBox),
					FTransform::Identity,
					NumPlanes,
					(int32)GetValue<float>(Context, &RandomSeed),
					1.f,
					true,
					GetValue<float>(Context, &Grout),
					GetValue<float>(Context, &Amplitude),
					GetValue<float>(Context, &Frequency),
					GetValue<float>(Context, &Persistence),
					GetValue<float>(Context, &Lacunarity),
					GetValue<int32>(Context, &OctaveNumber),
					GetValue<float>(Context, &PointSpacing),
					GetValue<bool>(Context, &AddSamplesForCollision),
					GetValue<float>(Context, &CollisionSampleSpacing));

				SetValue(Context, MoveTemp(InCollection), &Collection);

				return;
			}
		}

		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		SetValue(Context, InCollection, &Collection);
	}
}

/*--------------------------------------------------------------------------------------------------------*/

static FLinearColor GetRandomColor(const int32 RandomSeed, int32 Idx)
{
	FRandomStream RandomStream(RandomSeed * 23 + Idx * 4078);
	
	const uint8 R = static_cast<uint8>(RandomStream.FRandRange(128, 255));
	const uint8 G = static_cast<uint8>(RandomStream.FRandRange(128, 255));
	const uint8 B = static_cast<uint8>(RandomStream.FRandRange(128, 255));

	return FLinearColor(FColor(R, G, B, 255));
}

void FPlaneCutterDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection) ||
		Out->IsA(&TransformSelection) ||
		Out->IsA(&NewGeometryTransformSelection))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);
		FBox InBoundingBox = GetValue(Context, &BoundingBox);

		//
		// If not connected get bounding box of incoming collection
		//
		if (!IsConnected(&BoundingBox))
		{
			GeometryCollection::Facades::FBoundsFacade BoundsFacade(InCollection);
			const FBox& BoundingBoxInCollectionSpace = BoundsFacade.GetBoundingBoxInCollectionSpace();

			InBoundingBox = BoundingBoxInCollectionSpace;
		}

		FDataflowTransformSelection InTransformSelection = GetValue(Context, &TransformSelection);

		//
		// If not connected select everything by default
		//
		if (!IsConnected<FDataflowTransformSelection>(&TransformSelection))
		{
			GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
			const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectAll();

			FDataflowTransformSelection NewTransformSelection;
			NewTransformSelection.InitializeFromCollection(InCollection, false);
			NewTransformSelection.SetFromArray(SelectionArr);

			InTransformSelection = NewTransformSelection;
		}

		const int32 InNumPlanes = GetValue(Context, &NumPlanes);
		const TArray<FTransform>& InCutPlaneTransforms = GetValue(Context, &CutPlanes);

		if (InTransformSelection.AnySelected())
		{
			int32 ResultGeometryIndex = FFractureEngineFracturing::PlaneCutter(InCollection,
				InTransformSelection,
				InBoundingBox,
				GetValue(Context, &Transform),
				InNumPlanes,
				GetValue(Context, &RandomSeed),
				GetValue(Context, &ChanceToFracture),
				FIslandSplitSettings(SplitIslands, (double)CloseVertexDistance, (double)VertexToSurfaceBridgeDistance),
				GetValue(Context, &Grout),
				GetValue(Context, &Amplitude),
				GetValue(Context, &Frequency),
				GetValue(Context, &Persistence),
				GetValue(Context, &Lacunarity),
				GetValue(Context, &OctaveNumber),
				GetValue(Context, &PointSpacing),
				AddSamplesForCollision,
				GetValue(Context, &CollisionSampleSpacing),
				InCutPlaneTransforms);

			FDataflowTransformSelection NewSelection;
			FDataflowTransformSelection OriginalSelection;

			if (ResultGeometryIndex != INDEX_NONE)
			{
				if (InCollection.HasAttribute("TransformIndex", FGeometryCollection::GeometryGroup))
				{
					const TManagedArray<int32>& TransformIndices = InCollection.GetAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);

					NewSelection.InitializeFromCollection(InCollection, false);
					OriginalSelection.InitializeFromCollection(InCollection, false);

					// The newly fractured pieces are added to the end of the transform array (starting position is ResultGeometryIndex)
					for (int32 Idx = ResultGeometryIndex; Idx < TransformIndices.Num(); ++Idx)
					{
						int32 BoneIdx = TransformIndices[Idx];
						NewSelection.SetSelected(BoneIdx);
					}

					for (int32 Idx = 0; Idx < InTransformSelection.Num(); ++Idx)
					{
						if (InTransformSelection.IsSelected(Idx))
						{
							OriginalSelection.SetSelected(Idx);
						}
					}
				}
			}

			SetValue(Context, MoveTemp(InCollection), &Collection);
			SetValue(Context, OriginalSelection, &TransformSelection);
			SetValue(Context, NewSelection, &NewGeometryTransformSelection);

			return;
		}

		SafeForwardInput(Context, &Collection, &Collection);
		SetValue(Context, InTransformSelection, &TransformSelection);
		SetValue(Context, FDataflowTransformSelection(), &NewGeometryTransformSelection);
	}
}

#if WITH_EDITOR
bool FPlaneCutterDataflowNode_v2::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FPlaneCutterDataflowNode_v2::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		FBox InBoundingBox = GetValue(Context, &BoundingBox);

		if (!IsConnected(&BoundingBox))
		{
			GeometryCollection::Facades::FBoundsFacade BoundsFacade(InCollection);
			const FBox& BoundingBoxInCollectionSpace = BoundsFacade.GetBoundingBoxInCollectionSpace();

			InBoundingBox = BoundingBoxInCollectionSpace;
		}

		float PrincipalAxisLength = 2.f * FMath::Max3(InBoundingBox.GetExtent().X, InBoundingBox.GetExtent().Y, InBoundingBox.GetExtent().Z);
		float PlaneSize = PrincipalAxisLength * PlaneSizeMultiplier;

		const int32 InNumPlanes = GetValue(Context, &NumPlanes);

		FNoiseSettings NoiseSettings;
		NoiseSettings.Amplitude = GetValue(Context, &Amplitude);
		NoiseSettings.Frequency = GetValue(Context, &Frequency);
		NoiseSettings.Lacunarity = GetValue(Context, &Lacunarity);
		NoiseSettings.Persistence = GetValue(Context, &Persistence);
		NoiseSettings.Octaves = GetValue(Context, &OctaveNumber);
		NoiseSettings.PointSpacing = GetValue(Context, &PointSpacing);

		const int32 InRandomSeed = GetValue<int32>(Context, &RandomSeed);

		TArray<FTransform> PlaneTransforms = GetValue(Context, &CutPlanes);

		FFractureEngineFracturing::GenerateSliceTransforms(InBoundingBox, InRandomSeed, InNumPlanes, PlaneTransforms);

		if (PlaneTransforms.IsEmpty())
		{
			return;
		}

		FVector Center(0, 0, 0);

		DataflowRenderingInterface.SetLineWidth(LineWidthMultiplier);
		if (RenderType == EDataflowDebugDrawRenderType::Shaded)
		{
			DataflowRenderingInterface.SetShaded(true);
			DataflowRenderingInterface.SetTranslucent(bTranslucent);
			DataflowRenderingInterface.SetWireframe(true);
		}
		else
		{
			DataflowRenderingInterface.SetShaded(false);
			DataflowRenderingInterface.SetWireframe(true);
		}
		DataflowRenderingInterface.SetWorldPriority();
		DataflowRenderingInterface.SetColor(FLinearColor::Gray);

		FRandomStream RandomStream(InRandomSeed);
		FNoiseOffsets NoiseOffset(RandomStream);

		TArray<FSimpleDebugDrawMesh> DebugMeshes;
		DebugMeshes.SetNum(PlaneTransforms.Num());
		
		FTransform CollectionTransform = GetValue(Context, &Transform);

		for (int32 PlaneIdx = 0; PlaneIdx < PlaneTransforms.Num(); ++PlaneIdx)
		{
			if (bRandomizeColors)
			{
				DataflowRenderingInterface.SetColor(GetRandomColor(ColorRandomSeed, PlaneIdx));
			}

			const float Width = PlaneSize;
			const float Height = PlaneSize;

			constexpr int32 MaxCountPerDim = 2000;
			const int32 WidthVertexCount = FMath::Min(MaxCountPerDim, Width / NoiseSettings.PointSpacing);
			const int32 HeightVertexCount = FMath::Min(MaxCountPerDim, Height / NoiseSettings.PointSpacing);

			const FTransform& PlaneTransform = PlaneTransforms[PlaneIdx];
			const FVector NoisePivot = CollectionTransform.GetLocation();
			FVector Normal = PlaneTransform.GetUnitAxis(EAxis::Z);

			FSimpleDebugDrawMesh Mesh;
			DebugMeshes[PlaneIdx] = Mesh;

			DebugMeshes[PlaneIdx].MakeRectangleMesh(Center, Width, Height, WidthVertexCount, HeightVertexCount);

			for (int32 VertexIdx = 0; VertexIdx < DebugMeshes[PlaneIdx].GetMaxVertexIndex(); ++VertexIdx)
			{
				FVector WorldPos = PlaneTransforms[PlaneIdx].TransformPosition(DebugMeshes[PlaneIdx].Vertices[VertexIdx]);
				FVector NewWorldPos = WorldPos + NoiseSettings.NoiseVector(WorldPos - NoisePivot, NoiseOffset).Dot(Normal) * Normal;
				
				DebugMeshes[PlaneIdx].Vertices[VertexIdx] = CollectionTransform.InverseTransformPosition(NewWorldPos);
			}

			DataflowRenderingInterface.DrawMesh(DebugMeshes[PlaneIdx]);
		}
	}
}
#endif

/*--------------------------------------------------------------------------------------------------------*/

void FExplodedViewDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		// Translate collection
		FVector InOffset = GetValue(Context, &Offset);
		if (InOffset.Length() > UE_KINDA_SMALL_NUMBER)
		{
			TManagedArray<FVector3f>& Vertex = InCollection.ModifyAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
			for (int32 VertexIdx = 0; VertexIdx < Vertex.Num(); ++VertexIdx)
			{
				Vertex[VertexIdx] += FVector3f(InOffset);
			}
		}

		FFractureEngineFracturing::GenerateExplodedViewAttribute(InCollection, GetValue<FVector>(Context, &Scale), GetValue<float>(Context, &UniformScale));

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

void FSliceCutterDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection) ||
		Out->IsA<FDataflowTransformSelection>(&TransformSelection) ||
		Out->IsA<FDataflowTransformSelection>(&NewGeometryTransformSelection))
	{
		FDataflowTransformSelection InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);

		//
		// If not connected select everything by default
		//
		if (!IsConnected<FDataflowTransformSelection>(&TransformSelection))
		{
			const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

			GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
			const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectAll();

			FDataflowTransformSelection NewTransformSelection;
			NewTransformSelection.InitializeFromCollection(InCollection, false);
			NewTransformSelection.SetFromArray(SelectionArr);

			InTransformSelection = NewTransformSelection;
		}

		if (InTransformSelection.AnySelected())
		{
			FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

			int32 ResultGeometryIndex = FFractureEngineFracturing::SliceCutter(InCollection,
				InTransformSelection,
				GetValue<FBox>(Context, &BoundingBox),
				GetValue<int32>(Context, &SlicesX),
				GetValue<int32>(Context, &SlicesY),
				GetValue<int32>(Context, &SlicesZ),
				GetValue<float>(Context, &SliceAngleVariation),
				GetValue<float>(Context, &SliceOffsetVariation),
				GetValue<int32>(Context, &RandomSeed),
				GetValue<float>(Context, &ChanceToFracture),
				FIslandSplitSettings(SplitIslands, (double)CloseVertexDistance, (double)VertexToSurfaceBridgeDistance),
				GetValue<float>(Context, &Grout),
				GetValue<float>(Context, &Amplitude),
				GetValue<float>(Context, &Frequency),
				GetValue<float>(Context, &Persistence),
				GetValue<float>(Context, &Lacunarity),
				GetValue<int32>(Context, &OctaveNumber),
				GetValue<float>(Context, &PointSpacing),
				AddSamplesForCollision,
				GetValue<float>(Context, &CollisionSampleSpacing));

			FDataflowTransformSelection NewSelection;
			FDataflowTransformSelection OriginalSelection;

			if (ResultGeometryIndex != INDEX_NONE)
			{
				if (InCollection.HasAttribute("TransformIndex", FGeometryCollection::GeometryGroup))
				{
					const TManagedArray<int32>& TransformIndices = InCollection.GetAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);

					NewSelection.InitializeFromCollection(InCollection, false);
					OriginalSelection.InitializeFromCollection(InCollection, false);

					// The newly fractured pieces are added to the end of the transform array (starting position is ResultGeometryIndex)
					for (int32 Idx = ResultGeometryIndex; Idx < TransformIndices.Num(); ++Idx)
					{
						int32 BoneIdx = TransformIndices[Idx];
						NewSelection.SetSelected(BoneIdx);
					}

					for (int32 Idx = 0; Idx < InTransformSelection.Num(); ++Idx)
					{
						if (InTransformSelection.IsSelected(Idx))
						{
							OriginalSelection.SetSelected(Idx);
						}
					}
				}
			}

			SetValue(Context, MoveTemp(InCollection), &Collection);
			SetValue(Context, OriginalSelection, &TransformSelection);
			SetValue(Context, NewSelection, &NewGeometryTransformSelection);

			return;
		}

		SafeForwardInput(Context, &Collection, &Collection);
		SetValue(Context, InTransformSelection, &TransformSelection);
		SetValue(Context, FDataflowTransformSelection(), &NewGeometryTransformSelection);
	}
}

void FBrickCutterDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection) ||
		Out->IsA<FDataflowTransformSelection>(&TransformSelection) ||
		Out->IsA<FDataflowTransformSelection>(&NewGeometryTransformSelection))
	{
		FDataflowTransformSelection InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);
		//
		// If not connected select everything by default
		//
		if (!IsConnected<FDataflowTransformSelection>(&TransformSelection))
		{
			const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

			GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
			const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectAll();

			FDataflowTransformSelection NewTransformSelection;
			NewTransformSelection.InitializeFromCollection(InCollection, false);
			NewTransformSelection.SetFromArray(SelectionArr);

			InTransformSelection = NewTransformSelection;
		}

		FBox InBoundingBox = GetValue<FBox>(Context, &BoundingBox);
		//
		// If not connected set bounds to collection bounds
		//
		if (!IsConnected<FBox>(&BoundingBox))
		{
			const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

			GeometryCollection::Facades::FBoundsFacade BoundsFacade(InCollection);
			const FBox& BoundingBoxInCollectionSpace = BoundsFacade.GetBoundingBoxInCollectionSpace();

			InBoundingBox = BoundingBoxInCollectionSpace;
		}

		if (InTransformSelection.AnySelected())
		{
			FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

			int32 ResultGeometryIndex = FFractureEngineFracturing::BrickCutter(InCollection,
				InTransformSelection,
				InBoundingBox,
				GetValue<FTransform>(Context, &Transform),
				Bond,
				GetValue<float>(Context, &BrickLength),
				GetValue<float>(Context, &BrickHeight),
				GetValue<float>(Context, &BrickDepth),
				GetValue<int32>(Context, &RandomSeed),
				GetValue<float>(Context, &ChanceToFracture),
				FIslandSplitSettings(SplitIslands, (double)CloseVertexDistance, (double)VertexToSurfaceBridgeDistance),
				GetValue<float>(Context, &Grout),
				GetValue<float>(Context, &Amplitude),
				GetValue<float>(Context, &Frequency),
				GetValue<float>(Context, &Persistence),
				GetValue<float>(Context, &Lacunarity),
				GetValue<int32>(Context, &OctaveNumber),
				GetValue<float>(Context, &PointSpacing),
				AddSamplesForCollision,
				GetValue<float>(Context, &CollisionSampleSpacing));

			FDataflowTransformSelection NewSelection;
			FDataflowTransformSelection OriginalSelection;

			if (ResultGeometryIndex != INDEX_NONE)
			{
				if (InCollection.HasAttribute("TransformIndex", FGeometryCollection::GeometryGroup))
				{
					const TManagedArray<int32>& TransformIndices = InCollection.GetAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);

					NewSelection.InitializeFromCollection(InCollection, false);
					OriginalSelection.InitializeFromCollection(InCollection, false);

					// The newly fractured pieces are added to the end of the transform array (starting position is ResultGeometryIndex)
					for (int32 Idx = ResultGeometryIndex; Idx < TransformIndices.Num(); ++Idx)
					{
						int32 BoneIdx = TransformIndices[Idx];
						NewSelection.SetSelected(BoneIdx);
					}

					for (int32 Idx = 0; Idx < InTransformSelection.Num(); ++Idx)
					{
						if (InTransformSelection.IsSelected(Idx))
						{
							OriginalSelection.SetSelected(Idx);
						}
					}
				}
			}

			SetValue(Context, MoveTemp(InCollection), &Collection);
			SetValue(Context, OriginalSelection, &TransformSelection);
			SetValue(Context, NewSelection, &NewGeometryTransformSelection);

			return;
		}

		SafeForwardInput(Context, &Collection, &Collection);
		SetValue(Context, InTransformSelection, &TransformSelection);
		SetValue(Context, FDataflowTransformSelection(), &NewGeometryTransformSelection);
	}
}

void FMeshCutterDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection) ||
		Out->IsA(&TransformSelection) ||
		Out->IsA(&NewGeometryTransformSelection))
	{
		FDataflowTransformSelection InTransformSelection = GetValue(Context, &TransformSelection);
		//
		// If not connected select everything by default
		//
		if (!IsConnected(&TransformSelection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

			GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
			const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectAll();

			FDataflowTransformSelection NewTransformSelection;
			NewTransformSelection.InitializeFromCollection(InCollection, false);
			NewTransformSelection.SetFromArray(SelectionArr);

			InTransformSelection = NewTransformSelection;
		}

		FBox InBoundingBox = GetValue(Context, &BoundingBox);
		//
		// If not connected set bounds to collection bounds
		//
		if (!IsConnected(&BoundingBox))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

			GeometryCollection::Facades::FBoundsFacade BoundsFacade(InCollection);
			const FBox& BoundingBoxInCollectionSpace = BoundsFacade.GetBoundingBoxInCollectionSpace();

			InBoundingBox = BoundingBoxInCollectionSpace;
		}

		if (InTransformSelection.AnySelected())
		{
			FManagedArrayCollection InCollection = GetValue(Context, &Collection);

			FDynamicMesh3 LocalMesh; // can be used for local storage of a converted static mesh
			TArray<const FDynamicMesh3*> UseMeshes;

			if (TObjectPtr<UStaticMesh> InCuttingMesh = GetValue(Context, &CuttingStaticMesh))
			{
#if WITH_EDITORONLY_DATA
				if (FMeshDescription* MeshDescription = bUseHiRes ? InCuttingMesh->GetHiResMeshDescription() : InCuttingMesh->GetMeshDescription(LODLevel))
				{
					// If HiRes is empty then use LoRes
					if (bUseHiRes && MeshDescription->Vertices().Num() == 0)
					{
						MeshDescription = InCuttingMesh->GetMeshDescription(LODLevel);
					}

					if (MeshDescription->Vertices().Num() > 0)
					{
						int32 NumUVLayers = GeometryCollection::UV::GetNumUVLayers(InCollection);
						LocalMesh = ConvertMeshDescriptionToCuttingDynamicMesh(MeshDescription, NumUVLayers);
						UseMeshes.Add(&LocalMesh);
					}

				}
#else
				// TODO: for runtime usage, could try to fallback to the render mesh (if available on CPU)
				ensureMsgf(false, TEXT("FMeshCutterDataflowNode's Static Mesh support is currently editor-only."));
#endif
			}

			const TArray<TObjectPtr<UDynamicMesh>>& InDynamicMeshes = GetValue(Context, &CuttingDynamicMeshes);
			for (TObjectPtr<UDynamicMesh> MeshObj : InDynamicMeshes)
			{
				if (MeshObj && MeshObj->GetMeshPtr())
				{
					UseMeshes.Add(MeshObj->GetMeshPtr());
				}
			}

			if (UseMeshes.Num() > 0)
			{
				const int32 InRandomSeed = GetValue(Context, &RandomSeed);
				const int32 InNumberToScatter = GetValue(Context, &NumberToScatter);
				const int32 InGridX = GetValue(Context, &GridX);
				const int32 InGridY = GetValue(Context, &GridY);
				const int32 InGridZ = GetValue(Context, &GridZ);
				const float InVariability = GetValue(Context, &Variability);
				const float InMinScaleFactor = GetValue(Context, &MinScaleFactor);
				const float InMaxScaleFactor = GetValue(Context, &MaxScaleFactor);
				const float InRollRange = GetValue(Context, &RollRange);
				const float InPitchRange = GetValue(Context, &PitchRange);
				const float InYawRange = GetValue(Context, &YawRange);
				const FTransform InTransform = GetValue(Context, &Transform);
				const float InChanceToFracture = GetValue(Context, &ChanceToFracture);
				const float InCollisionSampleSpacing = AddSamplesForCollision ? GetValue(Context, &CollisionSampleSpacing) : 0.f;

				// Note: per-cut mesh selection is not currently a dataflow input
				const EMeshCutterPerCutMeshSelection InPerCutMeshSelection = PerCutMeshSelection;

				TArray<FTransform> MeshTransforms;

				if (CutDistribution == EMeshCutterCutDistribution::SingleCut)
				{
					MeshTransforms.Add(InTransform);
				}
				else
				{
					FFractureEngineFracturing::GenerateMeshTransforms(MeshTransforms,
						InBoundingBox,
						InRandomSeed,
						CutDistribution,
						InNumberToScatter,
						InGridX,
						InGridY,
						InGridZ,
						InVariability,
						InMinScaleFactor,
						InMaxScaleFactor,
						bRandomOrientation,
						InRollRange,
						InPitchRange,
						InYawRange);
				}

				int32 ResultGeometryIndex = FFractureEngineFracturing::MeshArrayCutter(MeshTransforms,
					InCollection,
					InTransformSelection,
					UseMeshes,
					InPerCutMeshSelection,
					InRandomSeed,
					InChanceToFracture,
					FIslandSplitSettings(SplitIslands, (double)CloseVertexDistance, (double)VertexToSurfaceBridgeDistance),
					InCollisionSampleSpacing);

				FDataflowTransformSelection NewSelection;
				FDataflowTransformSelection OriginalSelection;

				if (ResultGeometryIndex != INDEX_NONE)
				{
					if (InCollection.HasAttribute("TransformIndex", FGeometryCollection::GeometryGroup))
					{
						const TManagedArray<int32>& TransformIndices = InCollection.GetAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);

						NewSelection.InitializeFromCollection(InCollection, false);
						OriginalSelection.InitializeFromCollection(InCollection, false);

						// The newly fractured pieces are added to the end of the transform array (starting position is ResultGeometryIndex)
						for (int32 Idx = ResultGeometryIndex; Idx < TransformIndices.Num(); ++Idx)
						{
							int32 BoneIdx = TransformIndices[Idx];
							NewSelection.SetSelected(BoneIdx);
						}

						for (int32 Idx = 0; Idx < InTransformSelection.Num(); ++Idx)
						{
							if (InTransformSelection.IsSelected(Idx))
							{
								OriginalSelection.SetSelected(Idx);
							}
						}
					}
				}

				SetValue(Context, MoveTemp(InCollection), &Collection);
				SetValue(Context, OriginalSelection, &TransformSelection);
				SetValue(Context, NewSelection, &NewGeometryTransformSelection);

				return;
			}
		}

		SafeForwardInput(Context, &Collection, &Collection);
		SetValue(Context, InTransformSelection, &TransformSelection);
		SetValue(Context, FDataflowTransformSelection(), &NewGeometryTransformSelection);
	}
}

FUniformFractureDataflowNode::FUniformFractureDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&TransformSelection);
	RegisterInputConnection(&Transform);
	RegisterInputConnection(&MinVoronoiSites)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&MaxVoronoiSites)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&InternalMaterialID);
	RegisterInputConnection(&RandomSeed)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&ChanceToFracture)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&Grout)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&Amplitude)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&Frequency)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&Persistence)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&Lacunarity)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&OctaveNumber)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&PointSpacing)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&CollisionSampleSpacing)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&TransformSelection, &TransformSelection);
	RegisterOutputConnection(&NewGeometryTransformSelection);
}

void FUniformFractureDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection) ||
		Out->IsA(&TransformSelection) ||
		Out->IsA(&NewGeometryTransformSelection))
	{
		FDataflowTransformSelection InTransformSelection = GetValue(Context, &TransformSelection);

		//
		// If not connected select everything by default
		//
		if (!IsConnected(&TransformSelection))
		{
			const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

			GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
			const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectAll();

			FDataflowTransformSelection NewTransformSelection;
			NewTransformSelection.InitializeFromCollection(InCollection, false);
			NewTransformSelection.SetFromArray(SelectionArr);

			InTransformSelection = NewTransformSelection;
		}

		if (InTransformSelection.AnySelected())
		{
			FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

			FUniformFractureSettings UniformFractureSettings;
			UniformFractureSettings.Transform = GetValue(Context, &Transform);
			UniformFractureSettings.MinVoronoiSites = GetValue(Context, &MinVoronoiSites);
			UniformFractureSettings.MaxVoronoiSites = GetValue(Context, &MaxVoronoiSites);
			UniformFractureSettings.InternalMaterialID = InternalMaterialID;
			UniformFractureSettings.RandomSeed = GetValue(Context, &RandomSeed);
			UniformFractureSettings.ChanceToFracture = GetValue(Context, &ChanceToFracture);
			UniformFractureSettings.GroupFracture = GroupFracture;
			UniformFractureSettings.SplitIslands = SplitIslands;
			UniformFractureSettings.CloseVertexDistance = (double)CloseVertexDistance;
			UniformFractureSettings.VertexToSurfaceBridgeDistance = (double)VertexToSurfaceBridgeDistance;
			UniformFractureSettings.Grout = GetValue(Context, &Grout);
			UniformFractureSettings.NoiseSettings.Amplitude = GetValue(Context, &Amplitude);
			UniformFractureSettings.NoiseSettings.Frequency = GetValue(Context, &Frequency);
			UniformFractureSettings.NoiseSettings.Persistence = GetValue(Context, &Persistence);
			UniformFractureSettings.NoiseSettings.Lacunarity = GetValue(Context, &Lacunarity);
			UniformFractureSettings.NoiseSettings.Octaves = GetValue(Context, &OctaveNumber);
			UniformFractureSettings.NoiseSettings.PointSpacing = GetValue(Context, &PointSpacing);
			UniformFractureSettings.AddSamplesForCollision = AddSamplesForCollision;
			UniformFractureSettings.CollisionSampleSpacing = GetValue(Context, &CollisionSampleSpacing);

			int32 ResultGeometryIndex = FFractureEngineFracturing::UniformFracture(
				InCollection,
				InTransformSelection,
				UniformFractureSettings);

			FDataflowTransformSelection NewSelection;
			FDataflowTransformSelection OriginalSelection;

			if (ResultGeometryIndex != INDEX_NONE)
			{
				if (InCollection.HasAttribute("TransformIndex", FGeometryCollection::GeometryGroup))
				{
					const TManagedArray<int32>& GeometryToTransformIndices = InCollection.GetAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);

					NewSelection.InitializeFromCollection(InCollection, false);
					OriginalSelection.InitializeFromCollection(InCollection, false);

					// The newly fractured pieces are added to the end of the transform array (starting position is ResultGeometryIndex)
					for (int32 GeometryIdx = ResultGeometryIndex; GeometryIdx < GeometryToTransformIndices.Num(); ++GeometryIdx)
					{
						const int32 TransformIdx = GeometryToTransformIndices[GeometryIdx];
						NewSelection.SetSelected(TransformIdx);
					}

					for (int32 TransformIdx = 0; TransformIdx < InTransformSelection.Num(); ++TransformIdx)
					{
						if (InTransformSelection.IsSelected(TransformIdx))
						{
							OriginalSelection.SetSelected(TransformIdx);
						}
					}
				}
			}

			SetValue(Context, MoveTemp(InCollection), &Collection);
			SetValue(Context, OriginalSelection, &TransformSelection);
			SetValue(Context, NewSelection, &NewGeometryTransformSelection);

			return;
		}

		SafeForwardInput(Context, &Collection, &Collection);
		SetValue(Context, InTransformSelection, &TransformSelection);
		SetValue(Context, FDataflowTransformSelection(), &NewGeometryTransformSelection);
	}
}

/** ------------------------------------------------------------------------------------------------------------ **/

FVisualizeFractureDataflowNode::FVisualizeFractureDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&RandomSeed)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&Level)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&ExplodeAmount)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&Scale)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&Offset)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterOutputConnection(&Collection, &Collection);
}

void FVisualizeFractureDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);

		// Translate collection
		FVector InOffset = GetValue(Context, &Offset);
		if (InOffset.Length() > UE_KINDA_SMALL_NUMBER)
		{
			TManagedArray<FVector3f>& Vertex = InCollection.ModifyAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
			for (int32 VertexIdx = 0; VertexIdx < Vertex.Num(); ++VertexIdx)
			{
				Vertex[VertexIdx] += FVector3f(InOffset);
			}
		}

		const int32 InLevel = GetValue(Context, &Level);
		check(InLevel >= 0);

		if (bApplyExplodedView)
		{
			FFractureEngineFracturing::GenerateExplodedViewAttribute(
				InCollection, 
				GetValue(Context, &Scale), 
				GetValue(Context, &ExplodeAmount),
				InLevel);
		}

		if (bApplyColor)
		{
			if (InCollection.HasAttribute("BoneColor", FGeometryCollection::TransformGroup))
			{
				TManagedArray<FLinearColor>& BoneColors = InCollection.ModifyAttribute<FLinearColor>("BoneColor", FGeometryCollection::TransformGroup);

				const int32 NumBones = BoneColors.Num();
				const int32 InRandomSeed = GetValue(Context, &RandomSeed);

				FRandomStream RandomStream(NumBones + InRandomSeed);

				// Clear BoneColors
				FFractureEngineFracturing::InitColors(InCollection);

				if (ColoringType == EDataflowVisualizeFractureColoringType::ColorByParent && 
					InCollection.HasAttribute(FTransformCollection::LevelAttribute, FGeometryCollection::TransformGroup))
				{
					FFractureEngineFracturing::SetBoneColorByParent(InCollection, RandomStream, InLevel, RandomColorRangeMin, RandomColorRangeMax);
				}
				else if (ColoringType == EDataflowVisualizeFractureColoringType::ColorByLevel &&
					InCollection.HasAttribute(FTransformCollection::LevelAttribute, FGeometryCollection::TransformGroup))
				{
					FFractureEngineFracturing::SetBoneColorByLevel(InCollection, InLevel);
				}
				else if (ColoringType == EDataflowVisualizeFractureColoringType::ColorByCluster &&
					InCollection.HasAttribute(FTransformCollection::LevelAttribute, FGeometryCollection::TransformGroup))
				{
					// This what the Geometry Tools uses
					FFractureEngineFracturing::SetBoneColorByCluster(InCollection, RandomStream, InLevel, RandomColorRangeMin, RandomColorRangeMax);
				}
				else if (ColoringType == EDataflowVisualizeFractureColoringType::ColorByLeafLevel &&
					InCollection.HasAttribute(FTransformCollection::LevelAttribute, FGeometryCollection::TransformGroup))
				{
					FFractureEngineFracturing::SetBoneColorByLeafLevel(InCollection, InLevel);
				}
				else if (ColoringType == EDataflowVisualizeFractureColoringType::ColorByLeaf)
				{
					FFractureEngineFracturing::SetBoneColorByLeaf(InCollection, RandomStream, InLevel, RandomColorRangeMin, RandomColorRangeMax);
				}
				else if (ColoringType == EDataflowVisualizeFractureColoringType::ColorByAttr)
				{
					FFractureEngineFracturing::SetBoneColorByAttr(InCollection,
						Attribute,
						Min.MinAttrValue,
						Max.MaxAttrValue,
						Min.MinColor,
						Max.MaxColor);
				}

				// Transfer BoneColors to VertexColor
				FFractureEngineFracturing::TransferBoneColorToVertexColor(InCollection);

				SetValue(Context, MoveTemp(InCollection), &Collection);
			}
		}
		else
		{
			SafeForwardInput(Context, &Collection, &Collection);
		}
	}
}

FSetFloatAttributeDataflowNode::FSetFloatAttributeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&RandomSeed)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&NoiseScale)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&FloatArray);
}

void FSetFloatAttributeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection) || Out->IsA(&FloatArray))
	{
		if (IsConnected(&Collection))
		{
			FManagedArrayCollection InCollection = GetValue(Context, &Collection);
			const FName AttrName = FName(*Attribute);

			if (InCollection.HasAttribute(AttrName, FGeometryCollection::TransformGroup))
			{
				TManagedArray<float>& AttrValues = InCollection.ModifyAttribute<float>(AttrName, FGeometryCollection::TransformGroup);
				const int32 NumAttrValues = AttrValues.Num();

				TArray<float> OutFloatArray; OutFloatArray.AddUninitialized(NumAttrValues);

				if (Method == EDataflowSetFloatArrayMethod::Random)
				{
					const int32 InRandomSeed = GetValue(Context, &RandomSeed);
					FRandomStream RandomStream(InRandomSeed);

					for (int32 Idx = 0; Idx < NumAttrValues; ++Idx)
					{
						AttrValues[Idx] = RandomStream.FRandRange(0.f, 1.f);
						OutFloatArray[Idx] = AttrValues[Idx];
					}
				}
				else if (Method == EDataflowSetFloatArrayMethod::Noise)
				{
					const int32 InNoiseScale = GetValue(Context, &NoiseScale);

					const TManagedArray<int32>& TransformToGeometryIndices = InCollection.GetAttribute<int32>(FGeometryCollection::TransformToGeometryIndexAttribute, FGeometryCollection::TransformGroup);
					const TManagedArray<FBox>& BoundingBoxes = InCollection.GetAttribute<FBox>(FGeometryCollection::BoundingBoxAttribute, FGeometryCollection::GeometryGroup);

					for (int32 Idx = 0; Idx < NumAttrValues; ++Idx)
					{
						int32 GeometryIdx = TransformToGeometryIndices[Idx];
						if (GeometryIdx != -1)
						{
							FVector Center = BoundingBoxes[GeometryIdx].GetCenter();

							AttrValues[Idx] = 0.5f * FMath::PerlinNoise3D(InNoiseScale * Center) + 1.f;
						}
						else
						{
							AttrValues[Idx] = 0.f;
						}
					}
				}
				else if (Method == EDataflowSetFloatArrayMethod::ByBoundingBox)
				{
					const TManagedArray<int32>& TransformToGeometryIndices = InCollection.GetAttribute<int32>(FGeometryCollection::TransformToGeometryIndexAttribute, FGeometryCollection::TransformGroup);
					const TManagedArray<FBox>& BoundingBoxes = InCollection.GetAttribute<FBox>(FGeometryCollection::BoundingBoxAttribute, FGeometryCollection::GeometryGroup);

					// Compute BoundingBox for the entire collection
					FBox BBox = FBox(ForceInit);

					for (int32 Idx = 0; Idx < NumAttrValues; ++Idx)
					{
						int32 GeometryIdx = TransformToGeometryIndices[Idx];
						if (GeometryIdx != -1)
						{
							BBox += BoundingBoxes[GeometryIdx];
						}
					}

					for (int32 Idx = 0; Idx < NumAttrValues; ++Idx)
					{
						int32 GeometryIdx = TransformToGeometryIndices[Idx];
						if (GeometryIdx != -1)
						{
							FVector Center = BoundingBoxes[GeometryIdx].GetCenter();

							AttrValues[Idx] = (Center.X - BBox.Min.X) / (BBox.Max.X - BBox.Min.X);
						}
						else
						{
							AttrValues[Idx] = 0.f;
						}
					}
				}

				SetValue(Context, MoveTemp(InCollection), &Collection);
				SetValue(Context, OutFloatArray, &FloatArray);
				return;
			}
		}

		SafeForwardInput(Context, &Collection, &Collection);
		SetValue(Context, TArray<float>(), &FloatArray);
	}
}
