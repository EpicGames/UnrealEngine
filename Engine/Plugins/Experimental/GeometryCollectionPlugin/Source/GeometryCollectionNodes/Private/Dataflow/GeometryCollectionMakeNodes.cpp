// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionMakeNodes.h"
#include "Dataflow/DataflowCore.h"
#if WITH_EDITOR
#include "Dataflow/DataflowRenderingViewMode.h"
#endif

#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionEngineUtility.h"
#include "GeometryCollection/GeometryCollectionEngineRemoval.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealTypePrivate.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "StaticMeshAttributes.h"
#include "DynamicMeshEditor.h"
#include "Operations/MeshBoolean.h"
#include "Materials/Material.h"

#include "EngineGlobals.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "Voronoi/Voronoi.h"
#include "PlanarCut.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "FractureEngineClustering.h"
#include "FractureEngineSelection.h"
#include "GeometryCollection/Facades/CollectionBoundsFacade.h"
#include "GeometryCollection/Facades/CollectionAnchoringFacade.h"
#include "GeometryCollection/Facades/CollectionRemoveOnBreakFacade.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Dataflow/DataflowDebugDraw.h"
#include "Dataflow/DataflowSimpleDebugDrawMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Generators/SphereGenerator.h"
#include "Generators/BoxSphereGenerator.h"
#include "Generators/CapsuleGenerator.h"
#include "Generators/SweepGenerator.h"
#include "Generators/GridBoxMeshGenerator.h"
#include "Generators/DiscMeshGenerator.h"
#include "Generators/StairGenerator.h"
#include "Generators/RectangleMeshGenerator.h"
#include "Algo/Reverse.h"
#include "Dataflow/DataflowUtils.h"
#include "GeometryCollection/Facades/PointsFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionMakeNodes)

#define LOCTEXT_NAMESPACE "GeometryCollectionMakeNodes"

namespace UE::Dataflow
{
	void GeometryCollectionMakeNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeLiteralStringDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakePointsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeBoxDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeSphereDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeLiteralFloatDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeLiteralDoubleDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeLiteralIntDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeLiteralBoolDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeLiteralVectorDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeQuaternionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeFloatArrayDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeRotatorDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBreakTransformDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeLiteralStringDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeLiteralFloatDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeLiteralIntDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeLiteralBoolDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeTransformDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeIntArrayDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeIntArrayFromListDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakePlaneDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakePointsDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeNumericArrayDataflowNode);

		// Deprecated
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeTransformDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakePlaneDataflowNode);
	}
}

void FMakeLiteralStringDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FString>(&String))
	{
		SetValue(Context, Value, &String);
	}
}

void FMakeLiteralStringDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FString>(&String))
	{
		SetValue(Context, String, &String);
	}
}

/*--------------------------------------------------------------------------------------------------------*/

void FMakePointsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<FVector>>(&Points))
	{
		SetValue(Context, Point, &Points);
	}
}

/*--------------------------------------------------------------------------------------------------------*/

FMakeBoxDataflowNode::FMakeBoxDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Min);
	RegisterInputConnection(&Max);
	RegisterInputConnection(&Center);
	RegisterInputConnection(&Size);
	RegisterOutputConnection(&Box);
}

void FMakeBoxDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FBox>(&Box))
	{
		if (DataType == EMakeBoxDataTypeEnum::Dataflow_MakeBox_DataType_MinMax)
		{
			const FVector InMin = GetValue(Context, &Min);
			FVector InMax = GetValue(Context, &Max);

			const FBox OutBox = FBox(InMin, InMax);
			const FVector Extent = OutBox.GetExtent();

			if (Extent.X < 0)
			{
				Context.Error(LOCTEXT("InvalidBoundingboxExtentX", "Invalid box specified. ExtentX is negative."), this, Out);
			}

			if (Extent.Y < 0)
			{
				Context.Error(LOCTEXT("InvalidBoundingboxExtentY", "Invalid box specified. ExtentY is negative."), this, Out);
			}

			if (Extent.Z < 0)
			{
				Context.Error(LOCTEXT("InvalidBoundingboxExtentZ", "Invalid box specified. ExtentZ is negative."), this, Out);
			}

			SetValue(Context, OutBox, &Box);
		}
		else if (DataType == EMakeBoxDataTypeEnum::Dataflow_MakeBox_DataType_CenterSize)
		{
			const FVector InCenter = GetValue(Context, &Center);
			const FVector InSize = GetValue(Context, &Size);

			SetValue(Context, FBox(InCenter - 0.5 * InSize, InCenter + 0.5 * InSize), &Box);
		}
	}
}

/*--------------------------------------------------------------------------------------------------------*/

void FMakeSphereDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FSphere>(&Sphere))
	{
		FVector CenterVal = GetValue<FVector>(Context, &Center);
		float RadiusVal = GetValue<float>(Context, &Radius);

		SetValue(Context, FSphere(CenterVal, RadiusVal), &Sphere);
	}
}

/*--------------------------------------------------------------------------------------------------------*/

void FMakeLiteralFloatDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&Float))
	{
		SetValue(Context, Value, &Float);
	}
}

void FMakeLiteralFloatDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&Float))
	{
		SetValue(Context, Float, &Float);
	}
}

//-----------------------------------------------------------------------------------------------

FMakeLiteralDoubleDataflowNode::FMakeLiteralDoubleDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Double);
}

void FMakeLiteralDoubleDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Double))
	{
		SetValue(Context, Double, &Double);
	}
}

//-----------------------------------------------------------------------------------------------

void FMakeLiteralIntDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<int32>(&Int))
	{
		SetValue(Context, Value, &Int);
	}
}

void FMakeLiteralIntDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<int32>(&Int))
	{
		SetValue(Context, Int, &Int);
	}
}

void FMakeLiteralBoolDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<bool>(&Bool))
	{
		SetValue(Context, Value, &Bool);
	}
}

void FMakeLiteralBoolDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<bool>(&Bool))
	{
		SetValue(Context, Bool, &Bool);
	}
}

void FMakeLiteralVectorDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FVector>(&Vector))
	{
		const FVector Value(GetValue<float>(Context, &X, X), GetValue<float>(Context, &Y, Y), GetValue<float>(Context, &Z, Z));
		SetValue(Context, Value, &Vector);
	}
}

void FMakeTransformDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FTransform>(&OutTransform))
	{
		SetValue(Context,
			FTransform(FQuat::MakeFromEuler(GetValue<FVector>(Context, &InRotation))
				, GetValue<FVector>(Context, &InTranslation)
				, GetValue<FVector>(Context, &InScale))
			, &OutTransform);
	}
}

/* -------------------------------------------------------------------------------- */

FMakeTransformDataflowNode_v2::FMakeTransformDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Translation);
	RegisterInputConnection(&Rotation);
	RegisterInputConnection(&Rotator).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Quat).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Scale);
	RegisterOutputConnection(&Transform);
}

void FMakeTransformDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FTransform>(&Transform))
	{
		const FVector InTranslation = GetValue(Context, &Translation);
		const FVector InScale = GetValue(Context, &Scale);
		FQuat OutQuat;
		if (IsConnected(&Rotation))
		{
			const FVector InRotation = GetValue(Context, &Rotation);
			OutQuat = FQuat::MakeFromEuler(InRotation);
		}
		else if (IsConnected(&Rotator))
		{
			const FRotator InRotator = GetValue(Context, &Rotator);
			OutQuat = FQuat::MakeFromRotator(InRotator);
		}
		else if (IsConnected(&Quat))
		{
			const FQuat InQuat = GetValue(Context, &Quat);
			OutQuat = InQuat;
		}

		FTransform OutTransform = FTransform(OutQuat, InTranslation, InScale);
		SetValue(Context, OutTransform, &Transform);
	}
}

/* -------------------------------------------------------------------------------- */

void FMakeQuaternionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FQuat>(&Quaternion))
	{
		const FQuat Value(GetValue<float>(Context, &X, X), GetValue<float>(Context, &Y, Y), GetValue<float>(Context, &Z, Z), GetValue<float>(Context, &W, W));
		SetValue(Context, Value, &Quaternion);
	}
}

void FMakeFloatArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&FloatArray))
	{
		const int32 InNumElements = GetValue(Context, &NumElements);
		const float InValue = GetValue(Context, &Value);

		TArray<float> OutFloatArray;
		OutFloatArray.Init(InValue, InNumElements);

		SetValue(Context, OutFloatArray, &FloatArray);
	}
}

void FMakeCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		if (bAddRootTransform)
		{
			FGeometryCollection NewCollection;
			const int32 RootIndex = NewCollection.AddElements(1, FGeometryCollection::TransformGroup);
			NewCollection.Parent[RootIndex] = INDEX_NONE;
			NewCollection.BoneColor[RootIndex] = FLinearColor::White;
			NewCollection.BoneName[RootIndex] = TEXT("Root");
			SetValue(Context, static_cast<const FManagedArrayCollection&>(NewCollection), &Collection);
			return;
		}
		// completely empty collection 
		SetValue(Context, FManagedArrayCollection(), &Collection);
	}
}

/* -------------------------------------------------------------------------------- */

FMakeRotatorDataflowNode::FMakeRotatorDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Pitch);
	RegisterInputConnection(&Yaw);
	RegisterInputConnection(&Roll);
	RegisterOutputConnection(&Rotator);
}

void FMakeRotatorDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Rotator))
	{
		const float InPitch = GetValue(Context, &Pitch);
		const float InYaw = GetValue(Context, &Yaw);
		const float InRoll = GetValue(Context, &Roll);
		SetValue(Context, FRotator(InPitch, InYaw, InRoll), &Rotator);
	}
}

/* -------------------------------------------------------------------------------- */

FBreakTransformDataflowNode::FBreakTransformDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Transform);
	RegisterOutputConnection(&Translation);
	RegisterOutputConnection(&Rotation);
	RegisterOutputConnection(&Rotator);
	RegisterOutputConnection(&Quat);
	RegisterOutputConnection(&Scale);
}

void FBreakTransformDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Translation) ||
		Out->IsA(&Rotation) ||
		Out->IsA(&Rotator) ||
		Out->IsA(&Quat) ||
		Out->IsA(&Scale))
	{
		const FTransform InTransform = GetValue(Context, &Transform);

		const FVector OutTranslation = InTransform.GetTranslation();
		const FVector OutRotationAsEuler = InTransform.GetRotation().Euler();
		const FRotator OutRotator = InTransform.GetRotation().Rotator();
		const FQuat OutQuat = InTransform.GetRotation();
		const FVector OutScale = InTransform.GetScale3D();

		SetValue(Context, OutTranslation, &Translation);
		SetValue(Context, OutRotationAsEuler, &Rotation);
		SetValue(Context, OutRotator, &Rotator);
		SetValue(Context, OutQuat, &Quat);
		SetValue(Context, OutScale, &Scale);
	}
}

/* -------------------------------------------------------------------------------- */

FMakePlaneDataflowNode::FMakePlaneDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&BasePoint).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Normal).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Plane);
}

void FMakePlaneDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Plane))
	{
		const FVector InBasePoint = GetValue(Context, &BasePoint);
		const FVector InNormal = GetValue(Context, &Normal);

		const FPlane OutPlane = FPlane(InBasePoint, InNormal);

		SetValue(Context, OutPlane, &Plane);
	}
}

#if WITH_EDITOR
bool FMakePlaneDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FMakePlaneDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		DebugDrawRenderSettings.SetDebugDrawSettings(DataflowRenderingInterface);

		const FVector InBasePoint = GetValue(Context, &BasePoint);
		FVector InNormal = GetValue(Context, &Normal);

		FSimpleDebugDrawMesh Mesh;
		Mesh.MakeRectangleMesh(FVector(0.0), PlaneSizeMultiplier * 10.f, PlaneSizeMultiplier * 10.f, 11, 11);

		const FVector Up = FVector::UpVector;
		FQuat Quat = FQuat::FindBetweenVectors(Up, Normal);

		FTransform PlaneTransform = FTransform::Identity;
		PlaneTransform.SetRotation(Quat);
		PlaneTransform.SetTranslation(InBasePoint);

		for (int32 VertexIdx = 0; VertexIdx < Mesh.GetMaxVertexIndex(); ++VertexIdx)
		{
			Mesh.Vertices[VertexIdx] = PlaneTransform.TransformPosition(Mesh.Vertices[VertexIdx]);
		}

		DataflowRenderingInterface.DrawMesh(Mesh);

		// Draw normal
		InNormal.Normalize();
		DataflowRenderingInterface.DrawLine(InBasePoint, InBasePoint + InNormal * 2.f);
	}
}
#endif

/* -------------------------------------------------------------------------------- */

FMakeIntArrayDataflowNode::FMakeIntArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Start);
	RegisterInputConnection(&Count);
	RegisterOutputConnection(&IntArray);
}

void FMakeIntArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&IntArray))
	{
		const int32 InStart = GetValue(Context, &Start);
		const int32 InCount = FMath::Max(0, GetValue(Context, &Count));

		TArray<int32> OutIntArray;
		OutIntArray.SetNumUninitialized(InCount);

		for (int32 Idx = 0; Idx < InCount; ++Idx)
		{
			OutIntArray[Idx] = InStart + Idx;
		}

		if (bReverseOrder)
		{
			Algo::Reverse(OutIntArray);
		}

		SetValue(Context, OutIntArray, &IntArray);
	}
}

/* -------------------------------------------------------------------------------- */

FMakeIntArrayFromListDataflowNode::FMakeIntArrayFromListDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&IntArray);
}

void FMakeIntArrayFromListDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&IntArray))
	{
		TArray<int32> OutIntArray;

		using namespace UE::Dataflow::Utils;

		EErrorCode ErrorCode = UE::Dataflow::Utils::ParseIndicesStr(List, OutIntArray);

		if (ErrorCode == EErrorCode::InvalidChars)
		{
			SetError(Context, &IntArray, TEXT("Invalid character(s) specified in list"));
		}
		else if (ErrorCode == EErrorCode::InvalidFormatInSegment)
		{
			SetError(Context, &IntArray, TEXT("Invalid format in segment"));
		}

		if (bReverseOrder)
		{
			Algo::Reverse(OutIntArray);
		}

		SetValue(Context, OutIntArray, &IntArray);
	}
}

/* -------------------------------------------------------------------------------- */

FMakePlaneDataflowNode_v2::FMakePlaneDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&BasePoint).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Normal).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Plane);
}

void FMakePlaneDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Plane))
	{
		const FVector InBasePoint = GetValue(Context, &BasePoint);
		const FVector InNormal = GetValue(Context, &Normal);

		FDataflowPlane OutPlane = FDataflowPlane(InBasePoint, InNormal);

		SetValue(Context, OutPlane, &Plane);
	}
}

/* -------------------------------------------------------------------------------- */

FMakePointsDataflowNode_v2::FMakePointsDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Points);
}

void FMakePointsDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Points))
	{
		FDataflowPoints OutPoints;

		GeometryCollection::Facades::FPointsFacade PointFacadeInPoints = OutPoints.GetPointsFacade();
		PointFacadeInPoints.AddPoints(Point);

		SetValue(Context, MoveTemp(OutPoints), &Points);
	}
}

/* -------------------------------------------------------------------------------- */

FMakeNumericArrayDataflowNode::FMakeNumericArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&NumElements);
	RegisterInputConnection(&DefaultValue);
	RegisterInputConnection(&Generator);
	RegisterOutputConnection(&Array);

	DefaultValue.Value = 0.0;
}

void FMakeNumericArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Array))
	{
		const int32 InNumElements = FMath::Max(0, GetValue(Context, &NumElements));
		const double InDefaultValue = GetValue(Context, &DefaultValue);

		TArray<double> OutArray;
		OutArray.Init(InDefaultValue, InNumElements);

		const FDataflowValueGenerator& InGenerator = GetValue(Context, &Generator);
		InGenerator.GenerateValues(OutArray);

		SetValue(Context, MoveTemp(OutArray), &Array);
	}
}

#undef LOCTEXT_NAMESPACE
