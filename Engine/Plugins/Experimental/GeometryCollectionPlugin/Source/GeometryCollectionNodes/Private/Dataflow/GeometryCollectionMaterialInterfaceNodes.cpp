// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionMaterialInterfaceNodes.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowObject.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "GeometryCollection/GeometryCollection.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Misc/PackageName.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"

#include "AssetRegistry/AssetRegistryModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionMaterialInterfaceNodes)

namespace UE::Dataflow
{
	void RegisterGeometryCollectionMaterialInterfaceNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeMaterialInterfaceArrayDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetMaterialInterfaceAssetDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetIntoMaterialInterfaceArrayDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAddToMaterialInterfaceArrayDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAssignMaterialInterfaceToCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMaterialInterfaceTextureOverrideDataflowNode);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMaterialInterfaceSetFloatParameterDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMaterialInterfaceSetVectorParameterDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMaterialInterfaceSetTexture2DParameterDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMaterialInterfaceTerminalDataflowNode);

		// Deprecated nodes
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetMaterialInterfaceArraySizeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetFromMaterialInterfaceArrayDataflowNode);

		DATAFLOW_NODE_REGISTER_GETTER_FOR_ASSET(UMaterialInterface, FGetMaterialInterfaceAssetDataflowNode);
	}
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////
FMakeMaterialInterfaceArrayDataflowNode::FMakeMaterialInterfaceArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&MaterialArray);
}

void FMakeMaterialInterfaceArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&MaterialArray))
	{
		SetValue(Context, MaterialArray, &MaterialArray);
	}
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////
FGetMaterialInterfaceAssetDataflowNode::FGetMaterialInterfaceAssetDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Material);
}

void FGetMaterialInterfaceAssetDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Material))
	{
		SetValue(Context, Material, &Material);
	}
}

bool FGetMaterialInterfaceAssetDataflowNode::SupportsAssetProperty(UObject* Asset) const
{
	return (Cast<UMaterialInterface>(Asset) != nullptr);
}

void FGetMaterialInterfaceAssetDataflowNode::SetAssetProperty(UObject* Asset)
{
	if (UMaterialInterface* MaterialAsset = Cast<UMaterialInterface>(Asset))
	{
		Material = MaterialAsset;
	}
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////
FGetMaterialInterfaceArraySizeDataflowNode::FGetMaterialInterfaceArraySizeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MaterialArray);
	RegisterOutputConnection(&MaterialArray, &MaterialArray);
	RegisterOutputConnection(&Size);
}

void FGetMaterialInterfaceArraySizeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&MaterialArray))
	{
		SafeForwardInput(Context, &MaterialArray, &MaterialArray);
	}
	else if (Out->IsA(&Size))
	{
		const TArray<TObjectPtr<UMaterialInterface>>& InMaterialArray = GetValue(Context, &MaterialArray);
		SetValue(Context, InMaterialArray.Num(), &Size);
	}
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////

FGetFromMaterialInterfaceArrayDataflowNode::FGetFromMaterialInterfaceArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MaterialArray);
	RegisterInputConnection(&Index);
	RegisterOutputConnection(&MaterialArray, &MaterialArray);
	RegisterOutputConnection(&Material);
}

void FGetFromMaterialInterfaceArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&MaterialArray))
	{
		SafeForwardInput(Context, &MaterialArray, &MaterialArray);
	}
	else if (Out->IsA(&Material))
	{
		const TArray<TObjectPtr<UMaterialInterface>>& InMaterialArray = GetValue(Context, &MaterialArray);
		const int32 InIndex = GetValue(Context, &Index);

		TObjectPtr<UMaterialInterface> OutMaterial = nullptr;
		if (InMaterialArray.IsValidIndex(InIndex))
		{
			OutMaterial = InMaterialArray[InIndex];
		}
		SetValue(Context, OutMaterial, &Material);
	}
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////

FSetIntoMaterialInterfaceArrayDataflowNode::FSetIntoMaterialInterfaceArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MaterialArray);
	RegisterInputConnection(&Material);
	RegisterInputConnection(&Index);
	RegisterOutputConnection(&MaterialArray, &MaterialArray);
}

void FSetIntoMaterialInterfaceArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&MaterialArray))
	{
		TArray<TObjectPtr<UMaterialInterface>> InMaterialArray = GetValue(Context, &MaterialArray);
		TObjectPtr<UMaterialInterface> InMaterial = GetValue(Context, &Material);
		const int32 InIndex = GetValue(Context, &Index);

		if (InMaterialArray.IsValidIndex(InIndex))
		{
			InMaterialArray[InIndex] = InMaterial;
		}
		SetValue(Context, MoveTemp(InMaterialArray), &MaterialArray);
	}
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////

FAddToMaterialInterfaceArrayDataflowNode::FAddToMaterialInterfaceArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MaterialArray);
	RegisterOutputConnection(&MaterialArray, &MaterialArray);
	
	// Add initial variable inputs
	for (int32 Index = 0; Index < NumInitialVariableInputs; ++Index)
	{
		AddPins();
	}
}

void FAddToMaterialInterfaceArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&MaterialArray))
	{
		TArray<TObjectPtr<UMaterialInterface>> InMaterialArray = GetValue(Context, &MaterialArray);
		for (int32 Index = 0; Index < MaterialsToAdd.Num(); ++Index)
		{
			const TObjectPtr<UMaterialInterface> MaterialToAdd = GetValue(Context,GetConnectionReference(Index));
			InMaterialArray.Add(MaterialToAdd);
		}
		SetValue(Context, MoveTemp(InMaterialArray), &MaterialArray);
	}
}

bool FAddToMaterialInterfaceArrayDataflowNode::CanAddPin() const
{
	return true;
}

bool FAddToMaterialInterfaceArrayDataflowNode::CanRemovePin() const
{
	return MaterialsToAdd.Num() > 0;
}

UE::Dataflow::TConnectionReference<TObjectPtr<UMaterialInterface>> FAddToMaterialInterfaceArrayDataflowNode::GetConnectionReference(int32 Index) const
{
	return { &MaterialsToAdd[Index], Index, &MaterialsToAdd };
}

TArray<UE::Dataflow::FPin> FAddToMaterialInterfaceArrayDataflowNode::AddPins()
{
	const int32 Index = MaterialsToAdd.AddDefaulted();
	const FDataflowInput& Input = RegisterInputArrayConnection(GetConnectionReference(Index));
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}

TArray<UE::Dataflow::FPin> FAddToMaterialInterfaceArrayDataflowNode::GetPinsToRemove() const
{
	const int32 Index = (MaterialsToAdd.Num() - 1);
	check(MaterialsToAdd.IsValidIndex(Index));
	if (const FDataflowInput* const Input = FindInput(GetConnectionReference(Index)))
	{
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	}
	return Super::GetPinsToRemove();
}

void FAddToMaterialInterfaceArrayDataflowNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = MaterialsToAdd.Num() - 1;
	check(MaterialsToAdd.IsValidIndex(Index));
#if DO_CHECK
	const FDataflowInput* const Input = FindInput(GetConnectionReference(Index));
	check(Input);
	check(Input->GetName() == Pin.Name);
	check(Input->GetType() == Pin.Type);
#endif
	MaterialsToAdd.SetNum(Index);

	return Super::OnPinRemoved(Pin);
}

void FAddToMaterialInterfaceArrayDataflowNode::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		check(MaterialsToAdd.Num() >= 0);
		// register new elements from the array as inputs
		for (int32 Index = 0; Index < MaterialsToAdd.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetConnectionReference(Index));
		}
		if (Ar.IsTransacting())
		{
			// if we have more inputs than materials then we need to unregister the inputs 
			const int32 NumVariableInputs = (GetNumInputs() - NumOtherInputs);
			const int32 NumMaterials = MaterialsToAdd.Num();
			if (NumVariableInputs > NumMaterials)
			{
				// Inputs have been removed.
				// Temporarily expand Collections so we can get connection references.
				MaterialsToAdd.SetNum(NumVariableInputs);
				for (int32 Index = NumMaterials; Index < MaterialsToAdd.Num(); ++Index)
				{
					UnregisterInputConnection(GetConnectionReference(Index));
				}
				MaterialsToAdd.SetNum(NumMaterials);
			}
		}
		else
		{
			ensureAlways(MaterialsToAdd.Num() + NumOtherInputs == GetNumInputs());
		}
	}
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////

FAssignMaterialInterfaceToCollectionDataflowNode::FAssignMaterialInterfaceToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&FaceSelection);
	RegisterInputConnection(&MaterialArray);
	RegisterInputConnection(&Material);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&MaterialArray, &MaterialArray);
	RegisterOutputConnection(&MaterialIndex);
}

int32 FAssignMaterialInterfaceToCollectionDataflowNode::AddOrMergeMaterialToArray(TArray<TObjectPtr<UMaterialInterface>>& InOutMaterials, TObjectPtr<UMaterialInterface> InMaterialToAdd) const
{
	if (bMergeDuplicateMaterials)
	{
		return InOutMaterials.AddUnique(InMaterialToAdd);
	}
	return InOutMaterials.Add(InMaterialToAdd);
}

void FAssignMaterialInterfaceToCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		FDataflowFaceSelection InFaceSelection = GetValue<FDataflowFaceSelection>(Context, &FaceSelection);
		TArray<TObjectPtr<UMaterialInterface>> InMaterialArray = GetValue(Context, &MaterialArray);
		TObjectPtr<UMaterialInterface> InMaterial = GetValue(Context, &Material);

		// add the material to the array 
		const int32 InMaterialIndex = AddOrMergeMaterialToArray(InMaterialArray, InMaterial);

		const int32 NumFaces = InCollection.NumElements(FGeometryCollection::FacesGroup);
		if (!IsConnected(&FaceSelection))
		{
			// if not connected select all faces
			InFaceSelection.InitializeFromCollection(InCollection, true);
		}

		static const FName MaterialIdAttributeName = "MaterialID";
		if (InFaceSelection.Num() <= NumFaces && 
			InCollection.HasAttribute(MaterialIdAttributeName, FGeometryCollection::FacesGroup))
		{
			TManagedArray<int32>& MaterialIDs = InCollection.ModifyAttribute<int32>(MaterialIdAttributeName, FGeometryCollection::FacesGroup);

			// Update MaterialIdx for selected outside faces
			for (int32 FaceIdx = 0; FaceIdx < InFaceSelection.Num(); ++FaceIdx)
			{
				if (InFaceSelection.IsSelected(FaceIdx))
				{
					MaterialIDs[FaceIdx] = InMaterialIndex;
				}
			}
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);

	}
	else if (Out->IsA(&MaterialArray) || Out->IsA(&MaterialIndex))
	{
		TArray<TObjectPtr<UMaterialInterface>> InMaterialArray = GetValue(Context, &MaterialArray);
		TObjectPtr<UMaterialInterface> InMaterial = GetValue(Context, &Material);

		const int32 InMaterialIndex = AddOrMergeMaterialToArray(InMaterialArray, InMaterial);

		SetValue(Context, InMaterialArray, &MaterialArray);
		SetValue(Context, InMaterialIndex, &MaterialIndex);
	}
}


FMaterialInterfaceTextureOverrideDataflowNode::FMaterialInterfaceTextureOverrideDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Material);
	RegisterOutputConnection(&Material, &Material);
	RegisterInputConnection(&TargetTexture);
	RegisterInputConnection(&OverrideTexture);
}

void FMaterialInterfaceTextureOverrideDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
#if WITH_EDITOR
	if (Out->IsA(&Material) && TargetTexture)
	{
		const TObjectPtr<UTexture2D> InOverrideTexture = GetValue(Context, &OverrideTexture);

		if (InOverrideTexture)
		{
			if (TObjectPtr<const UMaterialInterface> SourceMaterial = GetValue<TObjectPtr<UMaterialInterface>>(Context, &Material))
			{
				while (const UMaterialInstanceDynamic* const InMID = Cast<const UMaterialInstanceDynamic>(SourceMaterial))
				{
					SourceMaterial = InMID->Parent;
				}

				if (const UMaterialInstanceConstant* const SourceMIC = Cast<const UMaterialInstanceConstant>(SourceMaterial))
				{
					if (UMaterialInstanceConstant* const DuplicateMaterial = DuplicateObject<UMaterialInstanceConstant>(SourceMIC, nullptr))
					{
						for (FTextureParameterValue& Param : DuplicateMaterial->TextureParameterValues)
						{
							if (Param.ParameterValue == TargetTexture)
							{
								DuplicateMaterial->SetTextureParameterValueEditorOnly(Param.ParameterInfo, InOverrideTexture);
							}
						}

						if (const TObjectPtr<UMaterialInterface> OutMI = CastChecked<UMaterialInterface>(DuplicateMaterial))		// should be a straightforward upcast
						{
							// TODO: Make a MaterialTerminalNode so we can also save this out as an asset

							SetValue(Context, OutMI, &Material);
							return;
						}
					}
					else
					{
						Context.Warning(TEXT("Error creating duplicate material"), this, Out);
					}
				}
				else
				{
					// TODO: handle non-UMaterialInstanceConstant materials

					Context.Warning(TEXT("Input material is not a UMaterialInstanceConstant"), this, Out);
				}
			}
			else
			{
				Context.Warning(TEXT("Input material or its parent is not UMaterialInstanceConstant"), this, Out);
			}
		}
	}
#else	// WITH_EDITOR

	// TODO: Do something sensible for non-editor execution

	Context.Error(TEXT("FMaterialInterfaceTextureOverrideDataflowNode is only available in Editor"), this, Out);

#endif	// WITH_EDITOR

	SafeForwardInput(Context, &Material, &Material);
}


//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FMaterialInterfaceSetFloatParameterDataflowNode::FMaterialInterfaceSetFloatParameterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MaterialParameters);
	RegisterInputConnection(&ParameterName);
	RegisterInputConnection(&ParameterValue);
	RegisterOutputConnection(&MaterialParameters, &MaterialParameters);
}

void FMaterialInterfaceSetFloatParameterDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&MaterialParameters))
	{
		FDataflowMaterialParameters OutMaterialParameters = GetValue(Context, &MaterialParameters);

		FDataflowMaterialFloatParameter FloatParameter;
		FloatParameter.Name = FName(GetValue(Context, &ParameterName));
		FloatParameter.Value = static_cast<float>(GetValue(Context, &ParameterValue));

		if (!FloatParameter.Name.IsNone())
		{
			OutMaterialParameters.FloatParameters.Add(FloatParameter);
		}

		SetValue(Context, MoveTemp(OutMaterialParameters), &MaterialParameters);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FMaterialInterfaceSetVectorParameterDataflowNode::FMaterialInterfaceSetVectorParameterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MaterialParameters);
	RegisterInputConnection(&ParameterName);
	RegisterInputConnection(&ParameterValue);
	RegisterOutputConnection(&MaterialParameters, &MaterialParameters);
}

void FMaterialInterfaceSetVectorParameterDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&MaterialParameters))
	{
		FDataflowMaterialParameters OutMaterialParameters = GetValue(Context, &MaterialParameters);

		FDataflowMaterialVectorParameter VectorParameter;
		VectorParameter.Name = FName(GetValue(Context, &ParameterName));
		VectorParameter.Value = GetValue(Context, &ParameterValue);

		if (!VectorParameter.Name.IsNone())
		{
			OutMaterialParameters.VectorParameters.Add(VectorParameter);
		}

		SetValue(Context, MoveTemp(OutMaterialParameters), &MaterialParameters);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FMaterialInterfaceSetTexture2DParameterDataflowNode::FMaterialInterfaceSetTexture2DParameterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MaterialParameters);
	RegisterInputConnection(&ParameterName);
	RegisterInputConnection(&ParameterValue);
	RegisterOutputConnection(&MaterialParameters, &MaterialParameters);
}

void FMaterialInterfaceSetTexture2DParameterDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&MaterialParameters))
	{
		FDataflowMaterialParameters OutMaterialParameters = GetValue(Context, &MaterialParameters);

		FDataflowMaterialTextureParameter TextureParameter;
		TextureParameter.Name = FName(GetValue(Context, &ParameterName));
		TextureParameter.Value = GetValue(Context, &ParameterValue);

		if (!TextureParameter.Name.IsNone())
		{
			OutMaterialParameters.TextureParameters.Add(TextureParameter);
		}

		SetValue(Context, MoveTemp(OutMaterialParameters), &MaterialParameters);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FMaterialInterfaceTerminalDataflowNode::FMaterialInterfaceTerminalDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&Material);
	RegisterInputConnection(&AssetPath);
	RegisterInputConnection(&MaterialParameters);
	RegisterOutputConnection(&Material, &Material);
}

void FMaterialInterfaceTerminalDataflowNode::Evaluate(UE::Dataflow::FContext& Context) const
{
	// Asset is created and stored by the SetAssetValue
	// Evaluate only return an existing one or nullptr
	const FString& InAssetPath = GetValue(Context, &AssetPath);
	TObjectPtr<UMaterialInterface> OutMaterial = ::LoadObject<UMaterialInterface>(nullptr, InAssetPath);
	SetValue(Context, OutMaterial, &Material);
}

void FMaterialInterfaceTerminalDataflowNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	TObjectPtr<UMaterialInterface> NullMaterialPtr = nullptr;

#if WITH_EDITOR
	const TObjectPtr<UMaterialInterface> ParentMaterial = GetValue(Context, &Material);
	if (ParentMaterial == nullptr)
	{
		Context.Warning(TEXT("Parent Material is null, material instance cannot be created"), this);
		SetValue(Context, NullMaterialPtr, &Material);
		return;
	}

	const FDataflowMaterialParameters& InMaterialParameters = GetValue(Context, &MaterialParameters);
	const FString& InAssetPath = GetValue(Context, &AssetPath);
	if (!FPackageName::IsValidObjectPath(InAssetPath))
	{
		Context.Warning(FString::Format(TEXT("Asset Path input is not a valid path : {0}"), { InAssetPath }), this);
		SetValue(Context, NullMaterialPtr, &Material);
		return;
	}

	const FString PackageName = FPackageName::ObjectPathToPackageName(InAssetPath);

	UPackage* Package = Cast<UPackage>(FindPackage(nullptr, *PackageName));
	if (Package == nullptr)
	{
		Package = CreatePackage(*PackageName);
	}
	if (Package == nullptr)
	{
		Context.Warning(FString::Format(TEXT("Failed to find or create package {0}"), { PackageName }), this);
		SetValue(Context, NullMaterialPtr, &Material);
		return;
	}

	const FString AssetName = FPackageName::GetLongPackageAssetName(PackageName);
	UMaterialInstanceConstant* MaterialInstance = NewObject<UMaterialInstanceConstant>(Package, UMaterialInstanceConstant::StaticClass(), FName(AssetName), RF_Public | RF_Standalone | RF_Transactional);
	if (!MaterialInstance)
	{
		Context.Warning(FString::Format(TEXT("Material instance {0} with cannot be created"), { InAssetPath }), this);
		SetValue(Context, NullMaterialPtr, &Material);
		return;
	}

	// Setup the new material instance
	MaterialInstance->SetParentEditorOnly(ParentMaterial);

	for (const FDataflowMaterialFloatParameter& FloatParameter : InMaterialParameters.FloatParameters)
	{
		MaterialInstance->SetScalarParameterValueEditorOnly(*FloatParameter.Name.ToString(), FloatParameter.Value);
	}

	for (const FDataflowMaterialVectorParameter& VectorParameter : InMaterialParameters.VectorParameters)
	{
		MaterialInstance->SetVectorParameterValueEditorOnly(*VectorParameter.Name.ToString(), FLinearColor(VectorParameter.Value));
	}

	for (const FDataflowMaterialTextureParameter& TextureParameter : InMaterialParameters.TextureParameters)
	{
		MaterialInstance->SetTextureParameterValueEditorOnly(*TextureParameter.Name.ToString(), TextureParameter.Value.Get());
	}

	// make sure the asset is set properly
	MaterialInstance->MarkPackageDirty();
	FAssetRegistryModule::AssetCreated(MaterialInstance);

	// Finalize the new material asset creation
	MaterialInstance->PreEditChange(nullptr);
	MaterialInstance->PostEditChange();

	TObjectPtr<UMaterialInterface> OutMaterialInstance = MaterialInstance;
	SetValue(Context, OutMaterialInstance, &Material);

#else
	Context.Error(TEXT("Creation of material instance only supported in Editor"), this);
	SetValue(Context, NullMaterialPtr, &Material);
#endif
}
