// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionMaterialNodes.h"
#include "Dataflow/DataflowCore.h"

#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionEngineUtility.h"
#include "GeometryCollection/GeometryCollectionEngineRemoval.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealTypePrivate.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "StaticMeshAttributes.h"
#include "DynamicMeshEditor.h"
#include "Operations/MeshBoolean.h"
#include "Materials/Material.h"

#include "EngineGlobals.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "Voronoi/Voronoi.h"
#include "PlanarCut.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "FractureEngineClustering.h"
#include "FractureEngineSelection.h"
#include "GeometryCollection/Facades/CollectionBoundsFacade.h"
#include "GeometryCollection/Facades/CollectionAnchoringFacade.h"
#include "GeometryCollection/Facades/CollectionRemoveOnBreakFacade.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/DynamicMesh3.h"

//#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionMaterialNodes)

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionMaterialNodes)

namespace UE::Dataflow
{
	void GeometryCollectionMaterialNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAddMaterialToCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FReAssignMaterialInCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMaterialsInfoDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetMaterialFromMaterialsArrayDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetMaterialInMaterialsArrayDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeMaterialDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeMaterialsArrayDataflowNode);
	}
}


void FAddMaterialToCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection) || Out->IsA<TArray<TObjectPtr<UMaterial>>>(&Materials))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		TArray<TObjectPtr<UMaterial>> InMaterials = GetValue<TArray<TObjectPtr<UMaterial>>>(Context, &Materials);
		const FDataflowFaceSelection& InFaceSelection = GetValue<FDataflowFaceSelection>(Context, &FaceSelection);
		TObjectPtr<UMaterial> InOutsideMaterial = GetValue<TObjectPtr<UMaterial>>(Context, &OutsideMaterial);
		TObjectPtr<UMaterial> InInsideMaterial = GetValue<TObjectPtr<UMaterial>>(Context, &InsideMaterial);

		int32 NewIndex;

		if (InCollection.HasAttribute("Internal", FGeometryCollection::FacesGroup))
		{
			if (bAssignOutsideMaterial || bAssignInsideMaterial)
			{
				TManagedArray<int32>& MaterialIDs = InCollection.ModifyAttribute<int32>("MaterialID", FGeometryCollection::FacesGroup);

				const TManagedArray<bool>& Internals = InCollection.GetAttribute<bool>("Internal", FGeometryCollection::FacesGroup);

				if (bAssignOutsideMaterial)
				{
					NewIndex = InMaterials.Add(InOutsideMaterial);

					// Update MaterialIdx for selected outside faces
					for (int32 FaceIdx = 0; FaceIdx < InFaceSelection.Num(); ++FaceIdx)
					{
						if (!Internals[FaceIdx] && InFaceSelection.IsSelected(FaceIdx))
						{
							MaterialIDs[FaceIdx] = NewIndex;
						}
					}
				}

				if (bAssignInsideMaterial)
				{
					NewIndex = InMaterials.Add(InInsideMaterial);

					// Update MaterialIdx for selected inside faces
					for (int32 FaceIdx = 0; FaceIdx < InFaceSelection.Num(); ++FaceIdx)
					{
						if (Internals[FaceIdx] && InFaceSelection.IsSelected(FaceIdx))
						{
							MaterialIDs[FaceIdx] = NewIndex;
						}
					}
				}
			}
		}

		SetValue(Context, MoveTemp(InMaterials), &Materials);
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}


void FReAssignMaterialInCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const FDataflowFaceSelection& InFaceSelection = GetValue<FDataflowFaceSelection>(Context, &FaceSelection);
		const TArray<TObjectPtr<UMaterial>>& InMaterials = GetValue<TArray<TObjectPtr<UMaterial>>>(Context, &Materials);
		int32 InOutsideMaterialIdx = GetValue<int32>(Context, &OutsideMaterialIdx);
		int32 InInsideMaterialIdx = GetValue<int32>(Context, &InsideMaterialIdx);

		if (InMaterials.Num() > 0)
		{
			if (InCollection.HasAttribute("Internal", FGeometryCollection::FacesGroup))
			{
				if (bAssignOutsideMaterial || bAssignInsideMaterial)
				{
					TManagedArray<int32>& MaterialIDs = InCollection.ModifyAttribute<int32>("MaterialID", FGeometryCollection::FacesGroup);

					const TManagedArray<bool>& Internals = InCollection.GetAttribute<bool>("Internal", FGeometryCollection::FacesGroup);

					if (bAssignOutsideMaterial)
					{
						if (InOutsideMaterialIdx >= 0 && InOutsideMaterialIdx < InMaterials.Num())
						{
							// Update MaterialIdx for selected outside faces
							for (int32 FaceIdx = 0; FaceIdx < InFaceSelection.Num(); ++FaceIdx)
							{
								if (!Internals[FaceIdx] && InFaceSelection.IsSelected(FaceIdx))
								{
									MaterialIDs[FaceIdx] = InOutsideMaterialIdx;
								}
							}
						}
					}

					if (bAssignInsideMaterial)
					{
						if (InInsideMaterialIdx >= 0 && InInsideMaterialIdx < InMaterials.Num())
						{
							// Update MaterialIdx for selected inside faces
							for (int32 FaceIdx = 0; FaceIdx < InFaceSelection.Num(); ++FaceIdx)
							{
								if (Internals[FaceIdx] && InFaceSelection.IsSelected(FaceIdx))
								{
									MaterialIDs[FaceIdx] = InInsideMaterialIdx;
								}
							}
						}
					}
				}
			}
		}

		// move the collection to the output to avoid making another copy
		SetValue(Context, MoveTemp(InCollection), &Collection);
		SetValue(Context, InMaterials, &Materials);
	}
}


void FMaterialsInfoDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FString>(&String))
	{
		const TArray<TObjectPtr<UMaterial>>& InMaterials = GetValue<TArray<TObjectPtr<UMaterial>>>(Context, &Materials);

		FString OutputStr;

		OutputStr.Appendf(TEXT("\n----------------------------------------\n"));
		OutputStr.Appendf(TEXT("Number of Materials: %d\n"), InMaterials.Num());

		int32 Idx = 0;
		for (const TObjectPtr<UMaterial>& Material : InMaterials)
		{
			OutputStr.Appendf(TEXT("%4d: %s\n"), Idx, *(Material->GetFullName()));

			Idx++;
		}

		OutputStr.Appendf(TEXT("----------------------------------------\n"));

		SetValue(Context, OutputStr, &String);
	}
}


void FGetMaterialFromMaterialsArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TObjectPtr<UMaterial>>(&Material))
	{
		const TArray<TObjectPtr<UMaterial>>& InMaterials = GetValue<TArray<TObjectPtr<UMaterial>>>(Context, &Materials);
		int32 InMaterialIdx = GetValue<int32>(Context, &MaterialIdx);

		if (InMaterials.Num() > 0)
		{
			if (InMaterialIdx >= 0 && InMaterialIdx < InMaterials.Num())
			{
				SetValue(Context, InMaterials[InMaterialIdx], &Material);

				return;
			}
		}

		SetValue(Context, TObjectPtr<UMaterial>(), &Material);
	}
}


void FSetMaterialInMaterialsArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<TObjectPtr<UMaterial>>>(&Materials))
	{
		TArray<TObjectPtr<UMaterial>> InMaterials = GetValue<TArray<TObjectPtr<UMaterial>>>(Context, &Materials);
		TObjectPtr<UMaterial> InMaterial = GetValue<TObjectPtr<UMaterial>>(Context, &Material);
		int32 InMaterialIdx = GetValue<int32>(Context, &MaterialIdx);

		if (Operation == ESetMaterialOperationTypeEnum::Dataflow_SetMaterialOperationType_Add)
		{
			InMaterials.Add(InMaterial);
		}
		else
		{
			if (InMaterialIdx >= 0 && InMaterialIdx < InMaterials.Num())
			{
				InMaterials.Insert(InMaterial, InMaterialIdx);
			}
		}

		SetValue(Context, MoveTemp(InMaterials), &Materials);
	}
}


void FMakeMaterialDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TObjectPtr<UMaterial>>(&Material))
	{
		SetValue(Context, InMaterial, &Material);
	}
}


void FMakeMaterialsArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<TObjectPtr<UMaterial>>>(&Materials))
	{
		SetValue(Context, TArray<TObjectPtr<UMaterial>>(), &Materials);
	}
}

