// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionMathNodes.h"
#include "Math/BasicMathExpressionEvaluator.h"
#include "Dataflow/DataflowCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionMathNodes)

namespace UE::Dataflow
{

	void GeometryCollectionMathNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAddDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSubtractDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMultiplyDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSafeDivideDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDivideDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDivisionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSafeReciprocalDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSquareDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSquareRootDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FInverseSqrtDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCubeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FNegateDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAbsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFloorDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCeilDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRoundDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FTruncDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFracDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMinDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMaxDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMin3DataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMax3DataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSignDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FClampDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFitDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FEFitDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FPowDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FLogDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FLogeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FLerpDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FWrapDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FExpDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSinDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FArcSinDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCosDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FArcCosDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FTanDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FArcTanDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FArcTan2DataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FNormalizeToRangeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FScaleVectorDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDotProductDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCrossProductDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FNormalizeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FLengthDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDistanceDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FIsNearlyZeroDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRandomFloatDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRandomFloatInRangeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRandomUnitVectorDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRandomUnitVectorInConeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRadiansToDegreesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDegreesToRadiansDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMathConstantsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FOneMinusDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFloatMathExpressionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMathExpressionDataflowNode);
	}
}

void FAddDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloatA = GetValue<float>(Context, &FloatA, FloatA);
		const float InFloatB = GetValue<float>(Context, &FloatB, FloatB);

		const float Result = InFloatA + InFloatB;
		SetValue(Context, Result, &ReturnValue);
	}
}

void FSubtractDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloatA = GetValue<float>(Context, &FloatA, FloatA);
		const float InFloatB = GetValue<float>(Context, &FloatB, FloatB);

		const float Result = InFloatA - InFloatB;
		SetValue(Context, Result, &ReturnValue);
	}
}

void FMultiplyDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloatA = GetValue<float>(Context, &FloatA, FloatA);
		const float InFloatB = GetValue<float>(Context, &FloatB, FloatB);

		const float Result = InFloatA * InFloatB;
		SetValue(Context, Result, &ReturnValue);
	}
}

void FSafeDivideDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloatA = GetValue<float>(Context, &FloatA, FloatA);
		const float InFloatB = GetValue<float>(Context, &FloatB, FloatB);

		float Result = 0.f;
		if (InFloatB != 0.f)
		{
			Result = InFloatA / InFloatB;
			SetValue(Context, Result, &ReturnValue);
		}
		else
		{
			SetError(Context, &ReturnValue, TEXT("Division by zero error"));
		}
	}
}

void FDivisionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&Remainder) ||
		Out->IsA<int32>(&ReturnValue))
	{
		const float InDividend = GetValue<float>(Context, &Dividend, Dividend);
		const float InDivisor = GetValue<float>(Context, &Divisor, Divisor);

		float ResultRemainder = 0.f;
		int32 Result = 0;

		if (InDivisor != 0.f)
		{
			Result = (int32)(InDividend / InDivisor);
			ResultRemainder = InDividend - (float)Result * InDivisor;
		}
		SetValue(Context, ResultRemainder, &Remainder);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FSafeReciprocalDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		float Result = 0.f;
		if (InFloat != 0.f)
		{
			Result = 1.f / InFloat;
		}
		SetValue(Context, Result, &ReturnValue);
	}
}

void FSquareDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		const float Result = InFloat * InFloat;
		SetValue(Context, Result, &ReturnValue);
	}
}

void FSquareRootDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		float Result = 0.f;
		if (InFloat > 0.f)
		{
			Result = FMath::Sqrt(InFloat);
		}
		SetValue(Context, Result, &ReturnValue);
	}
}

void FInverseSqrtDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		float Result = 0.f;
		if (InFloat > 0.f)
		{
			Result = FMath::InvSqrt(InFloat);
		}
		SetValue(Context, Result, &ReturnValue);
	}
}

void FCubeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		const float Result = InFloat * InFloat * InFloat;
		SetValue(Context, Result, &ReturnValue);
	}
}

void FNegateDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		const float Result = -1.f * InFloat;
		SetValue(Context, Result, &ReturnValue);
	}
}

void FAbsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		const float Result = FMath::Abs(InFloat);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FFloorDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		const float Result = FMath::Floor(InFloat);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FCeilDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		float Result = FMath::CeilToFloat(InFloat);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FRoundDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		float Result = FMath::RoundToFloat(InFloat);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FTruncDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		const float Result = FMath::TruncToFloat(InFloat);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FFracDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		const float Result = FMath::Frac(InFloat);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FMinDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloatA = GetValue<float>(Context, &FloatA, FloatA);
		const float InFloatB = GetValue<float>(Context, &FloatB, FloatB);

		const float Result = FMath::Min(InFloatA, InFloatB);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FMaxDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloatA = GetValue<float>(Context, &FloatA, FloatA);
		const float InFloatB = GetValue<float>(Context, &FloatB, FloatB);

		const float Result = FMath::Max(InFloatA, InFloatB);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FMin3DataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloatA = GetValue<float>(Context, &FloatA, FloatA);
		const float InFloatB = GetValue<float>(Context, &FloatB, FloatB);
		const float InFloatC = GetValue<float>(Context, &FloatC, FloatB);

		const float Result = FMath::Min3(InFloatA, InFloatB, InFloatC);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FMax3DataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloatA = GetValue<float>(Context, &FloatA, FloatA);
		const float InFloatB = GetValue<float>(Context, &FloatB, FloatB);
		const float InFloatC = GetValue<float>(Context, &FloatC, FloatB);

		const float Result = FMath::Max3(InFloatA, InFloatB, InFloatC);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FSignDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		const float Result = FMath::Sign(InFloat);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FClampDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);
		const float InMin = GetValue<float>(Context, &Min, Min);
		const float InMax = GetValue<float>(Context, &Max, Max);

		const float Result = FMath::Clamp(InFloat, InMin, InMax);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FFitDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		float InFloat = GetValue<float>(Context, &Float, Float);
		const float InOldMin = GetValue<float>(Context, &OldMin, OldMin);
		const float InOldMax = GetValue<float>(Context, &OldMax, OldMax);
		const float InNewMin = GetValue<float>(Context, &NewMin, NewMin);
		const float InNewMax = GetValue<float>(Context, &NewMax, NewMax);

		float Result = InFloat;
		if (InOldMax > InOldMin && InNewMax > InNewMin)
		{
			InFloat = FMath::Clamp(InFloat, InOldMin, InOldMax);
			float Q = (InFloat - InOldMin) / (InOldMax - InOldMin);
			Result = InNewMin + Q * (InNewMax - InNewMin);
		}
		SetValue(Context, Result, &ReturnValue);
	}
}

void FEFitDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);
		const float InOldMin = GetValue<float>(Context, &OldMin, OldMin);
		const float InOldMax = GetValue<float>(Context, &OldMax, OldMax);
		const float InNewMin = GetValue<float>(Context, &NewMin, NewMin);
		const float InNewMax = GetValue<float>(Context, &NewMax, NewMax);

		float Result = InFloat;
		if (InOldMax > InOldMin && InNewMax > InNewMin)
		{
			float Q = (InFloat - InOldMin) / (InOldMax - InOldMin);
			Result = InNewMin + Q * (InNewMax - InNewMin);
		}
		SetValue(Context, Result, &ReturnValue);
	}
}

void FPowDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InBase = GetValue<float>(Context, &Base, Base);
		const float InExp = GetValue<float>(Context, &Exp, Exp);

		const float Result = FMath::Pow(InBase, InExp);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FLogDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InBase = GetValue<float>(Context, &Base, Base);
		const float InA = GetValue<float>(Context, &A, A);

		float Result = 0;
		if (InBase > 0.f)
		{
			Result = FMath::LogX(InBase, A);
		}
		SetValue(Context, Result, &ReturnValue);
	}
}

void FLogeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InA = GetValue<float>(Context, &A, A);

		const float Result = FMath::Loge(A);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FLerpDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&ReturnValue))
	{
		const float InA = GetValue(Context, &A, A);
		const float InB = GetValue(Context, &B, B);
		const float InAlpha = GetValue(Context, &Alpha, Alpha);

		float Result = InA;
		Result = FMath::Lerp(InA, InB, InAlpha);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FWrapDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);
		const float InMin = GetValue<float>(Context, &Min, Min);
		const float InMax = GetValue<float>(Context, &Max, Max);

		float Result = InFloat;
		if (InMax > InMin)
		{
			Result = FMath::Wrap(InFloat, InMin, InMax);
		}
		SetValue(Context, Result, &ReturnValue);
	}
}

void FExpDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		const float Result = FMath::Exp(InFloat);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FSinDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		const float Result = FMath::Sin(InFloat);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FArcSinDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		float Result = 0.f;
		if (InFloat >= -1.f && InFloat <= 1.f)
		{
			Result = FMath::Asin(InFloat);
		}
		SetValue(Context, Result, &ReturnValue);
	}
}

void FCosDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		const float Result = FMath::Cos(InFloat);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FArcCosDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		float Result = 0.f;
		if (InFloat >= -1.f && InFloat <= 1.f)
		{
			Result = FMath::Acos(InFloat);
		}
		SetValue(Context, Result, &ReturnValue);
	}
}

void FTanDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		const float Result = FMath::Tan(InFloat);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FArcTanDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		const float Result = FMath::Atan(InFloat);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FArcTan2DataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InY = GetValue<float>(Context, &Y, Y);
		const float InX = GetValue<float>(Context, &X, X);

		const float Result = FMath::Atan2(InY, InX);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FNormalizeToRangeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);
		float IRangeMin = GetValue<float>(Context, &RangeMin, RangeMin);
		float InRangeMax = GetValue<float>(Context, &RangeMax, RangeMax);

		float Result;
		if (IRangeMin == InRangeMax)
		{
			if (InFloat < IRangeMin)
			{
				Result = 0.f;
			}
			else
			{
				Result = 1.f;
			}
		}

		if (IRangeMin > InRangeMax)
		{
			Swap(IRangeMin, InRangeMax);
		}
		Result = (InFloat - IRangeMin) / (InRangeMax - IRangeMin);

		SetValue(Context, Result, &ReturnValue);
	}
}


void FScaleVectorDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&ScaledVector))
	{
		const FVector InVector = GetValue(Context, &Vector, Vector);
		const float InScale = GetValue(Context, &Scale, Scale);

		const FVector Result = InVector * InScale;
		SetValue(Context, Result, &ScaledVector);
	}
}

void FDotProductDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const FVector InVectorA = GetValue<FVector>(Context, &VectorA, VectorA);
		const FVector InVectorB = GetValue<FVector>(Context, &VectorB, VectorB);

		const float Result = FVector::DotProduct(InVectorA, InVectorB);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FCrossProductDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FVector>(&ReturnValue))
	{
		const FVector InVectorA = GetValue<FVector>(Context, &VectorA, VectorA);
		const FVector InVectorB = GetValue<FVector>(Context, &VectorB, VectorB);

		const FVector Result = FVector::CrossProduct(InVectorA, InVectorB);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FNormalizeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FVector>(&ReturnValue))
	{
		const FVector InVectorA = GetValue<FVector>(Context, &VectorA, VectorA);
		const float InTolerance = GetValue<float>(Context, &Tolerance, Tolerance);

		const FVector Result = InVectorA.GetSafeNormal(Tolerance);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FLengthDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const FVector InVector = GetValue<FVector>(Context, &Vector, Vector);

		const float Result = InVector.Length();
		SetValue(Context, Result, &ReturnValue);
	}
}

void FDistanceDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const FVector InPointA = GetValue<FVector>(Context, &PointA, PointA);
		const FVector InPointB = GetValue<FVector>(Context, &PointB, PointB);

		const float Result = (InPointB - InPointA).Length();
		SetValue(Context, Result, &ReturnValue);
	}
}

void FIsNearlyZeroDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<bool>(&ReturnValue))
	{
		const float InFloat = GetValue<float>(Context, &Float, Float);

		const bool Result = FMath::IsNearlyZero(InFloat);
		SetValue(Context, Result, &ReturnValue);
	}
}

void FRandomFloatDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		if (bDeterministic)
		{
			const float RandomSeedVal = GetValue<float>(Context, &RandomSeed);

			const FRandomStream Stream(RandomSeedVal);
			SetValue(Context, Stream.FRand(), &ReturnValue);
		}
		else
		{
			SetValue(Context, FMath::FRand(), &ReturnValue);
		}
	}
}

void FRandomFloatInRangeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		float MinVal = GetValue<float>(Context, &Min);
		float MaxVal = GetValue<float>(Context, &Max);

		if (bDeterministic)
		{
			const float RandomSeedVal = GetValue<float>(Context, &RandomSeed);

			const FRandomStream Stream(RandomSeedVal);
			SetValue(Context, (float)Stream.FRandRange(MinVal, MaxVal), &ReturnValue);
		}
		else
		{
			SetValue(Context, FMath::FRandRange(MinVal, MaxVal), &ReturnValue);
		}
	}
}

void FRandomUnitVectorDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FVector>(&ReturnValue))
	{
		if (bDeterministic)
		{
			const float RandomSeedVal = GetValue<float>(Context, &RandomSeed);

			const FRandomStream Stream(RandomSeedVal);
			SetValue(Context, Stream.VRand(), &ReturnValue);
		}
		else
		{
			SetValue(Context, FMath::VRand(), &ReturnValue);
		}
	}
}

void FRandomUnitVectorInConeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FVector>(&ReturnValue))
	{
		FVector ConeDirectionVal = GetValue<FVector>(Context, &ConeDirection);
		float ConeHalfAngleVal = GetValue<float>(Context, &ConeHalfAngle);

		if (bDeterministic)
		{
			const float RandomSeedVal = GetValue<float>(Context, &RandomSeed);

			const FRandomStream Stream(RandomSeedVal);
			SetValue(Context, Stream.VRandCone(ConeDirectionVal, ConeHalfAngleVal), &ReturnValue);
		}
		else
		{
			SetValue(Context, FMath::VRandCone(ConeDirectionVal, ConeHalfAngleVal), &ReturnValue);
		}
	}
}

void FRadiansToDegreesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&Degrees))
	{
		SetValue(Context, FMath::RadiansToDegrees(GetValue<float>(Context, &Radians)), &Degrees);
	}
}

void FDegreesToRadiansDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&Radians))
	{
		SetValue(Context, FMath::DegreesToRadians(GetValue<float>(Context, &Degrees)), &Radians);
	}
}

void FMathConstantsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		if (Constant == EMathConstantsEnum::Dataflow_MathConstants_Pi)
		{
			SetValue(Context, FMathf::Pi, &ReturnValue);
		}
		else if (Constant == EMathConstantsEnum::Dataflow_MathConstants_HalfPi)
		{
			SetValue(Context, FMathf::HalfPi, &ReturnValue);
		}
		else if (Constant == EMathConstantsEnum::Dataflow_MathConstants_TwoPi)
		{
			SetValue(Context, FMathf::TwoPi, &ReturnValue);
		}
		else if (Constant == EMathConstantsEnum::Dataflow_MathConstants_FourPi)
		{
			SetValue(Context, FMathf::FourPi, &ReturnValue);
		}
		else if (Constant == EMathConstantsEnum::Dataflow_MathConstants_InvPi)
		{
			SetValue(Context, FMathf::InvPi, &ReturnValue);
		}
		else if (Constant == EMathConstantsEnum::Dataflow_MathConstants_InvTwoPi)
		{
			SetValue(Context, FMathf::InvTwoPi, &ReturnValue);
		}
		else if (Constant == EMathConstantsEnum::Dataflow_MathConstants_Sqrt2)
		{
			SetValue(Context, FMathf::Sqrt2, &ReturnValue);
		}
		else if (Constant == EMathConstantsEnum::Dataflow_MathConstants_InvSqrt2)
		{
			SetValue(Context, FMathf::InvSqrt2, &ReturnValue);
		}
		else if (Constant == EMathConstantsEnum::Dataflow_MathConstants_Sqrt3)
		{
			SetValue(Context, FMathf::Sqrt3, &ReturnValue);
		}
		else if (Constant == EMathConstantsEnum::Dataflow_MathConstants_InvSqrt3)
		{
			SetValue(Context, FMathf::InvSqrt3, &ReturnValue);
		}
		else if (Constant == EMathConstantsEnum::Dataflow_FloatToInt_Function_E)
		{
			SetValue(Context, 2.71828182845904523536f, &ReturnValue);
		}
		else if (Constant == EMathConstantsEnum::Dataflow_FloatToInt_Function_Gamma)
		{
			SetValue(Context, 0.577215664901532860606512090082f, &ReturnValue);
		}
		else if (Constant == EMathConstantsEnum::Dataflow_FloatToInt_Function_GoldenRatio)
		{
			SetValue(Context, 1.618033988749894f, &ReturnValue);
		}
		else if (Constant == EMathConstantsEnum::Dataflow_FloatToInt_Function_ZeroTolerance)
		{
			SetValue(Context, FMathf::ZeroTolerance, &ReturnValue);
		}
	}
}

void FOneMinusDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		const float InA = GetValue<float>(Context, &A, A);

		const float Result = 1.f - InA;
		SetValue(Context, Result, &ReturnValue);
	}
}

FFloatMathExpressionDataflowNode::FFloatMathExpressionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&A);
	RegisterInputConnection(&B);
	RegisterInputConnection(&C);
	RegisterInputConnection(&D);
	RegisterOutputConnection(&ReturnValue);
}

void FFloatMathExpressionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	check(Out->IsA(&ReturnValue));

	float FloatResult = 0.0f;

	FString ExpressionToEvaluate{ Expression };
	ExpressionToEvaluate = ExpressionToEvaluate.TrimStartAndEnd();
	if (!ExpressionToEvaluate.IsEmpty())
	{
		const FString VarA("{A}");
		if (ExpressionToEvaluate.Contains(VarA))
		{
			const float InA = GetValue(Context, &A);
			const FString StrA = FString::SanitizeFloat(InA);
			ExpressionToEvaluate.ReplaceInline(*VarA, *StrA, ESearchCase::CaseSensitive);
		}

		const FString VarB("{B}");
		if (ExpressionToEvaluate.Contains(VarB))
		{
			const float InB = GetValue(Context, &B);
			const FString StrB = FString::SanitizeFloat(InB);
			ExpressionToEvaluate.ReplaceInline(*VarB, *StrB, ESearchCase::CaseSensitive);
		}

		const FString VarC("{C}");
		if (ExpressionToEvaluate.Contains(VarC))
		{
			const float InC = GetValue(Context, &C);
			const FString StrC = FString::SanitizeFloat(InC);
			ExpressionToEvaluate.ReplaceInline(*VarC, *StrC, ESearchCase::CaseSensitive);
		}

		const FString VarD("{D}");
		if (ExpressionToEvaluate.Contains(VarD))
		{
			const float InD = GetValue(Context, &D);
			const FString StrD = FString::SanitizeFloat(InD);
			ExpressionToEvaluate.ReplaceInline(*VarD, *StrD, ESearchCase::CaseSensitive);
		}

		FBasicMathExpressionEvaluator Evaluator;
		TValueOrError<double, FExpressionError> Result = Evaluator.Evaluate(*ExpressionToEvaluate);
		if (Result.IsValid())
		{
			FloatResult = FMath::Clamp((float)Result.GetValue(), TNumericLimits<float>::Lowest(), TNumericLimits<float>::Max());
		}
	}
	SetValue(Context, FloatResult, &ReturnValue);
}



FMathExpressionDataflowNode::FMathExpressionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&A);
	RegisterInputConnection(&B);
	RegisterInputConnection(&C);
	RegisterInputConnection(&D);
	RegisterOutputConnection(&ReturnValue);
}

void FMathExpressionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	check(Out->IsA(&ReturnValue));

	double Result = 0.0f;

	
	FString ExpressionToEvaluate{ Expression };
	ExpressionToEvaluate = ExpressionToEvaluate.TrimStartAndEnd();
	if (!ExpressionToEvaluate.IsEmpty())
	{
		const FString VarA("{A}");
		if (ExpressionToEvaluate.Contains(VarA))
		{
			const double InA = GetValue(Context, &A);
			const FString StrA = FString::SanitizeFloat(InA);
			ExpressionToEvaluate.ReplaceInline(*VarA, *StrA, ESearchCase::CaseSensitive);
		}

		const FString VarB("{B}");
		if (ExpressionToEvaluate.Contains(VarB))
		{
			const double InB = GetValue(Context, &B);
			const FString StrB = FString::SanitizeFloat(InB);
			ExpressionToEvaluate.ReplaceInline(*VarB, *StrB, ESearchCase::CaseSensitive);
		}

		const FString VarC("{C}");
		if (ExpressionToEvaluate.Contains(VarC))
		{
			const double InC = GetValue(Context, &C);
			const FString StrC = FString::SanitizeFloat(InC);
			ExpressionToEvaluate.ReplaceInline(*VarC, *StrC, ESearchCase::CaseSensitive);
		}

		const FString VarD("{D}");
		if (ExpressionToEvaluate.Contains(VarD))
		{
			const double InD = GetValue(Context, &D);
			const FString StrD = FString::SanitizeFloat(InD);
			ExpressionToEvaluate.ReplaceInline(*VarD, *StrD, ESearchCase::CaseSensitive);
		}

		FBasicMathExpressionEvaluator Evaluator;
		TValueOrError<double, FExpressionError> EvalResult = Evaluator.Evaluate(*ExpressionToEvaluate);
		if (EvalResult.IsValid())
		{
			Result = FMath::Clamp((double)EvalResult.GetValue(), TNumericLimits<double>::Lowest(), TNumericLimits<double>::Max());
		}
	}
	SetValue(Context, Result, &ReturnValue);
}

