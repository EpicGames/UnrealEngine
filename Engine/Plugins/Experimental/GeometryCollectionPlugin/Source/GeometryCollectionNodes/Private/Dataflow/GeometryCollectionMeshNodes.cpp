// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionMeshNodes.h"
#include "Dataflow/DataflowCore.h"

#include "Engine/Blueprint.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionEngineUtility.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealTypePrivate.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "StaticMeshAttributes.h"
#include "DynamicMeshEditor.h"
#include "VertexConnectedComponents.h"

#include "GeometryCollectionToDynamicMesh.h"

#include "EngineGlobals.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "Voronoi/Voronoi.h"
#include "PlanarCut.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "FractureEngineClustering.h"
#include "FractureEngineSelection.h"
#include "FractureEngineUtility.h"
#include "Dataflow/DataflowAnyTypeRegistry.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshTransforms.h"
#include "UDynamicMesh.h"

#if WITH_EDITOR
#include "Editor.h"		// For GEditor
#include "Subsystems/AssetEditorSubsystem.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionMeshNodes)

DEFINE_LOG_CATEGORY_STATIC(LogGeometryCollectionMeshNodes, Warning, All);

namespace UE::Dataflow
{

	void GeometryCollectionMeshNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FPointsToMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBoxToMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMeshInfoDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMeshToCollectionDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionToMeshDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FStaticMeshToMeshDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetMeshBoundingBoxDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMeshAppendDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDataflowMeshAppendDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeDataflowMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FDuplicateMeshUVChannelNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSplitDataflowMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSplitMeshIslandsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMeshCopyToPointsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetMeshDataDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FApplyMeshProcessorToMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FApplyMeshProcessorToGeometryCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSelectionToMeshesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAppendMeshesToCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetMeshBoundingSphereDataflowNode);
		
		// Deeprecated
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FStaticMeshToMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetMeshBoundingBoxDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionToMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMeshToCollectionDataflowNode);

		UE_DATAFLOW_REGISTER_AUTOCONVERT(TObjectPtr<UStaticMesh>, TObjectPtr<UDynamicMesh>, FStaticMeshToMeshDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(FManagedArrayCollection, TObjectPtr<UDynamicMesh>, FCollectionToMeshDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(TObjectPtr<UDynamicMesh>, FManagedArrayCollection, FMeshToCollectionDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(TObjectPtr<UDynamicMesh>, FBox, FGetMeshBoundingBoxDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(FBox, TObjectPtr<UDynamicMesh>, FBoxToMeshDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(TObjectPtr<UDynamicMesh>, TObjectPtr<UDataflowMesh>, FMakeDataflowMeshDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(TObjectPtr<UDataflowMesh>, TObjectPtr<UDynamicMesh>, FSplitDataflowMeshDataflowNode);
		UE_DATAFLOW_REGISTER_AUTOCONVERT(TObjectPtr<UStaticMesh>, TObjectPtr<UDataflowMesh>, FStaticMeshToMeshDataflowNode_v2);
	}
}


void FPointsToMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TObjectPtr<UDynamicMesh>>(&Mesh) || Out->IsA<int32>(&TriangleCount))
	{
		const TArray<FVector>& PointsArr = GetValue<TArray<FVector>>(Context, &Points);

		if (PointsArr.Num() > 0)
		{
			TObjectPtr<UDynamicMesh> DynamicMesh = NewObject<UDynamicMesh>();
			DynamicMesh->Reset();

			UE::Geometry::FDynamicMesh3& DynMesh = DynamicMesh->GetMeshRef();

			for (auto& Point : PointsArr)
			{
				DynMesh.AppendVertex(Point);
			}

			SetValue(Context, DynamicMesh, &Mesh);
			SetValue(Context, DynamicMesh->GetTriangleCount(), &TriangleCount);
		}
		else
		{
			SetValue(Context, TObjectPtr<UDynamicMesh>(NewObject<UDynamicMesh>()), &Mesh);
			SetValue(Context, 0, &TriangleCount);
		}
	}
}


void FBoxToMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TObjectPtr<UDynamicMesh>>(&Mesh) || Out->IsA<int32>(&TriangleCount))
	{
		TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
		NewMesh->Reset();

		UE::Geometry::FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();

		FBox InBox = GetValue<FBox>(Context, &Box);

		TArray<FVector3f> Vertices;
		TArray<FIntVector> Triangles;

		FFractureEngineUtility::ConvertBoxToVertexAndTriangleData(InBox, Vertices, Triangles);
		FFractureEngineUtility::ConstructMesh(DynMesh, Vertices, Triangles);

		SetValue(Context, NewMesh, &Mesh);
		SetValue(Context, NewMesh->GetTriangleCount(), &TriangleCount);
	}
}


void FMeshInfoDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FString>(&InfoString))
	{
		if (const TObjectPtr<const UDynamicMesh> InMesh = GetValue(Context, &Mesh))
		{
			const UE::Geometry::FDynamicMesh3& DynMesh = InMesh->GetMeshRef();

			SetValue(Context, DynMesh.MeshInfoString(), &InfoString);
		}
		else
		{
			SetValue(Context, FString(""), &InfoString);
		}
	}
}

// ------------------------------------------------------------------------------------------------------------------------

FMeshToCollectionDataflowNode::FMeshToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&bSplitIslands).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bAddClusterRootForSingleMesh).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Collection);
}


void FMeshToCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		if (const TObjectPtr<const UDynamicMesh> InMesh = GetValue(Context, &Mesh))
		{
			const FDynamicMesh3& DynMesh = InMesh->GetMeshRef();

			bool bInSplitIslands = GetValue(Context, &bSplitIslands);
			bool bInAlwaysAddRoot = GetValue(Context, &bAddClusterRootForSingleMesh);

			if (DynMesh.VertexCount() > 0)
			{
				FGeometryCollection NewGeometryCollection = FGeometryCollection();
				FGeometryCollectionToDynamicMeshes Convert;
				FGeometryCollectionToDynamicMeshes::FToCollectionOptions Options;

				TArray<TUniquePtr<FDynamicMesh3>> SplitMeshes;
				if (bInSplitIslands)
				{
					FVertexConnectedComponents Components(DynMesh.MaxVertexID());
					Components.ConnectTriangles(DynMesh);
					if (bConnectIslandsByVertexOverlap)
					{
						Components.ConnectCloseVertices(DynMesh, ConnectVerticesThreshold, 2);
					}
					if (VertexToSurfaceBridgeDistance > 0)
					{
						TMeshAABBTree3<FDynamicMesh3> Spatial(&DynMesh);
						Components.ConnectVerticesToNearestDifferentComponent(Spatial, (double)VertexToSurfaceBridgeDistance);
					}
					FDynamicMeshEditor::SplitMesh(&DynMesh, SplitMeshes, false, [&Components, &DynMesh](int32 TID)
						{
							return Components.GetComponent(DynMesh.GetTriangle(TID).A);
						});
				}

				auto AddRoot = [](FGeometryCollection& ToCollection) -> int32
					{
						int32 Idx = ToCollection.AddElements(1, FGeometryCollection::TransformGroup);
						ToCollection.Parent[Idx] = INDEX_NONE;
						ToCollection.BoneColor[Idx] = FLinearColor::White;
						return Idx;
					};

				if (SplitMeshes.Num() > 1)
				{
					Options.NewMeshParentIndex = AddRoot(NewGeometryCollection);
					for (const TUniquePtr<FDynamicMesh3>& SplitMesh : SplitMeshes)
					{
						Convert.AppendMeshToCollection(NewGeometryCollection, *SplitMesh, FTransform::Identity, Options);
					}
				}
				else
				{
					if (bInAlwaysAddRoot)
					{
						Options.NewMeshParentIndex = AddRoot(NewGeometryCollection);
					}
					else
					{
						Options.bAllowAppendAsRoot = true;	
					}
					Convert.AppendMeshToCollection(NewGeometryCollection, DynMesh, FTransform::Identity, Options);
				}

				FManagedArrayCollection NewCollection = FManagedArrayCollection();
				NewGeometryCollection.CopyTo(&NewCollection);

				SetValue(Context, MoveTemp(NewCollection), &Collection);

				return;
			}
		}

		SetValue(Context, FManagedArrayCollection(), &Collection);
	}
}

// -----------------------------------------------------------------------------------------------------------------------------------

FMeshToCollectionDataflowNode_v2::FMeshToCollectionDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&bSplitIslands).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bAddClusterRootForSingleMesh).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Collection);
	RegisterOutputConnection(&Materials);
}


void FMeshToCollectionDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;
	if (Out->IsA(&Collection) || Out->IsA(&Materials))
	{
		if (const TObjectPtr<const UDataflowMesh> InMesh = GetValue(Context, &Mesh))
		{
			const FDynamicMesh3& DynMesh = InMesh->GetDynamicMeshRef();

			bool bInSplitIslands = GetValue(Context, &bSplitIslands);
			bool bInAlwaysAddRoot = GetValue(Context, &bAddClusterRootForSingleMesh);

			if (DynMesh.VertexCount() > 0)
			{
				FGeometryCollection NewGeometryCollection = FGeometryCollection();
				FGeometryCollectionToDynamicMeshes Convert;
				FGeometryCollectionToDynamicMeshes::FToCollectionOptions Options;

				TArray<TUniquePtr<FDynamicMesh3>> SplitMeshes;
				if (bInSplitIslands)
				{
					FVertexConnectedComponents Components(DynMesh.MaxVertexID());
					Components.ConnectTriangles(DynMesh);
					if (bConnectIslandsByVertexOverlap)
					{
						Components.ConnectCloseVertices(DynMesh, ConnectVerticesThreshold, 2);
					}
					if (VertexToSurfaceBridgeDistance > 0)
					{
						TMeshAABBTree3<FDynamicMesh3> Spatial(&DynMesh);
						Components.ConnectVerticesToNearestDifferentComponent(Spatial, (double)VertexToSurfaceBridgeDistance);
					}
					FDynamicMeshEditor::SplitMesh(&DynMesh, SplitMeshes, false, [&Components, &DynMesh](int32 TID)
						{
							return Components.GetComponent(DynMesh.GetTriangle(TID).A);
						});
				}

				auto AddRoot = [](FGeometryCollection& ToCollection) -> int32
					{
						int32 Idx = ToCollection.AddElements(1, FGeometryCollection::TransformGroup);
						ToCollection.Parent[Idx] = INDEX_NONE;
						ToCollection.BoneColor[Idx] = FLinearColor::White;
						return Idx;
					};

				if (SplitMeshes.Num() > 1)
				{
					Options.NewMeshParentIndex = AddRoot(NewGeometryCollection);
					for (const TUniquePtr<FDynamicMesh3>& SplitMesh : SplitMeshes)
					{
						Convert.AppendMeshToCollection(NewGeometryCollection, *SplitMesh, FTransform::Identity, Options);
					}
				}
				else
				{
					if (bInAlwaysAddRoot)
					{
						Options.NewMeshParentIndex = AddRoot(NewGeometryCollection);
					}
					else
					{
						Options.bAllowAppendAsRoot = true;
					}
					Convert.AppendMeshToCollection(NewGeometryCollection, DynMesh, FTransform::Identity, Options);
				}

				FManagedArrayCollection NewCollection = FManagedArrayCollection();
				NewGeometryCollection.CopyTo(&NewCollection);

				SetValue(Context, MoveTemp(NewCollection), &Collection);

				TArray<TObjectPtr<UMaterialInterface>> OutMaterials = InMesh->GetMaterials();
				SetValue(Context, OutMaterials, &Materials);

				return;
			}
		}

		SetValue(Context, FManagedArrayCollection(), &Collection);
		SetValue(Context, TArray<TObjectPtr<UMaterialInterface>>(), &Materials);
	}
}

// -----------------------------------------------------------------------------------------------------------------------------------

void FCollectionToMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
		FDataflowTransformSelection InTransformSelection = GetValue(Context, &TransformSelection);

		//
		// If not connected select everything by default
		//
		if (!IsConnected(&TransformSelection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

			InTransformSelection.InitializeFromCollection(InCollection, true);
		}

		if (InTransformSelection.AnySelected())
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

			TArray<int32> TransformSelectionArray = InTransformSelection.AsArray();

			GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(InCollection);
			Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(InCollection);
			TArray<int32> LeafSelectionArray = TransformSelectionArray;
			SelectionFacade.ConvertSelectionToRigidNodes(LeafSelectionArray);

			UE::Geometry::FGeometryCollectionToDynamicMeshes CollectionToMeshes;
			UE::Geometry::FGeometryCollectionToDynamicMeshes::FToMeshOptions ToMeshOptions;
			ToMeshOptions.bWeldVertices = bWeldVertices;
			ToMeshOptions.bSaveIsolatedVertices = bPreserveIsolatedVertices;
			if (CollectionToMeshes.InitFromTransformSelection(InCollection, LeafSelectionArray, ToMeshOptions)
				&& !CollectionToMeshes.Meshes.IsEmpty())
			{
				NewMesh->EditMesh([this, &CollectionToMeshes](UE::Geometry::FDynamicMesh3& CombinedMesh)
					{
						CombinedMesh = MoveTemp(*CollectionToMeshes.Meshes[0].Mesh);
						for (int32 MeshIdx = 1; MeshIdx < CollectionToMeshes.Meshes.Num(); ++MeshIdx)
						{
							CombinedMesh.AppendWithOffsets(*CollectionToMeshes.Meshes[MeshIdx].Mesh);
						}
						if (bCenterPivot)
						{
							MeshTransforms::Translate(CombinedMesh, -CombinedMesh.GetBounds().Center());
						}
					}
				);
			}
		}

		SetValue(Context, NewMesh, &Mesh);
	}
}

// -----------------------------------------------------------------------------------------------------------------------------------

FCollectionToMeshDataflowNode_v2::FCollectionToMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Materials);
	RegisterInputConnection(&TransformSelection);
	RegisterOutputConnection(&Mesh);
}

void FCollectionToMeshDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
		FDataflowTransformSelection InTransformSelection = GetValue(Context, &TransformSelection);

		//
		// If not connected select everything by default
		//
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

		if (!TransformSelection.IsValidForCollection(InCollection))
		{

			InTransformSelection.InitializeFromCollection(InCollection, true);
		}

		if (InTransformSelection.AnySelected())
		{
			TArray<int32> TransformSelectionArray = InTransformSelection.AsArray();

			GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(InCollection);
			Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(InCollection);
			TArray<int32> LeafSelectionArray = TransformSelectionArray;
			SelectionFacade.ConvertSelectionToRigidNodes(LeafSelectionArray);

			UE::Geometry::FGeometryCollectionToDynamicMeshes CollectionToMeshes;
			UE::Geometry::FGeometryCollectionToDynamicMeshes::FToMeshOptions ToMeshOptions;
			ToMeshOptions.bWeldVertices = bWeldVertices;
			ToMeshOptions.bSaveIsolatedVertices = bPreserveIsolatedVertices;
			if (CollectionToMeshes.InitFromTransformSelection(InCollection, LeafSelectionArray, ToMeshOptions)
				&& !CollectionToMeshes.Meshes.IsEmpty())
			{
				NewMesh->EditMesh([this, &CollectionToMeshes](UE::Geometry::FDynamicMesh3& CombinedMesh)
					{
						CombinedMesh = MoveTemp(*CollectionToMeshes.Meshes[0].Mesh);
						for (int32 MeshIdx = 1; MeshIdx < CollectionToMeshes.Meshes.Num(); ++MeshIdx)
						{
							CombinedMesh.AppendWithOffsets(*CollectionToMeshes.Meshes[MeshIdx].Mesh);
						}
						if (bCenterPivot)
						{
							MeshTransforms::Translate(CombinedMesh, -CombinedMesh.GetBounds().Center());
						}
					}
				);
			}
		}

		UE::Geometry::FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();

		TObjectPtr<UDataflowMesh> NewDataflowMesh = NewObject<UDataflowMesh>();
		NewDataflowMesh->SetDynamicMesh(DynMesh);

		const TArray<TObjectPtr<UMaterialInterface>>& InMaterials = GetValue(Context, &Materials);

		NewDataflowMesh->SetMaterials(InMaterials);

		SetValue(Context, NewDataflowMesh, &Mesh);
	}
}

// -----------------------------------------------------------------------------------------------------------------------------------

void FStaticMeshToMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
#if WITH_EDITORONLY_DATA
	if (Out->IsA(&Mesh))
	{
		if (const UStaticMesh* const InStaticMesh = GetValue(Context, &StaticMesh))
		{
			if (const FMeshDescription* const MeshDescription = bUseHiRes ? InStaticMesh->GetHiResMeshDescription() : InStaticMesh->GetMeshDescription(LODLevel))
			{
				TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
				NewMesh->Reset();

				UE::Geometry::FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();
				{
					FMeshDescriptionToDynamicMesh ConverterToDynamicMesh;
					ConverterToDynamicMesh.Convert(MeshDescription, DynMesh);
				}

				SetValue(Context, NewMesh, &Mesh);
				return;
			}
		}
		SetValue(Context, TObjectPtr<UDynamicMesh>(NewObject<UDynamicMesh>()), &Mesh);
	}
	else if (Out->IsA(&MaterialArray))
	{
		// The dynamic mesh converter will set the MaterialIDs = PolyGroupID by default.
		// Output materials to match this.
		TArray<TObjectPtr<UMaterialInterface>> OutMaterials;
		if (const UStaticMesh* const InStaticMesh = GetValue(Context, &StaticMesh))
		{
			const TArray<FStaticMaterial>& StaticMaterials = InStaticMesh->GetStaticMaterials();
			if (const FMeshDescription* const MeshDescription = bUseHiRes ? InStaticMesh->GetHiResMeshDescription() : InStaticMesh->GetMeshDescription(LODLevel))
			{
				if (bUseHiRes)
				{
					const FStaticMeshConstAttributes MeshDescriptionAttributes(*MeshDescription);
					TPolygonGroupAttributesConstRef<FName> MaterialSlotNames = MeshDescriptionAttributes.GetPolygonGroupMaterialSlotNames();
					OutMaterials.Reserve(MaterialSlotNames.GetNumElements());
					for (int32 PolyGroupID = 0; PolyGroupID < MaterialSlotNames.GetNumElements(); ++PolyGroupID)
					{
						const int32 MaterialIndex = InStaticMesh->GetMaterialIndexFromImportedMaterialSlotName(MaterialSlotNames[PolyGroupID]);
						OutMaterials.Emplace(StaticMaterials.IsValidIndex(MaterialIndex) ? StaticMaterials[MaterialIndex].MaterialInterface : nullptr);
					}
				}
				else
				{
					const FMeshSectionInfoMap& SectionMap = InStaticMesh->GetSectionInfoMap();
					const int32 LODSectionNum = SectionMap.GetSectionNumber(LODLevel);
					for (int32 SectionIndex = 0; SectionIndex < LODSectionNum; ++SectionIndex)
					{
						const int32 MaterialIndex = SectionMap.IsValidSection(LODLevel, SectionIndex) ? SectionMap.Get(LODLevel, SectionIndex).MaterialIndex : INDEX_NONE;
						OutMaterials.Emplace(StaticMaterials.IsValidIndex(MaterialIndex) ? StaticMaterials[MaterialIndex].MaterialInterface : nullptr);
					}
				}
			}
		}
		SetValue(Context, OutMaterials, &MaterialArray);
	}
#endif
}

// -----------------------------------------------------------------------------------------------------------------------------------

FStaticMeshToMeshDataflowNode_v2::FStaticMeshToMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&StaticMesh);
	RegisterOutputConnection(&Mesh);
}

void FStaticMeshToMeshDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
#if WITH_EDITORONLY_DATA
	if (Out->IsA(&Mesh))
	{
		if (const UStaticMesh* const InStaticMesh = GetValue(Context, &StaticMesh))
		{
			if (const FMeshDescription* const MeshDescription = bUseHiRes ? InStaticMesh->GetHiResMeshDescription() : InStaticMesh->GetMeshDescription(LODLevel))
			{
				TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
				NewMesh->Reset();

				UE::Geometry::FDynamicMesh3& DynMesh = NewMesh->GetMeshRef();
				{
					FMeshDescriptionToDynamicMesh ConverterToDynamicMesh;
					ConverterToDynamicMesh.Convert(MeshDescription, DynMesh);
				}

				TObjectPtr<UDataflowMesh> NewDataflowMesh = NewObject<UDataflowMesh>();
				NewDataflowMesh->SetDynamicMesh(DynMesh);

				// Materials
				TArray<TObjectPtr<UMaterialInterface>> OutMaterials;

				const TArray<FStaticMaterial>& StaticMaterials = InStaticMesh->GetStaticMaterials();

				if (bUseHiRes)				
				{
					const FStaticMeshConstAttributes MeshDescriptionAttributes(*MeshDescription);
					TPolygonGroupAttributesConstRef<FName> MaterialSlotNames = MeshDescriptionAttributes.GetPolygonGroupMaterialSlotNames();
					OutMaterials.Reserve(MaterialSlotNames.GetNumElements());
					for (int32 PolyGroupID = 0; PolyGroupID < MaterialSlotNames.GetNumElements(); ++PolyGroupID)
					{
						const int32 MaterialIndex = InStaticMesh->GetMaterialIndexFromImportedMaterialSlotName(MaterialSlotNames[PolyGroupID]);
						OutMaterials.Emplace(StaticMaterials.IsValidIndex(MaterialIndex) ? StaticMaterials[MaterialIndex].MaterialInterface : nullptr);
					}
				}
				else
				{
					const FMeshSectionInfoMap& SectionMap = InStaticMesh->GetSectionInfoMap();
					const int32 LODSectionNum = SectionMap.GetSectionNumber(LODLevel);
					for (int32 SectionIndex = 0; SectionIndex < LODSectionNum; ++SectionIndex)
					{
						const int32 MaterialIndex = SectionMap.IsValidSection(LODLevel, SectionIndex) ? SectionMap.Get(LODLevel, SectionIndex).MaterialIndex : INDEX_NONE;
						OutMaterials.Emplace(StaticMaterials.IsValidIndex(MaterialIndex) ? StaticMaterials[MaterialIndex].MaterialInterface : nullptr);
					}
				}

				NewDataflowMesh->SetMaterials(OutMaterials);

				SetValue(Context, NewDataflowMesh, &Mesh);
				return;
			}
		}

		SetValue(Context, TObjectPtr<UDataflowMesh>(NewObject<UDataflowMesh>()), &Mesh);
	}
#endif
}

// -----------------------------------------------------------------------------------------------------------------------------------

FGetMeshBoundingBoxDataflowNode::FGetMeshBoundingBoxDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid) 
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);

	RegisterOutputConnection(&BoundingBox);
	RegisterOutputConnection(&Center);
	RegisterOutputConnection(&Dimensions);
}

void FGetMeshBoundingBoxDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&BoundingBox) || Out->IsA(&Center) || Out->IsA(&Dimensions))
	{
		FBox OutBoundingBox(ForceInit);

		if (const TObjectPtr<const UDynamicMesh> InMesh = GetValue(Context, &Mesh))
		{
			if (InMesh->GetMeshPtr())
			{
				const UE::Geometry::FAxisAlignedBox3d LocalBounds = InMesh->GetMeshPtr()->GetBounds();
				OutBoundingBox.Min = LocalBounds.Min;
				OutBoundingBox.Max = LocalBounds.Max;
			}
		}

		SetValue(Context, OutBoundingBox, &BoundingBox);
		SetValue(Context, OutBoundingBox.GetCenter(), &Center);
		SetValue(Context, OutBoundingBox.GetSize(), &Dimensions);
	}
}

// -----------------------------------------------------------------------------------------------------------------------------------

FGetMeshBoundingBoxDataflowNode_v2::FGetMeshBoundingBoxDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);

	RegisterOutputConnection(&BoundingBox);
	RegisterOutputConnection(&Center);
	RegisterOutputConnection(&Dimensions);
}

void FGetMeshBoundingBoxDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&BoundingBox) || Out->IsA(&Center) || Out->IsA(&Dimensions))
	{
		FBox OutBoundingBox(ForceInit);

		if (const TObjectPtr<const UDataflowMesh> InMesh = GetValue(Context, &Mesh))
		{
			const FDynamicMesh3& DynMesh = InMesh->GetDynamicMeshRef();

			const UE::Geometry::FAxisAlignedBox3d LocalBounds = DynMesh.GetBounds();
			
			OutBoundingBox.Min = LocalBounds.Min;
			OutBoundingBox.Max = LocalBounds.Max;
		}

		SetValue(Context, OutBoundingBox, &BoundingBox);
		SetValue(Context, OutBoundingBox.GetCenter(), &Center);
		SetValue(Context, OutBoundingBox.GetSize(), &Dimensions);
	}
}

// -----------------------------------------------------------------------------------------------------------------------------------

void FMeshAppendDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TObjectPtr<UDynamicMesh>>(&Mesh))
	{
		if (TObjectPtr<UDynamicMesh> InMesh1 = GetValue<TObjectPtr<UDynamicMesh>>(Context, &Mesh1))
		{
			if (TObjectPtr<UDynamicMesh> InMesh2 = GetValue<TObjectPtr<UDynamicMesh>>(Context, &Mesh2))
			{
				const UE::Geometry::FDynamicMesh3& DynMesh1 = InMesh1->GetMeshRef();
				const UE::Geometry::FDynamicMesh3& DynMesh2 = InMesh2->GetMeshRef();

				if (DynMesh1.VertexCount() > 0 || DynMesh2.VertexCount() > 0)
				{
					TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
					NewMesh->Reset();

					UE::Geometry::FDynamicMesh3& ResultDynMesh = NewMesh->GetMeshRef();

					UE::Geometry::FDynamicMeshEditor MeshEditor(&ResultDynMesh);

					UE::Geometry::FMeshIndexMappings IndexMaps1;
					MeshEditor.AppendMesh(&DynMesh1, IndexMaps1);

					UE::Geometry::FMeshIndexMappings IndexMaps2;
					MeshEditor.AppendMesh(&DynMesh2, IndexMaps2);

					SetValue(Context, NewMesh, &Mesh);

					return;
				}
			}
		}

		SetValue(Context, TObjectPtr<UDynamicMesh>(NewObject<UDynamicMesh>()), &Mesh);
	}
}

FDataflowMeshAppendDataflowNode::FDataflowMeshAppendDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterOutputConnection(&Mesh, &Mesh);
	RegisterInputConnection(&AppendMesh);
}

void FDataflowMeshAppendDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDataflowMesh> NewMesh = NewObject<UDataflowMesh>();

		if (const UDataflowMesh* const DataflowMesh1 = GetValue(Context, &Mesh))
		{
			if (const UDataflowMesh* const DataflowMesh2 = GetValue(Context, &AppendMesh))
			{
				if (const UE::Geometry::FDynamicMesh3* const DynamicMesh1 = DataflowMesh1->GetDynamicMesh())
				{
					if (const UE::Geometry::FDynamicMesh3* const DynamicMesh2 = DataflowMesh2->GetDynamicMesh())
					{
						if (DynamicMesh1->VertexCount() > 0 && DynamicMesh2->VertexCount() > 0)
						{
							UE::Geometry::FDynamicMesh3 ResultDynamicMesh;
							UE::Geometry::FDynamicMeshEditor MeshEditor(&ResultDynamicMesh);
							ResultDynamicMesh.EnableAttributes();
							ResultDynamicMesh.Attributes()->EnableMaterialID();

							UE::Geometry::FMeshIndexMappings IndexMaps1;
							MeshEditor.AppendMesh(DynamicMesh1, IndexMaps1);

							UE::Geometry::FMeshIndexMappings IndexMaps2;
							MeshEditor.AppendMesh(DynamicMesh2, IndexMaps2);

							// Reindex material IDs
							if (DynamicMesh1->HasAttributes() && DynamicMesh1->Attributes()->HasMaterialID() && DynamicMesh2->HasAttributes() && DynamicMesh2->Attributes()->HasMaterialID())
							{
								const int32 MaterialIDOffset = DataflowMesh1->GetMaterials().Num();

								for (const int32 Mesh2TriangleIndex : DynamicMesh2->TriangleIndicesItr())
								{
									int32 InputMaterialID;
									DynamicMesh2->Attributes()->GetMaterialID()->GetValue(Mesh2TriangleIndex, &InputMaterialID);

									const int32 NewTriangleIndex = IndexMaps2.GetNewTriangle(Mesh2TriangleIndex);
									ResultDynamicMesh.Attributes()->GetMaterialID()->SetValue(NewTriangleIndex, MaterialIDOffset + InputMaterialID);
								}
							}

							NewMesh->SetDynamicMesh(MoveTemp(ResultDynamicMesh));
						}
						else if (DynamicMesh1->VertexCount() > 0)
						{
							NewMesh->SetDynamicMesh(*DynamicMesh1);
						}
						else if (DynamicMesh2->VertexCount() > 0)
						{
							NewMesh->SetDynamicMesh(*DynamicMesh2);
						}
					}
				}	// end if DynamicMesh1

				// Materials
				NewMesh->AddMaterials(DataflowMesh1->GetMaterials());
				NewMesh->AddMaterials(DataflowMesh2->GetMaterials());
			}
		}

		SetValue(Context, NewMesh, &Mesh);
	}
}

FMakeDataflowMeshDataflowNode::FMakeDataflowMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&InMesh);
	RegisterInputConnection(&InMaterials);
	RegisterOutputConnection(&Mesh);
}

void FMakeDataflowMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDataflowMesh> NewMesh = NewObject<UDataflowMesh>();
		
		if (UDynamicMesh* const InUDynamicMesh = GetValue(Context, &InMesh))
		{
			InUDynamicMesh->ProcessMesh([NewMesh](const UE::Geometry::FDynamicMesh3& InFDynamicMesh)
			{
				NewMesh->SetDynamicMesh(InFDynamicMesh);
			});
		}

		TArray<TObjectPtr<UMaterialInterface>> MaterialArray = GetValue(Context, &InMaterials);
		NewMesh->SetMaterials(MoveTemp(MaterialArray));

		SetValue(Context, NewMesh, &Mesh);
	}
}

FSplitMeshIslandsDataflowNode::FSplitMeshIslandsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterOutputConnection(&Meshes);
}

void FSplitMeshIslandsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Meshes))
	{
		TArray<TObjectPtr<UDynamicMesh>> OutMeshes;
		if (const TObjectPtr<UDynamicMesh> InMesh = GetValue(Context, &Mesh))
		{
			if (SplitMethod == EDataflowMeshSplitIslandsMethod::NoSplit)
			{
				OutMeshes.Add(InMesh);
			}
			else
			{
				InMesh->ProcessMesh([&OutMeshes, this](const FDynamicMesh3& ToSplit)
				{
					TArray<TUniquePtr<FDynamicMesh3>> SplitMeshes;
					FVertexConnectedComponents Components(ToSplit.MaxVertexID());
					Components.ConnectTriangles(ToSplit);
					if (SplitMethod == EDataflowMeshSplitIslandsMethod::ByVertexOverlap)
					{
						Components.ConnectCloseVertices(ToSplit, ConnectVerticesThreshold, 2);
					}
					FDynamicMeshEditor::SplitMesh(&ToSplit, SplitMeshes, true, [&Components, &ToSplit](int32 TID)
					{
						return Components.GetComponent(ToSplit.GetTriangle(TID).A);
					});
					OutMeshes.SetNum(SplitMeshes.Num());
					for (int32 Idx = 0; Idx < SplitMeshes.Num(); ++Idx)
					{
						OutMeshes[Idx] = NewObject<UDynamicMesh>();
						OutMeshes[Idx]->SetMesh(MoveTemp(*SplitMeshes[Idx]));
					}
				});
			}
		}
		SetValue(Context, OutMeshes, &Meshes);
	}
}


FSplitDataflowMeshDataflowNode::FSplitDataflowMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&InMesh);
	RegisterOutputConnection(&Mesh);
	RegisterOutputConnection(&MaterialArray);
}

/////////////////////////////////////////////////////////////////////////////////////////////////

void FSplitDataflowMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	if (Out->IsA(&Mesh))
	{
		if (const UDataflowMesh* const InDataflowMesh = GetValue(Context, &InMesh))
		{
			NewMesh->SetMesh(InDataflowMesh->GetDynamicMeshRef());
		}
		SetValue(Context, NewMesh, &Mesh);
	}
	else if (Out->IsA(&MaterialArray))
	{
		if (UDataflowMesh* const InDataflowMesh = GetValue(Context, &InMesh))
		{
			Materials = InDataflowMesh->GetMaterials();
		}
		SetValue(Context, Materials, &MaterialArray);
	}
}

void FSplitDataflowMeshDataflowNode::OnRenderOutput(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup, const TArray<UPrimitiveComponent*>& RenderComponents) const
{
	if (RenderGroup == TEXT("Surface") && OutputName == TEXT("Mesh"))
	{
		if (UDataflowMesh* const InDataflowMesh = GetValue(Context, &InMesh))
		{
			const TArray<TObjectPtr<UMaterialInterface>>& Materials = InDataflowMesh->GetMaterials();
			for (UPrimitiveComponent* Component : RenderComponents)
			{
				if (Component)
				{
					for (int32 MatIndex = 0; MatIndex < Materials.Num(); ++MatIndex)
					{
						Component->SetMaterial(MatIndex, Materials[MatIndex]);
					}
				}
			}
		}
	}
}

/////////////////////////////////////////////////////////////////////////////////////////////////

FDuplicateMeshUVChannelNode::FDuplicateMeshUVChannelNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterOutputConnection(&Mesh, &Mesh);
	RegisterOutputConnection(&NewUVChannel);
}

void FDuplicateMeshUVChannelNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	int32 NewUVLayerIndex = -1;

	if (Out->IsA(&Mesh) || Out->IsA(&NewUVChannel))
	{
		if (TObjectPtr<UDataflowMesh> InMesh = GetValue(Context, &Mesh))
		{
			if (const FDynamicMesh3* const InDynamicMesh = InMesh->GetDynamicMesh())
			{
				if (InDynamicMesh->HasAttributes() && SourceUVChannel >= 0 && SourceUVChannel < InDynamicMesh->Attributes()->NumUVLayers())
				{
					UE::Geometry::FDynamicMesh3 OutDynamicMesh;
					OutDynamicMesh.Copy(*InDynamicMesh);
					OutDynamicMesh.EnableAttributes();

					NewUVLayerIndex = OutDynamicMesh.Attributes()->NumUVLayers();
					OutDynamicMesh.Attributes()->SetNumUVLayers(NewUVLayerIndex + 1);

					const UE::Geometry::FDynamicMeshUVOverlay* const SourceUVLayer = OutDynamicMesh.Attributes()->GetUVLayer(SourceUVChannel);
					OutDynamicMesh.Attributes()->GetUVLayer(NewUVLayerIndex)->Copy(*SourceUVLayer);

					TObjectPtr<UDataflowMesh> OutMesh = NewObject<UDataflowMesh>();
					OutMesh->SetDynamicMesh(MoveTemp(OutDynamicMesh));
					OutMesh->SetMaterials(InMesh->GetMaterials());
					SetValue(Context, OutMesh, &Mesh);
					SetValue(Context, NewUVLayerIndex, &NewUVChannel);
					return;
				}
				else
				{
					Context.Warning(TEXT("Invalid Source UV Channel or the Mesh does not have an AttributeSet"), this, Out);
				}
			}
			else
			{
				Context.Warning(TEXT("Mesh is missing DynamicMesh object"), this, Out);
			}
		}
	}

	SafeForwardInput(Context, &Mesh, &Mesh);
	SetValue(Context, NewUVLayerIndex, &NewUVChannel);
}


void FMeshCopyToPointsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Mesh))
	{
		if (TObjectPtr<UDynamicMesh> InMeshToCopy = GetValue(Context, &MeshToCopy))
		{
			TObjectPtr<UDynamicMesh> NewMesh = nullptr;

			const UE::Geometry::FDynamicMesh3& InDynMeshToCopy = InMeshToCopy->GetMeshRef();

			const TArray<FVector>& InPoints = GetValue(Context, &Points);

			if (InPoints.Num() > 0 && InDynMeshToCopy.VertexCount() > 0)
			{
				NewMesh = NewObject<UDynamicMesh>();
				NewMesh->Reset();

				UE::Geometry::FDynamicMeshEditor MeshEditor(&NewMesh->GetMeshRef());

				for (const FVector& Point : InPoints)
				{
					UE::Geometry::FDynamicMesh3 DynMeshTemp(InDynMeshToCopy);
					UE::Geometry::FRefCountVector VertexRefCounts = DynMeshTemp.GetVerticesRefCounts();

					UE::Geometry::FRefCountVector::IndexIterator ItVertexID = VertexRefCounts.BeginIndices();
					const UE::Geometry::FRefCountVector::IndexIterator ItEndVertexID = VertexRefCounts.EndIndices();

					while (ItVertexID != ItEndVertexID)
					{
						DynMeshTemp.SetVertex(*ItVertexID, Scale * DynMeshTemp.GetVertex(*ItVertexID) + Point);
						++ItVertexID;
					}

					UE::Geometry::FMeshIndexMappings IndexMaps;
					MeshEditor.AppendMesh(&DynMeshTemp, IndexMaps);
				}
			}

			SetValue(Context, NewMesh, &Mesh);
			return;
		}

		SetValue(Context, TObjectPtr<UDynamicMesh>(NewObject<UDynamicMesh>()), &Mesh);
	}
	else if (Out->IsA(&Meshes))
	{
		TArray<TObjectPtr<UDynamicMesh>> OutMeshes;
		if (TObjectPtr<UDynamicMesh> InMeshToCopy = GetValue(Context, &MeshToCopy))
		{
			const UE::Geometry::FDynamicMesh3& InDynMeshToCopy = InMeshToCopy->GetMeshRef();

			const TArray<FVector>& InPoints = GetValue(Context, &Points);

			if (InPoints.Num() > 0 && InDynMeshToCopy.VertexCount() > 0)
			{
				for (const FVector& Point : InPoints)
				{
					TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
					NewMesh->Reset();
					OutMeshes.Add(NewMesh);

					UE::Geometry::FDynamicMeshEditor MeshEditor(&NewMesh->GetMeshRef());

					UE::Geometry::FDynamicMesh3 DynMeshTemp(InDynMeshToCopy);
					UE::Geometry::FRefCountVector VertexRefCounts = DynMeshTemp.GetVerticesRefCounts();

					UE::Geometry::FRefCountVector::IndexIterator ItVertexID = VertexRefCounts.BeginIndices();
					const UE::Geometry::FRefCountVector::IndexIterator ItEndVertexID = VertexRefCounts.EndIndices();

					while (ItVertexID != ItEndVertexID)
					{
						DynMeshTemp.SetVertex(*ItVertexID, Scale * DynMeshTemp.GetVertex(*ItVertexID) + Point);
						++ItVertexID;
					}

					UE::Geometry::FMeshIndexMappings IndexMaps;
					MeshEditor.AppendMesh(&DynMeshTemp, IndexMaps);				
				}
			}
		}

		SetValue(Context, MoveTemp(OutMeshes), &Meshes);
		return;
	}
}


void FGetMeshDataDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<int32>(&VertexCount))
	{
		if (const TObjectPtr<const UDynamicMesh> InMesh = GetValue<TObjectPtr<UDynamicMesh>>(Context, &Mesh))
		{
			SetValue(Context, InMesh->GetMeshRef().VertexCount(), &VertexCount);
		}
		else
		{
			SetValue(Context, 0, &VertexCount);
		}
	}
	else if (Out->IsA<int32>(&EdgeCount))
	{
		if (const TObjectPtr<const UDynamicMesh> InMesh = GetValue<TObjectPtr<UDynamicMesh>>(Context, &Mesh))
		{
			SetValue(Context, InMesh->GetMeshRef().EdgeCount(), &EdgeCount);
		}
		else
		{
			SetValue(Context, 0, &EdgeCount);
		}
	}
	else if (Out->IsA<int32>(&TriangleCount))
	{
		if (const TObjectPtr<const UDynamicMesh> InMesh = GetValue<TObjectPtr<UDynamicMesh>>(Context, &Mesh))
		{
			SetValue(Context, InMesh->GetMeshRef().TriangleCount(), &TriangleCount);
		}
		else
		{
			SetValue(Context, 0, &TriangleCount);
		}
	}
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FMeshProcessorDataflowNodeBase::PostSerialize(const FArchive& Ar)
{
	Super::PostSerialize(Ar);
	if (Ar.IsLoading())
	{
		TeardownBlueprintEvent();
		SetupBlueprintEvent();
	}
}

void FMeshProcessorDataflowNodeBase::OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet 
		&& PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FMeshProcessorDataflowNodeBase, MeshProcessor))
	{
		TeardownBlueprintEvent();
		if (MeshProcessor)
		{
			MeshProcessorInstance = NewObject<UDynamicMeshProcessorBlueprint>(OwningObject, MeshProcessor, NAME_None, RF_Transactional);
			SetupBlueprintEvent();
		}
		else
		{
			MeshProcessorInstance = nullptr;
			RefreshConnectionsFromBlueprint();
		}
	}
}

void FMeshProcessorDataflowNodeBase::SetupBlueprintEvent()
{
#if WITH_EDITOR
	if (MeshProcessor)
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(MeshProcessor->ClassGeneratedBy))
		{
			if (!ensure(!BlueprintChangeDelegateHandle.IsValid()))
			{
				TeardownBlueprintEvent();
			}
			BlueprintChangeDelegateHandle = Blueprint->OnChanged().AddLambda([this](UBlueprint* BP)
			{
				RefreshConnectionsFromBlueprint();
				Invalidate();
			});
		}
	}
	RefreshConnectionsFromBlueprint();
#endif
}

void FMeshProcessorDataflowNodeBase::TeardownBlueprintEvent()
{
#if WITH_EDITOR
	if (MeshProcessor && BlueprintChangeDelegateHandle.IsValid())
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(MeshProcessor->ClassGeneratedBy))
		{
			Blueprint->OnChanged().Remove(BlueprintChangeDelegateHandle);
			BlueprintChangeDelegateHandle.Reset();
		}
	}
#endif
}

void FMeshProcessorDataflowNodeBase::RefreshConnectionsFromBlueprint()
{
#if WITH_EDITOR
	PropertyBag.Reset();
	CachedPropertyNameMap.Reset();
	if (MeshProcessor)
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(MeshProcessor->ClassGeneratedBy))
		{
			if (Blueprint->GeneratedClass != nullptr)
			{
				TSet<FName> AddedSanitizedNames;
				for (TFieldIterator<FProperty> PropertyIt(Blueprint->GeneratedClass, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
				{
					FProperty* Property = *PropertyIt;
					if (!Property->HasAnyPropertyFlags(CPF_Parm) && Property->HasAllPropertyFlags(CPF_BlueprintVisible))
					{
						FName SanitizedName = FInstancedPropertyBag::SanitizePropertyName(Property->GetFName());
						bool bRenumbered = false;
						while (AddedSanitizedNames.Contains(SanitizedName))
						{
							SanitizedName.SetNumber(SanitizedName.GetNumber() + 1);
							bRenumbered = true;
						}
						if (bRenumbered)
						{
							UE_LOGF(LogGeometryCollectionMeshNodes, Warning, "Geometry Script node has multiple inputs that map to the same sanitized name; renaming input %ls to %ls", *Property->GetFName().ToString(), *SanitizedName.ToString());
						}
						AddedSanitizedNames.Add(SanitizedName);
						CachedPropertyNameMap.Add(SanitizedName, Property->GetFName());
						PropertyBag.AddProperty(SanitizedName, Property, false);
					}
				}
			}
		}
	}
	// Make sure the node is up to date with the serialized data
	DynamicConnections.Refresh();
#endif
}

void FMeshProcessorDataflowNodeBase::ApplyParametersToBlueprintInstance(UE::Dataflow::FContext& Context) const
{
#if WITH_EDITOR
	if (UBlueprint* Blueprint = Cast<UBlueprint>(MeshProcessor->ClassGeneratedBy))
	{
		if (Blueprint->GeneratedClass != nullptr && MeshProcessorInstance && PropertyBag.GetPropertyBagStruct())
		{
			TConstArrayView<FPropertyBagPropertyDesc> Descs = PropertyBag.GetPropertyBagStruct()->GetPropertyDescs();
			for (const FPropertyBagPropertyDesc& Desc : Descs)
			{
				const FName* BPMappedName = CachedPropertyNameMap.Find(Desc.Name);
				if (!ensureMsgf(BPMappedName, TEXT("Property %s not found in cached name map, so could not be applied to BP"), *Desc.Name.ToString()))
				{
					continue;
				}
				if (FProperty* BPProperty = Blueprint->GeneratedClass->FindPropertyByName(*BPMappedName))
				{
					if (BPProperty->GetClass() == Desc.CachedProperty->GetClass())
					{
						if (const FDataflowInput* Input = FindInput(Desc.Name))
						{
							// only send the data is the input is connected, other wise keep what was there before 
							if (const FDataflowOutput* Output = Input->GetConnection())
							{
								Input->PullValue(Context);
								if (Desc.ValueType == EPropertyBagPropertyType::Double)
								{
									// Special case : the internal storage type for Float BP variable is Double
									// so we need to pass a pointer on a double but reading a float from Dataflow (Same applies for array type)
									if (Desc.ContainerTypes.GetFirstContainerType() == EPropertyBagContainerType::Array)
									{
										const TArray<float> CacheValue = Context.GetData<TArray<float>>(Output->CacheKey(), Output->GetProperty(), {});
										TArray<double> DoubleArrayValue;
										Algo::Copy(CacheValue, DoubleArrayValue);
										void* TargetAddress = BPProperty->ContainerPtrToValuePtr<void>(MeshProcessorInstance);
										BPProperty->CopyCompleteValue(TargetAddress, &DoubleArrayValue);
									}
									else
									{
										const float CachedValue = Context.GetData<float>(Output->CacheKey(), Output->GetProperty(), 0.0f);
										const double DoubleValue{ CachedValue };
										void* TargetAddress = BPProperty->ContainerPtrToValuePtr<void>(MeshProcessorInstance);
										BPProperty->CopyCompleteValue(TargetAddress, &DoubleValue);
									}
								}
								else if (Desc.ValueType == EPropertyBagPropertyType::Object)
								{
									// Special case: BP uses raw UObject* while dataflow cache uses TObjectPtr<UObject> (same applies for array)
									if (Desc.ContainerTypes.GetFirstContainerType() == EPropertyBagContainerType::Array)
									{
										const TArray<TObjectPtr<UObject>> CacheValue = Context.GetData<TArray<TObjectPtr<UObject>>>(Output->CacheKey(), Output->GetProperty(), {});
										TArray<UObject*> RawPtrArray;
										Algo::Copy(CacheValue, RawPtrArray);
										void* TargetAddress = BPProperty->ContainerPtrToValuePtr<TArray<UObject*>>(MeshProcessorInstance);
										BPProperty->CopyCompleteValue(TargetAddress, &RawPtrArray);
									}
									else
									{
										const TObjectPtr<UObject> CachedValue = Context.GetData<TObjectPtr<UObject>>(Output->CacheKey(), Output->GetProperty(), nullptr);
										const UObject* RawPtrValue = { CachedValue.Get() };
										void* TargetAddress = BPProperty->ContainerPtrToValuePtr<UObject*>(MeshProcessorInstance);
										BPProperty->CopyCompleteValue(TargetAddress, &RawPtrValue);
									}
								}
								else if (const void* CachedValueAddress = Context.GetUntypedData(Output->CacheKey(), nullptr))
								{
									void* TargetAddress = BPProperty->ContainerPtrToValuePtr<void>(MeshProcessorInstance);
									BPProperty->CopyCompleteValue(TargetAddress, CachedValueAddress);
								}
							}
						}
					}
				}
			}
		}
	}
#else
	UE_LOGF(LogGeometryCollectionMeshNodes, Error, "FMeshProcessorDataflowNodeBase - trying to execute a mesh processing node ( running a Geometry script ) in a non-editor build - connected input parameters won't be passed to the script");
#endif
}

#if WITH_EDITOR
void FMeshProcessorDataflowNodeBase::OnDoubleClicked(UE::Dataflow::FContext* Context) const
{
	if (!MeshProcessor)
	{
		return;
	}
	if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(MeshProcessor->ClassGeneratedBy))
		{
			AssetEditorSubsystem->OpenEditorForAssets({ Blueprint });
		}
	}
}
#endif


///////////////////////////////////////////////////////////////////////////////////////////////////////////

void FApplyMeshProcessorToMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TObjectPtr<UDynamicMesh>>(&Mesh))
	{
		if (TObjectPtr<const UDynamicMesh> InMesh = GetValue<TObjectPtr<UDynamicMesh>>(Context, &Mesh))
		{
			if (!MeshProcessorInstance)
			{
				SafeForwardInput(Context, &Mesh, &Mesh);
				return;
			}

			// Creating a new mesh object from InMesh
			TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
			NewMesh->SetMesh(InMesh->GetMeshRef());

			ApplyParametersToBlueprintInstance(Context);

			bool bFailed = false;
			MeshProcessorInstance->ProcessDynamicMesh(NewMesh, bFailed);

			SetValue(Context, NewMesh, &Mesh);
		}
		else
		{
			SetValue(Context, TObjectPtr<UDynamicMesh>(NewObject<UDynamicMesh>()), &Mesh);
		}
	}
}


void FApplyMeshProcessorToGeometryCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection) ||
		Out->IsA(&TransformSelection))
	{
		if (!MeshProcessorInstance)
		{
			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		FDataflowTransformSelection InTransformSelection = GetValue(Context, &TransformSelection);

		//
		// If not connected select everything by default
		//
		if (!IsConnected(&TransformSelection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

			GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
			const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectAll();

			FDataflowTransformSelection NewTransformSelection;
			NewTransformSelection.InitializeFromCollection(InCollection, false);
			NewTransformSelection.SetFromArray(SelectionArr);

			InTransformSelection = NewTransformSelection;
		}

		if (InTransformSelection.AnySelected())
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

			UE::Geometry::FGeometryCollectionToDynamicMeshes CollectionToMeshes;
			UE::Geometry::FGeometryCollectionToDynamicMeshes::FToMeshOptions ToMeshOptions;
			ToMeshOptions.bWeldVertices = bWeldVertices;
			ToMeshOptions.bSaveIsolatedVertices = bPreserveIsolatedVertices;
			if (CollectionToMeshes.InitFromTransformSelection(InCollection, InTransformSelection.AsArray(), ToMeshOptions)
				&& !CollectionToMeshes.Meshes.IsEmpty())
			{
				// Temporarily create a UDynamicMesh as a container to hold the meshes we pass to BP
				TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();

				bool bAnySuccess = false;
				for (UE::Geometry::FGeometryCollectionToDynamicMeshes::FMeshInfo& MeshInfo : CollectionToMeshes.Meshes)
				{
					NewMesh->SetMesh(MoveTemp(*MeshInfo.Mesh));

					ApplyParametersToBlueprintInstance(Context);

					bool bFailed = false;
					MeshProcessorInstance->ProcessDynamicMesh(NewMesh, bFailed);
					if (!bFailed) // on success, move the mesh back
					{
						bAnySuccess = true;
						NewMesh->EditMesh([&MeshInfo](UE::Geometry::FDynamicMesh3& Mesh)
						{
							*MeshInfo.Mesh = MoveTemp(Mesh);
						}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, true);
					}
					else // on failure, clear the mesh so it won't be written back
					{
						MeshInfo.TransformIndex = -1;
						MeshInfo.Mesh = nullptr;
					}
				}

				if (bAnySuccess)
				{
					if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
					{
						UE::Geometry::FGeometryCollectionToDynamicMeshes::FToCollectionOptions ToCollectionOptions;
						ToCollectionOptions.bDefaultFaceInternal = false;
						ToCollectionOptions.bDefaultFaceVisible = true;
						CollectionToMeshes.UpdateGeometryCollection(*GeomCollection, ToCollectionOptions);
						SetValue<const FManagedArrayCollection&>(Context, *GeomCollection, &Collection);
						return;
					}
				}
			}
		}

		SafeForwardInput(Context, &Collection, &Collection);
	}
}

FAppendMeshesToCollectionDataflowNode::FAppendMeshesToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Meshes);
	RegisterInputConnection(&ParentIndex);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&AddedSelection);
}

void FAppendMeshesToCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection) ||
		Out->IsA<FDataflowTransformSelection>(&AddedSelection))
	{
		if (!IsConnected(&Collection))
		{
			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		int32 UseParentIndex = GetValue(Context, &ParentIndex);
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const TArray<TObjectPtr<UDynamicMesh>> InMeshes = GetValue(Context, &Meshes);

		FDataflowTransformSelection NewSelection;

		if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
		{
			bool bModifiedCollection = false;
			int32 FirstNewTransformIndex = INDEX_NONE;
			for (const TObjectPtr<UDynamicMesh>& MeshObject : InMeshes)
			{
				if (MeshObject)
				{
					MeshObject->ProcessMesh([&GeomCollection, &bModifiedCollection, UseParentIndex, &FirstNewTransformIndex](const FDynamicMesh3& Mesh)
					{
						UE::Geometry::FGeometryCollectionToDynamicMeshes::FToCollectionOptions Options;
						Options.NewMeshParentIndex = UseParentIndex;
						int32 AddedIdx = UE::Geometry::FGeometryCollectionToDynamicMeshes::AppendMeshToCollection(*GeomCollection, Mesh, FTransform::Identity, Options);
						if (AddedIdx != INDEX_NONE)
						{
							if (!bModifiedCollection)
							{
								FirstNewTransformIndex = AddedIdx;
							}
							bModifiedCollection = true;
						}
					});
				}
			}
			if (bModifiedCollection)
			{
				NewSelection.InitializeFromCollection(*GeomCollection, false);
				for (int32 Idx = FirstNewTransformIndex; Idx < NewSelection.Num(); ++Idx)
				{
					NewSelection.SetSelected(Idx);
				}
				SetValue<const FManagedArrayCollection&>(Context, MoveTemp(*GeomCollection), &Collection);
				SetValue(Context, NewSelection, &AddedSelection);
				return;
			}
		}

		SafeForwardInput(Context, &Collection, &Collection);
		SetValue(Context, NewSelection, &AddedSelection);
	}
}

FCollectionSelectionToMeshesDataflowNode::FCollectionSelectionToMeshesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&TransformSelection);
	RegisterOutputConnection(&Meshes);
}

void FCollectionSelectionToMeshesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Meshes))
	{
		TArray<TObjectPtr<UDynamicMesh>> NewMeshes;
		FDataflowTransformSelection InTransformSelection = GetValue(Context, &TransformSelection);

		//
		// If not connected select everything by default
		//
		if (!IsConnected(&TransformSelection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

			InTransformSelection.InitializeFromCollection(InCollection, true);
		}

		if (InTransformSelection.AnySelected())
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

			TArray<int32> TransformSelectionArray = InTransformSelection.AsArray();

			GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(InCollection);
			Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(InCollection);
			TArray<int32> LeafSelectionArray = TransformSelectionArray;
			SelectionFacade.ConvertSelectionToRigidNodes(LeafSelectionArray);

			UE::Geometry::FGeometryCollectionToDynamicMeshes CollectionToMeshes;
			UE::Geometry::FGeometryCollectionToDynamicMeshes::FToMeshOptions ToMeshOptions;
			ToMeshOptions.bWeldVertices = bWeldVertices;
			ToMeshOptions.bSaveIsolatedVertices = bPreserveIsolatedVertices;
			if (CollectionToMeshes.InitFromTransformSelection(InCollection, LeafSelectionArray, ToMeshOptions)
				&& !CollectionToMeshes.Meshes.IsEmpty())
			{
				NewMeshes.Reserve(bConvertSelectionToLeaves ? CollectionToMeshes.Meshes.Num() : TransformSelectionArray.Num());
				using FMeshInfo = UE::Geometry::FGeometryCollectionToDynamicMeshes::FMeshInfo;

				if (bConvertSelectionToLeaves)
				{
					for (FMeshInfo& MeshInfo : CollectionToMeshes.Meshes)
					{
						TObjectPtr<UDynamicMesh>& NewMesh = NewMeshes.Add_GetRef(NewObject<UDynamicMesh>());
						NewMesh->SetMesh(MoveTemp(*MeshInfo.Mesh));
					}
				}
				else
				{
					TMap<int32, FMeshInfo*> BoneToMeshInfo;
					TMap<int32, const FDynamicMesh3*> BoneToMesh;
					for (FMeshInfo& MeshInfo : CollectionToMeshes.Meshes)
					{
						BoneToMeshInfo.Add(MeshInfo.TransformIndex, &MeshInfo);
					}
					for (int32 BoneIdx : TransformSelectionArray)
					{
						if (BoneToMeshInfo.Contains(BoneIdx))
						{
							TObjectPtr<UDynamicMesh>& NewMesh = NewMeshes.Add_GetRef(NewObject<UDynamicMesh>());
							// move the mesh out of the collection and add the pointer to the BoneToMesh map instead
							// (in case we also have to make a cluster node using the same mesh)
							NewMesh->SetMesh(MoveTemp(*BoneToMeshInfo[BoneIdx]->Mesh));
							BoneToMeshInfo.Remove(BoneIdx);
							BoneToMesh.Add(BoneIdx, NewMesh->GetMeshPtr());
						}
						else
						{
							TObjectPtr<UDynamicMesh>& NewMesh = NewMeshes.Add_GetRef(NewObject<UDynamicMesh>());
							FDynamicMesh3& Mesh = NewMesh->GetMeshRef();
							UE::Geometry::FDynamicMeshEditor Editor(&Mesh);
							TArray<int32> SearchBones;
							SearchBones.Add(BoneIdx);
							while (!SearchBones.IsEmpty())
							{
								int32 SearchBoneIdx = SearchBones.Pop(EAllowShrinking::No);
								const FDynamicMesh3* FoundMesh = nullptr;
								if (BoneToMeshInfo.Contains(SearchBoneIdx))
								{
									FoundMesh = BoneToMeshInfo[SearchBoneIdx]->Mesh.Get();
								}
								else if (BoneToMesh.Contains(SearchBoneIdx))
								{
									FoundMesh = BoneToMesh[SearchBoneIdx];
								}

								if (FoundMesh)
								{
									Mesh.EnableMatchingAttributes(*FoundMesh);
									UE::Geometry::FMeshIndexMappings Unused;
									Editor.AppendMesh(FoundMesh, Unused);
								}
								else
								{
									// No mesh for this bone; search the children for meshes
									const TSet<int32>* Children = HierarchyFacade.FindChildren(SearchBoneIdx);
									if (Children)
									{
										SearchBones.Append(Children->Array());
									}
								}
							}
							// add the built mesh to the map, in case we want to build a parent of it later
							BoneToMesh.Add(BoneIdx, &Mesh);
						}
					}
				}
			}
		}

		SetValue(Context, NewMeshes, &Meshes);
	}
}

// -----------------------------------------------------------------------------------------------------------

FGetMeshBoundingSphereDataflowNode::FGetMeshBoundingSphereDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);

	RegisterOutputConnection(&BoundingSphere);
	RegisterOutputConnection(&Center);
	RegisterOutputConnection(&Radius);
}

void FGetMeshBoundingSphereDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&BoundingSphere) || Out->IsA(&Center) || Out->IsA(&Radius))
	{
		FVector OutSphereCenter = FVector::ZeroVector;
		float OutSphereRadius = 0.f;

		if (const TObjectPtr<const UDataflowMesh> InMesh = GetValue(Context, &Mesh))
		{
			const FDynamicMesh3& DynMesh = InMesh->GetDynamicMeshRef();

			TArray<FVector> Vertices;
			Vertices.Reserve(DynMesh.VertexCount());

			for (FVector Vertex : DynMesh.VerticesItr())
			{
				Vertices.Add(Vertex);
			}

			FSphere OutBoundingSphere(&Vertices[0], Vertices.Num());

			SetValue(Context, OutBoundingSphere, &BoundingSphere);
			SetValue(Context, OutBoundingSphere.Center, &Center);
			SetValue(Context, float(OutBoundingSphere.W), &Radius);

			return;
		}

		SetValue(Context, FSphere(), &BoundingSphere);
		SetValue(Context, OutSphereCenter, &Center);
		SetValue(Context, OutSphereRadius, &Radius);
	}
}

// -----------------------------------------------------------------------------------------------------------

