// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionNodes.h"
#include "Dataflow/DataflowCore.h"

#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionEngineUtility.h"
#include "GeometryCollection/GeometryCollectionEngineRemoval.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealTypePrivate.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "StaticMeshAttributes.h"
#include "DynamicMeshEditor.h"
#include "Operations/MeshBoolean.h"
#include "Materials/Material.h"

#include "EngineGlobals.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "Voronoi/Voronoi.h"
#include "PlanarCut.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "FractureEngineClustering.h"
#include "FractureEngineSelection.h"
#include "GeometryCollection/Facades/CollectionBoundsFacade.h"
#include "GeometryCollection/Facades/CollectionAnchoringFacade.h"
#include "GeometryCollection/Facades/CollectionRemoveOnBreakFacade.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowDebugDraw.h"
#if WITH_EDITOR
#include "Dataflow/DataflowRenderingViewMode.h"
#endif
#include "Dataflow/GeometryCollectionUtils.h"
#include "Dataflow/DataflowDebugDrawInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionNodes)

namespace UE::Dataflow
{
	void GeometryCollectionEngineNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAppendCollectionAssetsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FPrintStringDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FLogStringDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBoundingBoxRatioDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBoundingBoxAverageExtentsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FExpandBoundingBoxDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetBoxLengthsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FExpandVectorDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FStringAppendDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FHashStringDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FHashVectorDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetBoundingBoxesFromCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetRootIndexFromCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetCentroidsFromCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FTransformCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBakeTransformsInCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FTransformMeshDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCompareIntDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCompareFloatDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBooleanOperationDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBranchMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBranchCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetSchemaDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRemoveOnBreakDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetAnchorStateDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetDynamicStateDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FProximityDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSetPivotDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAddCustomCollectionAttributeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetNumElementsInCollectionGroupDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetCollectionAttributeDataTypedDataflowNode);
		// Commented out until AnyType outputs can properly change types
//		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetCollectionAttributeDataTypedDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetCollectionAttributeDataTypedDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMultiplyTransformDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FInvertTransformDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSelectionToVertexListDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBranchFloatDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBranchIntDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBoundingSphereDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FExpandBoundingSphereDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVisualizeTetrahedronsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FPointsToCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionToPointsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSpheresToPointsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionInfoDataflowNode);

		// Deprecated nodes
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetCollectionFromAssetDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FStringAppendDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FTransformMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBoundingBoxDataflowNode);
	}
}

// DEPRECATED 5.7
void FGetCollectionFromAssetDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		if (CollectionAsset)
		{
			if (const TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> AssetCollection = CollectionAsset->GetGeometryCollection())
			{
				SetValue(Context, (const FManagedArrayCollection&)(*AssetCollection), &Collection);
			}
			else
			{
				SetValue(Context, FManagedArrayCollection(), &Collection);
			}
		}
		else
		{
			SetValue(Context, FManagedArrayCollection(), &Collection);
		}
	}
}


void FAppendCollectionAssetsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection1))
	{
		FManagedArrayCollection InCollection1 = GetValue<DataType>(Context, &Collection1);
		const FManagedArrayCollection& InCollection2 = GetValue<DataType>(Context, &Collection2);
		TArray<FString> GeometryGroupGuidsLocal1, GeometryGroupGuidsLocal2;
		if (const TManagedArray<FString>* GuidArray1 = InCollection1.FindAttribute<FString>("Guid", FGeometryCollection::GeometryGroup))
		{
			GeometryGroupGuidsLocal1 = GuidArray1->GetConstArray();
		}
		InCollection1.Append(InCollection2);

		//Manually update indices in TransformToGeometryIndex, Parent and Children attributes, since they do not have group dependencies set to automatically manage this
		// TODO: Can we set up dependencies s.t. these indices are updated automatically, and then remove this manual fixup?
		{
			const int32 GeometryOffset = InCollection2.NumElements(FGeometryCollection::GeometryGroup);
			const int32 OtherSize = InCollection2.NumElements(FGeometryCollection::TransformGroup);
			const int32 Size = InCollection1.NumElements(FGeometryCollection::TransformGroup);
			if (TManagedArray<int32>* TransformToGeometryIndex = InCollection1.ModifyAttributeTyped<int32>("TransformToGeometryIndex", FGeometryCollection::TransformGroup))
			{
				for (int32 Idx = OtherSize; Idx < Size; ++Idx)
				{
					if ((*TransformToGeometryIndex)[Idx] != INDEX_NONE)
					{
						(*TransformToGeometryIndex)[Idx] += GeometryOffset;
					}
				}
			}
			if (TManagedArray<int32>* Parent = InCollection1.ModifyAttributeTyped<int32>("Parent", FGeometryCollection::TransformGroup))
			{
				for (int32 Idx = OtherSize; Idx < Size; ++Idx)
				{
					if ((*Parent)[Idx] != INDEX_NONE)
					{
						(*Parent)[Idx] += OtherSize;
					}
				}
			}
			if (TManagedArray<TSet<int32>>* Children = InCollection1.ModifyAttributeTyped<TSet<int32>>("Children", FGeometryCollection::TransformGroup))
			{
				for (int32 Idx = OtherSize; Idx < Size; ++Idx)
				{
					for (int32& Child : (*Children)[Idx])
					{
						if (Child != INDEX_NONE)
						{
							Child += OtherSize;
						}
					}
				}
			}
		}


		SetValue(Context, MoveTemp(InCollection1), &Collection1);
		if (const TManagedArray<FString>* GuidArray2 = InCollection2.FindAttribute<FString>("Guid", FGeometryCollection::GeometryGroup))
		{
			GeometryGroupGuidsLocal2 = GuidArray2->GetConstArray();
		}
		SetValue(Context, MoveTemp(GeometryGroupGuidsLocal1), &GeometryGroupGuidsOut1);
		SetValue(Context, MoveTemp(GeometryGroupGuidsLocal2), &GeometryGroupGuidsOut2);
	}
}


void FPrintStringDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	FString Value = GetValue<FString>(Context, &String);

	if (bPrintToScreen)
	{
		GEngine->AddOnScreenDebugMessage(-1, Duration, Color, Value);
	}
	if (bPrintToLog)
	{
		UE_LOGF(LogTemp, Warning, "Text, %ls", *Value);
	}
}

void FLogStringDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (bPrintToLog)
	{
		FString Value = GetValue<FString>(Context, &String);
		UE_LOGF(LogTemp, Warning, "[Dataflow Log] %ls", *Value);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////

FBoundingBoxDataflowNode::FBoundingBoxDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);

	RegisterOutputConnection(&BoundingBox);
	RegisterOutputConnection(&Center);
	RegisterOutputConnection(&Dimensions);
}

void FBoundingBoxDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&BoundingBox) || Out->IsA(&Center) || Out->IsA(&Dimensions))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

		GeometryCollection::Facades::FBoundsFacade BoundsFacade(InCollection);
		const FBox& BoundingBoxInCollectionSpace = BoundsFacade.GetBoundingBoxInCollectionSpace();

		SetValue(Context, BoundingBoxInCollectionSpace, &BoundingBox);
		SetValue(Context, BoundingBoxInCollectionSpace.GetCenter(), &Center);
		SetValue(Context, BoundingBoxInCollectionSpace.GetSize(), &Dimensions);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////

FBoundingBoxRatioDataflowNode::FBoundingBoxRatioDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&ReferenceBoxSize);

	RegisterOutputConnection(&BoundingBox);
	RegisterOutputConnection(&Dimensions);
	RegisterOutputConnection(&BoxRatio);
}

void FBoundingBoxRatioDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&BoundingBox) || Out->IsA(&BoxRatio) || Out->IsA(&Dimensions))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const float InReferenceBoxSize = GetValue(Context, &ReferenceBoxSize);
		const double ReferenceBoxVolume = InReferenceBoxSize * InReferenceBoxSize * InReferenceBoxSize;

		GeometryCollection::Facades::FBoundsFacade BoundsFacade(InCollection);
		const FBox& BoundingBoxInCollectionSpace = BoundsFacade.GetBoundingBoxInCollectionSpace();
		const FVector OutBoxDimension = BoundingBoxInCollectionSpace.GetSize();
		const double BoxVolume = OutBoxDimension.X * OutBoxDimension.Y * OutBoxDimension.Z;

		const float OutBoxRatio = static_cast<float>(BoxVolume / FMath::Max(ReferenceBoxVolume, UE_SMALL_NUMBER));

		SetValue(Context, BoundingBoxInCollectionSpace, &BoundingBox);
		SetValue(Context, OutBoxDimension, &Dimensions);
		SetValue(Context, OutBoxRatio, &BoxRatio);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////

FBoundingBoxAverageExtentsDataflowNode::FBoundingBoxAverageExtentsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&BoundingBox);

	RegisterOutputConnection(&AverageExtents);
	RegisterOutputConnection(&AverageHalfExtents);
	RegisterOutputConnection(&MinLength);
	RegisterOutputConnection(&MaxLength);
	RegisterOutputConnection(&LengthRatio);
}

void FBoundingBoxAverageExtentsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&AverageExtents) || Out->IsA(&AverageHalfExtents) || Out->IsA(&MinLength) || Out->IsA(&MaxLength) || Out->IsA(&LengthRatio))
	{
		const FBox& InBox = GetValue(Context, &BoundingBox);
		const FVector BoxSize = InBox.GetSize();

		const float OutAverageExtents = (BoxSize.X + BoxSize.Y + BoxSize.Z) / 3.f;
		const float OutAverageHalfExtents = OutAverageExtents * 0.5f;
		const float OutMinLength = BoxSize.GetMin();
		const float OutMaxLength = BoxSize.GetMax();
		const float OutLengthRatio = (OutMinLength / FMath::Max(OutMaxLength, UE_SMALL_NUMBER));

		SetValue(Context, OutAverageExtents, &AverageExtents);
		SetValue(Context, OutAverageHalfExtents, &AverageHalfExtents);
		SetValue(Context, OutMinLength, &MinLength);
		SetValue(Context, OutMaxLength, &MaxLength);
		SetValue(Context, OutLengthRatio, &LengthRatio);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////

void FGetBoxLengthsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Lengths))
	{
		const TArray<FBox>& InBoxes = GetValue(Context, &Boxes);

		TArray<float> OutLengths;
		OutLengths.SetNumUninitialized(InBoxes.Num());
		for (int32 Idx = 0; Idx < InBoxes.Num(); ++Idx)
		{
			const FBox& Box = InBoxes[Idx];
			OutLengths[Idx] = BoxToMeasurement(Box);
		}

		SetValue(Context, MoveTemp(OutLengths), &Lengths);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////

FExpandBoundingBoxDataflowNode::FExpandBoundingBoxDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&BoundingBox);
	RegisterOutputConnection(&Min);
	RegisterOutputConnection(&Max);
	RegisterOutputConnection(&Center);
	RegisterOutputConnection(&HalfExtents);
	RegisterOutputConnection(&Dimensions);
	RegisterOutputConnection(&Volume);
}

void FExpandBoundingBoxDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	FBox BBox = GetValue<FBox>(Context, &BoundingBox);

	if (Out->IsA<FVector>(&Min))
	{
		SetValue(Context, BBox.Min, &Min);
	}
	else if (Out->IsA<FVector>(&Max))
	{
		SetValue(Context, BBox.Max, &Max);
	}
	else if (Out->IsA<FVector>(&Center))
	{
		SetValue(Context, BBox.GetCenter(), &Center);
	}
	else if (Out->IsA<FVector>(&HalfExtents))
	{
		SetValue(Context, BBox.GetExtent(), &HalfExtents);
	}
	else if (Out->IsA<FVector>(&Dimensions))
	{
		SetValue(Context, BBox.GetSize(), &Dimensions);
	}
	else if (Out->IsA<float>(&Volume))
	{
		SetValue(Context, (float)BBox.GetVolume(), &Volume);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////

void FExpandVectorDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	FVector VectorVal = GetValue<FVector>(Context, &Vector);

	if (Out->IsA<float>(&X))
	{
		SetValue(Context, (float)VectorVal.X, &X);
	}
	else if (Out->IsA<float>(&Y))
	{
		SetValue(Context, (float)VectorVal.Y, &Y);
	}
	else if (Out->IsA<float>(&Z))
	{
		SetValue(Context, (float)VectorVal.Z, &Z);
	}
}

void FStringAppendDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FString>(&String))
	{
		FString StringOut = GetValue<FString>(Context, &String1) + GetValue<FString>(Context, &String2);
		SetValue(Context, MoveTemp(StringOut), &String);
	}
}

//-----------------------------------------------------------------------------------------------

FStringAppendDataflowNode_v2::FStringAppendDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&String);

	// Add initial variable inputs
	for (int32 Index = 0; Index < NumInitialVariableInputs; ++Index)
	{
		AddPins();
	}
}

void FStringAppendDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&String))
	{
		FString ResultStr;

		for (int32 Idx = 0; Idx < Inputs.Num(); ++Idx)
		{
			const FString InputValue = GetValue(Context, GetConnectionReference(Idx));

			ResultStr = ResultStr + InputValue;
		}

		SetValue(Context, ResultStr, &String);
	}
}

bool FStringAppendDataflowNode_v2::CanAddPin() const
{
	return true;
}

bool FStringAppendDataflowNode_v2::CanRemovePin() const
{
	return Inputs.Num() > 0;
}

UE::Dataflow::TConnectionReference<FDataflowStringConvertibleTypes> FStringAppendDataflowNode_v2::GetConnectionReference(int32 Index) const
{
	return { &Inputs[Index], Index, &Inputs };
}

TArray<UE::Dataflow::FPin> FStringAppendDataflowNode_v2::AddPins()
{
	const int32 Index = Inputs.AddDefaulted();
	const FDataflowInput& Input = RegisterInputArrayConnection(GetConnectionReference(Index));
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}

TArray<UE::Dataflow::FPin> FStringAppendDataflowNode_v2::GetPinsToRemove() const
{
	const int32 Index = (Inputs.Num() - 1);
	check(Inputs.IsValidIndex(Index));
	if (const FDataflowInput* const Input = FindInput(GetConnectionReference(Index)))
	{
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	}
	return Super::GetPinsToRemove();
}

void FStringAppendDataflowNode_v2::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = Inputs.Num() - 1;
	check(Inputs.IsValidIndex(Index));
#if DO_CHECK
	const FDataflowInput* const Input = FindInput(GetConnectionReference(Index));
	check(Input);
	check(Input->GetName() == Pin.Name);
	check(Input->GetType() == Pin.Type);
#endif
	Inputs.SetNum(Index);

	return Super::OnPinRemoved(Pin);
}

void FStringAppendDataflowNode_v2::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		check(Inputs.Num() >= 0);
		// register new elements from the array as inputs
		for (int32 Index = 0; Index < Inputs.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetConnectionReference(Index));
		}
		if (Ar.IsTransacting())
		{
			// if we have more inputs than materials then we need to unregister the inputs 
			const int32 NumVariableInputs = (GetNumInputs() - NumOtherInputs);
			const int32 NumInputs = Inputs.Num();
			if (NumVariableInputs > NumInputs)
			{
				// Inputs have been removed.
				// Temporarily expand Collections so we can get connection references.
				Inputs.SetNum(NumVariableInputs);
				for (int32 Index = NumInputs; Index < Inputs.Num(); ++Index)
				{
					UnregisterInputConnection(GetConnectionReference(Index));
				}
				Inputs.SetNum(NumInputs);
			}
		}
		else
		{
			ensureAlways(Inputs.Num() + NumOtherInputs == GetNumInputs());
		}
	}
}

//-----------------------------------------------------------------------------------------------

void FHashStringDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<int32>(&Hash))
	{
		SetValue(Context, (int32)GetTypeHash(GetValue<FString>(Context, &String)), &Hash);
	}
}

void FHashVectorDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<int32>(&Hash))
	{
		SetValue(Context, (int32)GetTypeHash(GetValue<FVector>(Context, &Vector)), &Hash);
	}
}

//-----------------------------------------------------------------------------------------------

void FGetBoundingBoxesFromCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&BoundingBoxes))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const FDataflowTransformSelection& InTransformSelection = GetValue(Context, &TransformSelection);

		TArray<FBox> BoundingBoxesArr;

		GeometryCollection::Facades::FBoundsFacade BoundsFacade(InCollection);
		GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(InCollection);
		if (BoundsFacade.IsValid() && TransformFacade.IsValid())
		{
			const bool bValidSelection = InTransformSelection.IsValidForCollection(InCollection);

			const TManagedArray<FBox>& InBoundingBoxes = BoundsFacade.GetBoundingBoxes();
			for (int32 Idx = 0; Idx < InBoundingBoxes.Num(); ++Idx)
			{
				const FBox BoundingBoxInBoneSpace = InBoundingBoxes[Idx];

				// Transform from BoneSpace to CollectionSpace
				const FTransform CollectionSpaceTransform = TransformFacade.ComputeCollectionSpaceTransform(Idx);
				const FBox BoundingBoxInCollectionSpace = BoundingBoxInBoneSpace.TransformBy(CollectionSpaceTransform);

				if (!bValidSelection || InTransformSelection.IsSelected(Idx))
				{
					BoundingBoxesArr.Add(BoundingBoxInCollectionSpace);
				}
			}

			SetValue(Context, MoveTemp(BoundingBoxesArr), &BoundingBoxes);
		}
	}
}

//-----------------------------------------------------------------------------------------------

void FGetRootIndexFromCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&RootIndex))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(InCollection);
		SetValue(Context, HierarchyFacade.GetRootIndex(), &RootIndex);
	}
}

//-----------------------------------------------------------------------------------------------

FGetCentroidsFromCollectionDataflowNode::FGetCentroidsFromCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid) 
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&TransformSelection);
	RegisterOutputConnection(&Centroids);
	
	// This is a temporary solution to store temp data
	RegisterOutputConnection(&Levels) .SetCanHidePin(true) .SetPinIsHidden(true); 

	PointSize.GetRichCurve()->AddKey(0.f, 12.f);
	PointSize.GetRichCurve()->AddKey(1.f, 6.f);
}

void FGetCentroidsFromCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Centroids))
	{
		TArray<FVector> OutCentroids;
		TArray<int32> OutLevels;

		FManagedArrayCollection InCollection = GetValue(Context, &Collection);

		Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(InCollection);
		if (!HierarchyFacade.HasLevelAttribute())
		{
			HierarchyFacade.GenerateLevelAttribute();
		}
		
		const FDataflowTransformSelection& InTransformSelection = GetValue(Context, &TransformSelection);

		const bool bIsSelectionConnected = IsConnected(&TransformSelection);

		GeometryCollection::Facades::FBoundsFacade BoundsFacade(InCollection);
		GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(InCollection);
		if (BoundsFacade.IsValid() && TransformFacade.IsValid())
		{
			const TArray<FVector>& InCentroids = BoundsFacade.GetCentroids();
			const TManagedArray<int32>& TransformToGeometryIndex = BoundsFacade.GetTransformToGeometryIndex();
			const TManagedArray<int32>& LevelsArr = InCollection.GetAttribute<int32>(FTransformCollection::LevelAttribute, FTransformCollection::TransformGroup);

			const int32 NumTransforms = InCollection.NumElements(FTransformCollection::TransformGroup);
			for (int32 TransformIdx = 0; TransformIdx < NumTransforms; ++TransformIdx)
			{
				const int32 GeometryIndex = TransformToGeometryIndex.IsValidIndex(TransformIdx) ? TransformToGeometryIndex[TransformIdx] : INDEX_NONE;
				if (InCentroids.IsValidIndex(GeometryIndex))
				{
					const FVector PositionInBoneSpace(InCentroids[GeometryIndex]);
					const FTransform CollectionSpaceTransform = TransformFacade.ComputeCollectionSpaceTransform(TransformIdx);
					const FVector PositionInCollectionSpace = CollectionSpaceTransform.TransformPosition(PositionInBoneSpace);
					if (!bIsSelectionConnected || TransformIdx >= InTransformSelection.Num() || InTransformSelection.IsSelected(TransformIdx))
					{
						OutCentroids.Add(PositionInCollectionSpace);
						OutLevels.Add(LevelsArr[TransformIdx]);
					}
				}
			}
		}

		SetValue(Context, MoveTemp(OutLevels), &Levels);
		SetValue(Context, MoveTemp(OutCentroids), &Centroids);
	}
}

#if WITH_EDITOR
bool FGetCentroidsFromCollectionDataflowNode::CanDebugDraw() const
{
	return true;
}

bool FGetCentroidsFromCollectionDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FGetCentroidsFromCollectionDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		if (IsConnected(&Collection))
		{
			FManagedArrayCollection InCollection = GetValue(Context, &Collection);
			Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(InCollection);

			if (HierarchyFacade.IsValid())
			{
				if (!HierarchyFacade.HasLevelAttribute())
				{
					HierarchyFacade.GenerateLevelAttribute();
				}
				float LevelsMin, LevelsMax;
				const TManagedArray<int32>& LevelsArr = InCollection.GetAttribute<int32>(FTransformCollection::LevelAttribute, FTransformCollection::TransformGroup);

				GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(InCollection);
				if (TransformFacade.IsValid())
				{
					TransformFacade.ComputeLevelsBounds(LevelsMin, LevelsMax);

					const FDataflowOutput* CentroidsOutput = FindOutput(&Centroids);
					if (CentroidsOutput)
					{
						const TArray<FVector> EmptyArray;
						const TArray<FVector>& OutCentroids = CentroidsOutput->GetValue<TArray<FVector>>(Context, EmptyArray);

						DataflowRenderingInterface.SetWireframe(true);
						DataflowRenderingInterface.SetWorldPriority();
						DataflowRenderingInterface.SetShaded(false);
						DataflowRenderingInterface.SetColor(Color);

						if (OutCentroids.Num() > 0)
						{
							for (int32 Idx = 0; Idx < OutCentroids.Num(); ++Idx)
							{
								if (bColorByLevel)
								{
									DataflowRenderingInterface.SetColor(UE::Dataflow::Utils::GetColorByLevel(InCollection, LevelsArr[Idx]));
								}

								float x;
								if (LevelsMin == LevelsMax)
								{
									x = (float)LevelsMin;
								}
								else
								{
									x = ((float)LevelsArr[Idx] - (float)LevelsMin) / ((float)LevelsMax - (float)LevelsMin);
								}
								const float PointSizeComputed = PointSize.GetRichCurveConst()->Eval(x);

								if (bSizeByLevel)
								{
									DataflowRenderingInterface.SetPointSize(PointSizeComputed);
								}
								else
								{
									DataflowRenderingInterface.SetPointSize(Size);
								}

								DataflowRenderingInterface.DrawPoint(FVector(OutCentroids[Idx]));
							}
						}
					}
				}
			}
		}
	}
}
#endif

//-----------------------------------------------------------------------------------------------

FTransformCollectionDataflowNode::FTransformCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&TransformSelection);
	RegisterInputConnection(&Translate);
	RegisterInputConnection(&Rotate);
	RegisterInputConnection(&Scale);
	RegisterInputConnection(&RotationOrder)
		.SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&UniformScale)
		.SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RotatePivot)
		.SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ScalePivot)
		.SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bInvertTransformation)
		.SetCanHidePin(true).SetPinIsHidden(true);
	
	RegisterOutputConnection(&Collection, &Collection);
}

void FTransformCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		const FVector InTranslate = GetValue(Context, &Translate);
		const FVector InRotate = GetValue(Context, &Rotate);
		const FVector InScale = GetValue(Context, &Scale);
		const ERotationOrderEnum InRotationOrder = GetValue(Context, &RotationOrder);
		const float InUniformScale = GetValue(Context, &UniformScale);
		const FVector InRotatePivot = GetValue(Context, &RotatePivot);
		const FVector InScalePivot = GetValue(Context, &ScalePivot);
		const bool bInInvertTransformation = GetValue(Context, &bInvertTransformation);

		const FTransform NewTransform = GeometryCollection::Facades::FCollectionTransformFacade::BuildTransform(
			InTranslate,
			(uint8)InRotationOrder,
			InRotate,
			InScale,
			InUniformScale,
			InRotatePivot,
			InScalePivot,
			bInInvertTransformation);

		GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(InCollection);

		if (!IsConnected(&TransformSelection))
		{
			TransformFacade.Transform(NewTransform);
		}
		else
		{
			const FDataflowTransformSelection& InTransformSelection = GetValue(Context, &TransformSelection);

			TransformFacade.Transform(NewTransform, InTransformSelection.AsArray());
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}


void FBakeTransformsInCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(InCollection);
		const TArray<FTransform>& CollectionSpaceTransforms = TransformFacade.ComputeCollectionSpaceTransforms();

		GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(InCollection);

		const int32 NumTransforms = InCollection.NumElements(FGeometryCollection::TransformGroup);

		for (int32 TransformIdx = 0; TransformIdx < NumTransforms; ++TransformIdx)
		{
			MeshFacade.BakeTransform(TransformIdx, CollectionSpaceTransforms[TransformIdx]);
			TransformFacade.SetBoneTransformToIdentity(TransformIdx);
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

// -----------------------------------------------------------------------------------------------------------------------------------

void FTransformMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TObjectPtr<UDynamicMesh>>(&Mesh))
	{
		if (TObjectPtr<const UDynamicMesh> InMesh = GetValue<TObjectPtr<UDynamicMesh>>(Context, &Mesh))
		{
			// Creating a new mesh object from InMesh
			TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
			NewMesh->SetMesh(InMesh->GetMeshRef());

			const FVector& InTranslate = GetValue(Context, &Translate);
			const FVector& InRotate = GetValue(Context, &Rotate);
			const FVector& InScale = GetValue(Context, &Scale);
			const float InUniformScale = GetValue(Context, &UniformScale);
			const FVector& InRotatePivot = GetValue(Context, &RotatePivot);
			const FVector& InScalePivot = GetValue(Context, &ScalePivot);
			const bool InbInvertTransformation = GetValue(Context, &bInvertTransformation);
			
			FTransform NewTransform = GeometryCollection::Facades::FCollectionTransformFacade::BuildTransform(
				InTranslate,
				(uint8)RotationOrder,
				InRotate,
				InScale,
				InUniformScale,
				InRotatePivot,
				InScalePivot,
				InbInvertTransformation);

			UE::Geometry::FDynamicMesh3& DynamicMesh = NewMesh->GetMeshRef();

			MeshTransforms::ApplyTransform(DynamicMesh, UE::Geometry::FTransformSRT3d(NewTransform), true);

			SetValue(Context, NewMesh, &Mesh);
		}
		else
		{
			SetValue(Context, TObjectPtr<UDynamicMesh>(NewObject<UDynamicMesh>()), &Mesh);
		}
	}
}

// -----------------------------------------------------------------------------------------------------------------------------------

FTransformMeshDataflowNode_v2::FTransformMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&Translate);
	RegisterInputConnection(&Rotate);
	RegisterInputConnection(&Scale);
	RegisterInputConnection(&UniformScale);
	RegisterInputConnection(&RotatePivot).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ScalePivot).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bInvertTransformation).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Mesh, &Mesh);
}

void FTransformMeshDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Mesh))
	{
		if (TObjectPtr<UDataflowMesh> InMesh = GetValue(Context, &Mesh))
		{
			// Creating a new mesh object from InMesh
			if (TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>())
			{
				NewMesh->SetMesh(*InMesh->GetDynamicMesh());

				const FVector& InTranslate = GetValue(Context, &Translate);
				const FVector& InRotate = GetValue(Context, &Rotate);
				const FVector& InScale = GetValue(Context, &Scale);
				const float InUniformScale = GetValue(Context, &UniformScale);
				const FVector& InRotatePivot = GetValue(Context, &RotatePivot);
				const FVector& InScalePivot = GetValue(Context, &ScalePivot);
				const bool InbInvertTransformation = GetValue(Context, &bInvertTransformation);

				FTransform NewTransform = GeometryCollection::Facades::FCollectionTransformFacade::BuildTransform(
					InTranslate,
					(uint8)RotationOrder,
					InRotate,
					InScale,
					InUniformScale,
					InRotatePivot,
					InScalePivot,
					InbInvertTransformation);

				UE::Geometry::FDynamicMesh3& DynamicMesh = NewMesh->GetMeshRef();

				MeshTransforms::ApplyTransform(DynamicMesh, UE::Geometry::FTransformSRT3d(NewTransform), true);

				if (TObjectPtr<UDataflowMesh> NewDataflowMesh = NewObject<UDataflowMesh>())
				{
					NewDataflowMesh->SetDynamicMesh(MoveTemp(DynamicMesh));
					NewDataflowMesh->SetMaterials(InMesh->GetMaterials());

					SetValue(Context, NewDataflowMesh, &Mesh);
					return;
				}
			}
		}

		SetValue(Context, TObjectPtr<UDataflowMesh>(NewObject<UDataflowMesh>()), &Mesh);
	}
}

// -----------------------------------------------------------------------------------------------------------------------------------

namespace
{
	// helper to apply an ECompareOperationEnum operation to various numeric types
	template <typename T>
	static bool ApplyDataflowOperationComparison(T A, T B, ECompareOperationEnum Operation)
	{
		switch (Operation)
		{
		case ECompareOperationEnum::Dataflow_Compare_Equal:
			return A == B;
		case ECompareOperationEnum::Dataflow_Compare_Smaller:
			return A < B;
		case ECompareOperationEnum::Dataflow_Compare_SmallerOrEqual:
			return A <= B;
		case ECompareOperationEnum::Dataflow_Compare_Greater:
			return A > B;
		case ECompareOperationEnum::Dataflow_Compare_GreaterOrEqual:
			return A >= B;
		case ECompareOperationEnum::Dataflow_Compare_NotEqual:
			return A != B;
		default:
			ensureMsgf(false, TEXT("Invalid ECompareOperationEnum value: %u"), (uint8)Operation);
		}

		return false;
	}
}


void FCompareIntDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<bool>(&Result))
	{
		const int32 IntAValue = GetValue<int32>(Context, &IntA);
		const int32 IntBValue = GetValue<int32>(Context, &IntB);
		const bool ResultValue = ApplyDataflowOperationComparison(IntAValue, IntBValue, Operation);

		SetValue(Context, ResultValue, &Result);
	}
}


void FCompareFloatDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Result))
	{
		const float AValue = GetValue(Context, &FloatA);
		const float BValue = GetValue(Context, &FloatB);
		const bool ResultValue = ApplyDataflowOperationComparison(AValue, BValue, Operation);

		SetValue(Context, ResultValue, &Result);
	}
}


void FBooleanOperationDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&bResult))
	{
		bool bResultValue = false;
		switch (Operation)
		{
		case EBooleanOperationEnum::Dataflow_And:
			bResultValue = GetValue(Context, &bBoolA) && GetValue(Context, &bBoolB);
			break;
		case EBooleanOperationEnum::Dataflow_Or:
			bResultValue = GetValue(Context, &bBoolA) || GetValue(Context, &bBoolB);
			break;
		case EBooleanOperationEnum::Dataflow_Not:
			bResultValue = !GetValue(Context, &bBoolA);
			break;
		default:
			ensureMsgf(false, TEXT("Invalid EBooleanOperationEnum value: %u"), (uint8)Operation);
		}

		SetValue(Context, bResultValue, &bResult);
	}
}


void FBranchMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TObjectPtr<UDynamicMesh>>(&Mesh))
	{
		bool InCondition = GetValue<bool>(Context, &bCondition);

		if (InCondition)
		{
			if (TObjectPtr<UDynamicMesh> InMeshA = GetValue<TObjectPtr<UDynamicMesh>>(Context, &MeshA))
			{
				SetValue(Context, InMeshA, &Mesh);

				return;
			}
		}
		else
		{
			if (TObjectPtr<UDynamicMesh> InMeshB = GetValue<TObjectPtr<UDynamicMesh>>(Context, &MeshB))
			{
				SetValue(Context, InMeshB, &Mesh);

				return;
			}
		}

		SetValue(Context, TObjectPtr<UDynamicMesh>(NewObject<UDynamicMesh>()), &Mesh);
	}
}


void FBranchCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&ChosenCollection))
	{
		bool InCondition = GetValue<bool>(Context, &bCondition);

		if (InCondition)
		{
			if (IsConnected(&TrueCollection))
			{
				const FManagedArrayCollection& InTrueCollection = GetValue(Context, &TrueCollection);
				SetValue(Context, InTrueCollection, &ChosenCollection);
				return;
			}
		}
		else
		{
			if (IsConnected(&FalseCollection))
			{
				const FManagedArrayCollection& InFalseCollection = GetValue(Context, &FalseCollection);
				SetValue(Context, InFalseCollection, &ChosenCollection);
				return;
			}
		}

		// default empty collection 
		SetValue(Context, FManagedArrayCollection(), &ChosenCollection);
	}
}


namespace {
	inline FName GetArrayTypeString(FManagedArrayCollection::EArrayType ArrayType)
	{
		switch (ArrayType)
		{
#define MANAGED_ARRAY_TYPE(a,A)	case EManagedArrayType::F##A##Type:\
			return FName(#A);
#include "GeometryCollection/ManagedArrayTypeValues.inl"
#undef MANAGED_ARRAY_TYPE
		}
		return FName();
	}
}

void FGetSchemaDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FString>(&String))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		FString OutputStr;
		OutputStr.Appendf(TEXT("\n----------------------------------------\n"));
		for (auto& Group : InCollection.GroupNames())
		{
			if (InCollection.HasGroup(Group))
			{
				int32 NumElems = InCollection.NumElements(Group);

				OutputStr.Appendf(TEXT("Group: %s  Number of Elements: %d\n"), *Group.ToString(), NumElems);
				OutputStr.Appendf(TEXT("Attributes:\n"));

				for (auto& Attr : InCollection.AttributeNames(Group))
				{
					if (InCollection.HasAttribute(Attr, Group))
					{
						FString TypeStr = GetArrayTypeString(InCollection.GetAttributeType(Attr, Group)).ToString();
						OutputStr.Appendf(TEXT("\t%s\t[%s]\n"), *Attr.ToString(), *TypeStr);
					}
				}

				OutputStr.Appendf(TEXT("\n--------------------\n"));
			}
		}
		OutputStr.Appendf(TEXT("----------------------------------------\n"));

		SetValue(Context, MoveTemp(OutputStr), &String);
	}
}


void FRemoveOnBreakDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		const bool& InEnableRemoval = GetValue(Context, &bEnabledRemoval, true);
		const FVector2f& InPostBreakTimer = GetValue(Context, &PostBreakTimer);
		const FVector2f& InRemovalTimer = GetValue(Context, &RemovalTimer);
		const bool& InClusterCrumbling = GetValue(Context, &bClusterCrumbling);

		// we are making a copy of the collection because we are modifying it 
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		GeometryCollection::Facades::FCollectionRemoveOnBreakFacade RemoveOnBreakFacade(InCollection);
		RemoveOnBreakFacade.DefineSchema();

		GeometryCollection::Facades::FRemoveOnBreakData Data;
		Data.SetBreakTimer(InPostBreakTimer.X, InPostBreakTimer.Y);
		Data.SetRemovalTimer(InRemovalTimer.X, InRemovalTimer.Y);
		Data.SetEnabled(InEnableRemoval);
		Data.SetClusterCrumbling(InClusterCrumbling);

		// selection is optional
		if (IsConnected<FDataflowTransformSelection>(&TransformSelection))
		{
			const FDataflowTransformSelection& InTransformSelection = GetValue(Context, &TransformSelection);
			TArray<int32> TransformIndices;
			InTransformSelection.AsArrayValidated(TransformIndices, InCollection);
			RemoveOnBreakFacade.SetFromIndexArray(TransformIndices, Data);
		}
		else
		{
			RemoveOnBreakFacade.SetToAll(Data);
		}

		// move the collection to the output to avoid making another copy
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

//////////////////////////////////////////////////////////////////////////////////////

FSetAnchorStateDataflowNode::FSetAnchorStateDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid )
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&TransformSelection);
	RegisterOutputConnection(&Collection, &Collection);
}

void FSetAnchorStateDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection OutCollection = GetValue(Context, &Collection);
		if (IsConnected(&Collection))
		{
			FDataflowTransformSelection InTransformSelection = GetValue(Context, &TransformSelection);

			Chaos::Facades::FCollectionAnchoringFacade AnchoringFacade(OutCollection);
			if (!AnchoringFacade.HasAnchoredAttribute())
			{
				AnchoringFacade.AddAnchoredAttribute();
			}

			const bool bAnchored = (AnchorState == EAnchorStateEnum::Dataflow_AnchorState_Anchored) ? true : false;
			TArray<int32> BoneIndices;
			InTransformSelection.AsArrayValidated(BoneIndices, OutCollection);
			AnchoringFacade.SetAnchored(BoneIndices, bAnchored);

			if (bSetNotSelectedBonesToOppositeState)
			{
				InTransformSelection.Invert();
				InTransformSelection.AsArrayValidated(BoneIndices, OutCollection);
				AnchoringFacade.SetAnchored(BoneIndices, !bAnchored);
			}
		}
		SetValue(Context, OutCollection, &Collection);
	}
}


#if WITH_EDITOR

bool FSetAnchorStateDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FSetAnchorStateDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const FDataflowTransformSelection& InTransformSelection = GetValue(Context, &TransformSelection);

		GeometryCollection::Facades::FBoundsFacade BoundsFacade(InCollection);
		const TArray<FVector> Centroids = BoundsFacade.GetCentroids();

		GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(InCollection);

		DataflowRenderingInterface.SetColor(FLinearColor::Blue);
		DataflowRenderingInterface.SetPointSize(5.f);
		DataflowRenderingInterface.ReservePoints(InTransformSelection.NumSelected());
		DataflowRenderingInterface.SetForegroundPriority();

		const int32 NumCentroids = Centroids.Num();
		for (int32 TransformIdx = 0; TransformIdx < InTransformSelection.Num(); TransformIdx++)
		{
			if (TransformIdx < NumCentroids && InTransformSelection.IsSelected(TransformIdx))
			{
				const FTransform CollectionSpaceTransform = TransformFacade.ComputeCollectionSpaceTransform(TransformIdx);
				const FVector Point = CollectionSpaceTransform.TransformPosition(Centroids[TransformIdx]);
				DataflowRenderingInterface.DrawPoint(Point);
			}
		}
	}
}

#endif

//////////////////////////////////////////////////////////////////////////////////////

FSetDynamicStateDataflowNode::FSetDynamicStateDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&TransformSelection);
	RegisterOutputConnection(&Collection, &Collection);
}

void FSetDynamicStateDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection OutCollection = GetValue(Context, &Collection);
		if (IsConnected(&Collection))
		{
			const FDataflowTransformSelection& InTransformSelection = GetValue(Context, &TransformSelection);

			const TArray<int32> BoneIndices = InTransformSelection.AsArrayValidated(OutCollection);

			Chaos::Facades::FCollectionAnchoringFacade AnchoringFacade(OutCollection);
			if (AnchoringFacade.HasInitialDynamicStateAttribute())
			{
				const Chaos::EObjectStateType ObjectState = [this]()
					{
						switch (DynamicState)
						{
						case EDataflowGeometryCollectionDynamicState::None:		return Chaos::EObjectStateType::Uninitialized;
						case EDataflowGeometryCollectionDynamicState::Dynamic:	return Chaos::EObjectStateType::Dynamic;
						case EDataflowGeometryCollectionDynamicState::Kinematic:return Chaos::EObjectStateType::Kinematic;
						case EDataflowGeometryCollectionDynamicState::Static:	return Chaos::EObjectStateType::Static;
						}
						return Chaos::EObjectStateType::Dynamic;
					}();
					

				AnchoringFacade.SetInitialDynamicState(BoneIndices, ObjectState);
			}
		}
		SetValue(Context, OutCollection, &Collection);
	}
}

//////////////////////////////////////////////////////////////////////////////////////

/* ---------------------------------------------------------------------------------------------------------------------------------*/

void FProximityDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
		{
			FGeometryCollectionProximityPropertiesInterface::FProximityProperties Properties = GeomCollection->GetProximityProperties();

			Properties.Method = (EProximityMethod)ProximityMethod;
			Properties.ContactMethod = (EProximityContactMethod)FilterContactMethod;
			Properties.DistanceThreshold = GetValue(Context, &DistanceThreshold);
			Properties.bUseAsConnectionGraph = bUseAsConnectionGraph;
			Properties.ContactAreaMethod = (EConnectionContactMethod)ContactAreaMethod;
			Properties.RequireContactAmount = GetValue(Context, &ContactThreshold);

			GeomCollection->SetProximityProperties(Properties);

			UE::GeometryCollectionConvexUtility::FConvexHulls TransformedExistingHulls;
			bool bUseExistingHulls = false;
			if (!bRecomputeConvexHulls)
			{
				bUseExistingHulls = UE::GeometryCollectionConvexUtility::GetExistingConvexHullsInSharedSpace(GeomCollection.Get(), TransformedExistingHulls, true);
			}

			// Invalidate proximity
			FGeometryCollectionProximityUtility ProximityUtility(GeomCollection.Get());
			ProximityUtility.InvalidateProximity();
			ProximityUtility.UpdateProximity(bUseExistingHulls ? &TransformedExistingHulls : nullptr);

			SetValue<const FManagedArrayCollection&>(Context, *GeomCollection, &Collection);
		}
	}
}

/* ---------------------------------------------------------------------------------------------------------------------------------*/

void FCollectionSetPivotDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const FTransform& InTransform = GetValue<FTransform>(Context, &Transform);

		GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(InCollection);
		TransformFacade.SetPivot(InTransform);

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}


static FName GetGroupName(const EStandardGroupNameEnum& InGroupName)
{
	FName GroupNameToUse;
	if (InGroupName == EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Transform)
	{
		GroupNameToUse = FGeometryCollection::TransformGroup;
	}
	else if (InGroupName == EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Geometry)
	{
		GroupNameToUse = FGeometryCollection::GeometryGroup;
	}
	else if (InGroupName == EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Faces)
	{
		GroupNameToUse = FGeometryCollection::FacesGroup;
	}
	else if (InGroupName == EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Vertices)
	{
		GroupNameToUse = FGeometryCollection::VerticesGroup;
	}
	else if (InGroupName == EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Material)
	{
		GroupNameToUse = FGeometryCollection::MaterialGroup;
	}
	else if (InGroupName == EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Breaking)
	{
		GroupNameToUse = FGeometryCollection::BreakingGroup;
	}

	return GroupNameToUse;
}


template<typename T>
static void AddAndFillAttribute(FManagedArrayCollection& InCollection, FName AttributeName, FName GroupName, const T& DefaultValue)
{
	TManagedArrayAccessor<T> CustomAttribute(InCollection, AttributeName, GroupName);
	CustomAttribute.AddAndFill(DefaultValue);
}

void FAddCustomCollectionAttributeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const int32 InNumElements = GetValue<int32>(Context, &NumElements);

		FName GroupNameToUse;
		if (GroupName != EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Custom)
		{
			GroupNameToUse = GetGroupName(GroupName);
		}
		else
		{
			GroupNameToUse = FName(*CustomGroupName);
		}

		if (GroupNameToUse.GetStringLength() > 0 && AttrName.Len() > 0)
		{
			// If the group already exists don't change the number of elements
			if (!InCollection.HasGroup(GroupNameToUse))
			{
				InCollection.AddGroup(GroupNameToUse);
				InCollection.AddElements(InNumElements, GroupNameToUse);
			}

			FName AttributeNameToUse = FName(*AttrName);

			switch (CustomAttributeType)
			{
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_UInt8:
				AddAndFillAttribute<uint8>(InCollection, AttributeNameToUse, GroupNameToUse, uint8(0));
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_Int32:
				AddAndFillAttribute<int32>(InCollection, AttributeNameToUse, GroupNameToUse, 0);
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_Float:
				AddAndFillAttribute<float>(InCollection, AttributeNameToUse, GroupNameToUse, float(0.0));
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_Double:
				AddAndFillAttribute<double>(InCollection, AttributeNameToUse, GroupNameToUse, double(0.0));
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_Bool:
				AddAndFillAttribute<bool>(InCollection, AttributeNameToUse, GroupNameToUse, false);
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_String:
				AddAndFillAttribute<FString>(InCollection, AttributeNameToUse, GroupNameToUse, FString());
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_Vector2f:
				AddAndFillAttribute<FVector2f>(InCollection, AttributeNameToUse, GroupNameToUse, FVector2f(EForceInit::ForceInitToZero));
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_Vector3f:
				AddAndFillAttribute<FVector3f>(InCollection, AttributeNameToUse, GroupNameToUse, FVector3f(EForceInit::ForceInitToZero));
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_Vector3d:
				AddAndFillAttribute<FVector3d>(InCollection, AttributeNameToUse, GroupNameToUse, FVector3d(EForceInit::ForceInitToZero));
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_Vector4f:
				AddAndFillAttribute<FVector4f>(InCollection, AttributeNameToUse, GroupNameToUse, FVector4f(EForceInit::ForceInitToZero));
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_LinearColor:
				AddAndFillAttribute<FLinearColor>(InCollection, AttributeNameToUse, GroupNameToUse, FLinearColor(0.f, 0.f, 0.f, 1.f));
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_Transform:
				AddAndFillAttribute<FTransform>(InCollection, AttributeNameToUse, GroupNameToUse, FTransform(FTransform::Identity));
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_Quat4f:
				AddAndFillAttribute<FQuat4f>(InCollection, AttributeNameToUse, GroupNameToUse, FQuat4f(ForceInitToZero));
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_Box:
				AddAndFillAttribute<FBox>(InCollection, AttributeNameToUse, GroupNameToUse, FBox(ForceInit));
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_Guid:
				AddAndFillAttribute<FGuid>(InCollection, AttributeNameToUse, GroupNameToUse, FGuid());
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_Int32Set:
				AddAndFillAttribute<TSet<int32>>(InCollection, AttributeNameToUse, GroupNameToUse, TSet<int32>());
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_Int32Array:
				AddAndFillAttribute<TArray<int32>>(InCollection, AttributeNameToUse, GroupNameToUse, TArray<int32>());
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_IntVector:
				AddAndFillAttribute<FIntVector>(InCollection, AttributeNameToUse, GroupNameToUse, FIntVector());
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_IntVector2:
				AddAndFillAttribute<FIntVector2>(InCollection, AttributeNameToUse, GroupNameToUse, FIntVector2());
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_IntVector4:
				AddAndFillAttribute<FIntVector4>(InCollection, AttributeNameToUse, GroupNameToUse, FIntVector4());
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_IntVector2Array:
				AddAndFillAttribute<TArray<FIntVector2>>(InCollection, AttributeNameToUse, GroupNameToUse, TArray<FIntVector2>());
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_FloatArray:
				AddAndFillAttribute<TArray<float>>(InCollection, AttributeNameToUse, GroupNameToUse, TArray<float>());
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_Vector2fArray:
				AddAndFillAttribute<TArray<FVector2f>>(InCollection, AttributeNameToUse, GroupNameToUse, TArray<FVector2f>());
				break;
			case ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_FVector3fArray:
				AddAndFillAttribute<TArray<FVector3f>>(InCollection, AttributeNameToUse, GroupNameToUse, TArray<FVector3f>());
				break;
			}
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}


void FGetNumElementsInCollectionGroupDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<int32>(&NumElements))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		
		int32 OutNumElements = 0;

		FName GroupNameToUse;
		if (GroupName != EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Custom)
		{
			GroupNameToUse = GetGroupName(GroupName);
		}
		else
		{
			GroupNameToUse = FName(*CustomGroupName);
		}

		if (GroupNameToUse.GetStringLength() > 0)
		{
			if (InCollection.HasGroup(GroupNameToUse))
			{
				OutNumElements = InCollection.NumElements(GroupNameToUse);
			}
		}

		SetValue(Context, OutNumElements, &NumElements);
	}
}

void FGetCollectionAttributeDataTypedDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<TArray<bool>>(&BoolAttributeData) ||
		Out->IsA<TArray<float>>(&FloatAttributeData) ||
		Out->IsA<TArray<double>>(&DoubleAttributeData) ||
		Out->IsA<TArray<int32>>(&Int32AttributeData) ||
		Out->IsA<TArray<FString>>(&StringAttributeData) ||
		Out->IsA<TArray<FVector3f>>(&Vector3fAttributeData) ||
		Out->IsA<TArray<FVector3d>>(&Vector3dAttributeData))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		FName InputGroupName;
		if (GroupName != EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Custom)
		{
			InputGroupName = GetGroupName(GroupName);
		}
		else
		{
			InputGroupName = FName(*CustomGroupName);
		}

		SetValue(Context, TArray<bool>(), &BoolAttributeData);
		SetValue(Context, TArray<float>(), &FloatAttributeData);
		SetValue(Context, TArray<double>(), &DoubleAttributeData);
		SetValue(Context, TArray<int32>(), &Int32AttributeData);
		SetValue(Context, TArray<FString>(), &StringAttributeData);
		SetValue(Context, TArray<FVector3f>(), &Vector3fAttributeData);
		SetValue(Context, TArray<FVector3d>(), &Vector3dAttributeData);

		FCollectionAttributeKey DefaultAttributeKey(AttrName, InputGroupName.ToString());
		FCollectionAttributeKey AttributeKeyVal = GetValue(Context, &AttributeKey, DefaultAttributeKey);
		FName GroupNameVal = FName(AttributeKeyVal.Group);
		FName AttributeNameVal = FName(AttributeKeyVal.Attribute);

		if (GroupNameVal.GetStringLength() > 0 && AttributeNameVal.GetStringLength() > 0)
		{
			if (InCollection.HasGroup(GroupNameVal))
			{
				if (InCollection.HasAttribute(AttributeNameVal, GroupNameVal))
				{
					FString TypeStr = GetArrayTypeString(InCollection.GetAttributeType(AttributeNameVal, GroupNameVal)).ToString();

					if (TypeStr == FString("Bool"))
					{
						if (const TManagedArray<bool>* AttributeArr = InCollection.FindAttribute<bool>(AttributeNameVal, GroupNameVal))
						{
							TArray<bool> BoolArray = AttributeArr->GetAsBoolArray();
							SetValue(Context, MoveTemp(BoolArray), &BoolAttributeData);
						}
					}
					else if (TypeStr == FString("Float"))
					{
						if (const TManagedArray<float>* AttributeArr = InCollection.FindAttribute<float>(AttributeNameVal, GroupNameVal))
						{
							SetValue(Context, AttributeArr->GetConstArray(), &FloatAttributeData);
						}
					}
					else if (TypeStr == FString("Double"))
					{
						if (const TManagedArray<double>* AttributeArr = InCollection.FindAttribute<double>(AttributeNameVal, GroupNameVal))
						{
							SetValue(Context, AttributeArr->GetConstArray(), &DoubleAttributeData);
						}
					}
					else if (TypeStr == FString("Int32"))
					{
						if (const TManagedArray<int32>* AttributeArr = InCollection.FindAttribute<int32>(AttributeNameVal, GroupNameVal))
						{
							SetValue(Context, AttributeArr->GetConstArray(), &Int32AttributeData);
						}
					}
					else if (TypeStr == FString("String"))
					{
						if (const TManagedArray<FString>* AttributeArr = InCollection.FindAttribute<FString>(AttributeNameVal, GroupNameVal))
						{
							SetValue(Context, AttributeArr->GetConstArray(), &StringAttributeData);
						}
					}
					else if (TypeStr == FString("Vector"))
					{
						if (const TManagedArray<FVector3f>* AttributeArr = InCollection.FindAttribute<FVector3f>(AttributeNameVal, GroupNameVal))
						{
							SetValue(Context, AttributeArr->GetConstArray(), &Vector3fAttributeData);
						}
					}
					else if (TypeStr == FString("Vector3d"))
					{
						if (const TManagedArray<FVector3d>* AttributeArr = InCollection.FindAttribute<FVector3d>(AttributeNameVal, GroupNameVal))
						{
							SetValue(Context, AttributeArr->GetConstArray(), &Vector3dAttributeData);
						}
					}
					else if (TypeStr == FString("LinearColor"))
					{
						if (const TManagedArray<FLinearColor>* AttributeArr = InCollection.FindAttribute<FLinearColor>(AttributeNameVal, GroupNameVal))
						{
							SetValue(Context, AttributeArr->GetConstArray(), &LinearColorAttributeData);
						}
					}
				}
			}
		}
	}
}

void FGetCollectionAttributeDataTypedDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA(&BoolAttributeData) ||
		Out->IsA(&NumericArray) ||
		Out->IsA(&VectorArray) ||
		Out->IsA(&StringArray))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		
		FName InputGroupName;
		if (GroupName != EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Custom)
		{
			InputGroupName = GetGroupName(GroupName);
		}
		else
		{
			InputGroupName = FName(*CustomGroupName);
		}

		SetValue(Context, TArray<bool>(), &BoolAttributeData);
		SetValue(Context, TArray<double>(), &NumericArray);
		SetValue(Context, TArray<FVector4>(), &VectorArray);
		SetValue(Context, TArray<FString>(), &StringArray);

		FCollectionAttributeKey DefaultAttributeKey(AttrName, InputGroupName.ToString());
		FCollectionAttributeKey AttributeKeyVal = GetValue(Context, &AttributeKey, DefaultAttributeKey);
		FName GroupNameVal = FName(AttributeKeyVal.Group);
		FName AttributeNameVal = FName(AttributeKeyVal.Attribute);

		if (GroupNameVal.GetStringLength() > 0 && AttributeNameVal.GetStringLength() > 0)
		{
			if (InCollection.HasGroup(GroupNameVal))
			{
				if (InCollection.HasAttribute(AttributeNameVal, GroupNameVal))
				{
					FString TypeStr = GetArrayTypeString(InCollection.GetAttributeType(AttributeNameVal, GroupNameVal)).ToString();

					if (TypeStr == FString("Bool"))
					{
						if (const TManagedArray<bool>* AttributeArr = InCollection.FindAttribute<bool>(AttributeNameVal, GroupNameVal))
						{
							TArray<bool> BoolArray = AttributeArr->GetAsBoolArray();
							SetValue(Context, MoveTemp(BoolArray), &BoolAttributeData);
						}
					}
					else if (TypeStr == FString("Float"))
					{
						if (const TManagedArray<float>* AttributeArr = InCollection.FindAttribute<float>(AttributeNameVal, GroupNameVal))
						{
							TArray<double> Arr; Arr.Reserve(AttributeArr->Num());
							Algo::Copy(AttributeArr->GetConstArray(), Arr);

							SetValue(Context, MoveTemp(Arr), &NumericArray);
						}
					}
					else if (TypeStr == FString("Double"))
					{
						if (const TManagedArray<double>* AttributeArr = InCollection.FindAttribute<double>(AttributeNameVal, GroupNameVal))
						{
							SetValue(Context, AttributeArr->GetConstArray(), &NumericArray);
						}
					}
					else if (TypeStr == FString("Int32"))
					{
						if (const TManagedArray<int32>* AttributeArr = InCollection.FindAttribute<int32>(AttributeNameVal, GroupNameVal))
						{
							TArray<double> Arr; Arr.Reserve(AttributeArr->Num());
							Algo::Copy(AttributeArr->GetConstArray(), Arr);

							SetValue(Context, MoveTemp(Arr), &NumericArray);
						}
					}
					else if (TypeStr == FString("String"))
					{
						if (const TManagedArray<FString>* AttributeArr = InCollection.FindAttribute<FString>(AttributeNameVal, GroupNameVal))
						{
							SetValue(Context, AttributeArr->GetConstArray(), &StringArray);
						}
					}
					else if (TypeStr == FString("Vector"))
					{
						if (const TManagedArray<FVector3f>* AttributeArr = InCollection.FindAttribute<FVector3f>(AttributeNameVal, GroupNameVal))
						{
							TArray<FVector4> Arr; Arr.Reserve(AttributeArr->Num());
							for (int32 Idx = 0; Idx < AttributeArr->Num(); ++Idx)
							{
								const FVector3f Vec = AttributeArr->GetConstArray()[Idx];
								Arr[Idx] = FVector4(Vec.X, Vec.Y, Vec.Z, 0.0);
							}
							SetValue(Context, MoveTemp(Arr), &VectorArray);
						}
					}
					else if (TypeStr == FString("Vector3d"))
					{
						if (const TManagedArray<FVector3d>* AttributeArr = InCollection.FindAttribute<FVector3d>(AttributeNameVal, GroupNameVal))
						{
							TArray<FVector4> Arr; Arr.Reserve(AttributeArr->Num());
							for (int32 Idx = 0; Idx < AttributeArr->Num(); ++Idx)
							{
								const FVector3d Vec = AttributeArr->GetConstArray()[Idx];
								Arr[Idx] = FVector4(Vec.X, Vec.Y, Vec.Z, 0.0);
							}
							SetValue(Context, MoveTemp(Arr), &VectorArray);
						}
					}
					else if (TypeStr == FString("LinearColor"))
					{
						if (const TManagedArray<FLinearColor>* AttributeArr = InCollection.FindAttribute<FLinearColor>(AttributeNameVal, GroupNameVal))
						{
							TArray<FVector4> Arr; Arr.Reserve(AttributeArr->Num());
							for (int32 Idx = 0; Idx < AttributeArr->Num(); ++Idx)
							{
								const FLinearColor Vec = AttributeArr->GetConstArray()[Idx];
								Arr[Idx] = FVector4(Vec.R, Vec.G, Vec.B, Vec.A);
							}
							SetValue(Context, MoveTemp(Arr), &VectorArray);
						}
					}
				}
			}
		}
	}
}

template<typename T>
static void SetAttributeData(const FDataflowNode* DataflowNode, UE::Dataflow::FContext& Context, FManagedArrayCollection& InCollection, const TArray<T>& Property, FName AttributeName, FName GroupName)
{
	if (DataflowNode && DataflowNode->IsConnected<TArray<T>>(&Property))
	{
		const TArray<T> & AttributeData = DataflowNode->GetValue<TArray<T>>(Context, &Property);
		if (InCollection.FindAttributeTyped<T>(AttributeName, GroupName))
		{
			TManagedArray<T>& AttributeArray = InCollection.ModifyAttribute<T>(AttributeName, GroupName);

			if (AttributeData.Num() == AttributeArray.Num())
			{
				for (int32 Idx = 0; Idx < AttributeArray.Num(); ++Idx)
				{
					AttributeArray[Idx] = AttributeData[Idx];
				}
			}
		}
	}
}

void FSetCollectionAttributeDataTypedDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		FName InputGroupName;
		if (GroupName != EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Custom)
		{
			InputGroupName = GetGroupName(GroupName);
		}
		else
		{
			InputGroupName = FName(*CustomGroupName);
		}

		FCollectionAttributeKey DefaultAttributeKey(AttrName, InputGroupName.ToString());
		FCollectionAttributeKey AttributeKeyVal = GetValue(Context, &AttributeKey, DefaultAttributeKey);
		FName GroupNameVal = FName(AttributeKeyVal.Group);
		FName AttributeNameVal = FName(AttributeKeyVal.Attribute);

		if (GroupNameVal.GetStringLength() && AttributeNameVal.GetStringLength() )
		{
			if (InCollection.HasGroup(GroupNameVal))
			{
				if (InCollection.HasAttribute(AttributeNameVal, GroupNameVal))
				{
					FString TypeStr = GetArrayTypeString(InCollection.GetAttributeType(AttributeNameVal, GroupNameVal)).ToString();
					
					if (TypeStr == FString("Bool"))
					{
						SetAttributeData<bool>(this, Context, InCollection, BoolAttributeData, AttributeNameVal, GroupNameVal);
					}
					else if (TypeStr == FString("Float"))
					{
						SetAttributeData<float>(this, Context, InCollection, FloatAttributeData, AttributeNameVal, GroupNameVal);
					}
					else if (TypeStr == FString("Double"))
					{
						SetAttributeData<double>(this, Context, InCollection, DoubleAttributeData, AttributeNameVal, GroupNameVal);
					}
					else if (TypeStr == FString("Int32"))
					{
						SetAttributeData<int32>(this, Context, InCollection, Int32AttributeData, AttributeNameVal, GroupNameVal);
					}
					else if (TypeStr == FString("String"))
					{
						SetAttributeData<FString>(this, Context, InCollection, StringAttributeData, AttributeNameVal, GroupNameVal);
					}
					else if (TypeStr == FString("Vector"))
					{
						SetAttributeData<FVector3f>(this, Context, InCollection, Vector3fAttributeData, AttributeNameVal, GroupNameVal);
					}
					else if (TypeStr == FString("Vector3d"))
					{
						SetAttributeData<FVector3d>(this, Context, InCollection, Vector3dAttributeData, AttributeNameVal, GroupNameVal);
					}
					else if (TypeStr == FString("LinearColor"))
					{
						SetAttributeData<FLinearColor>(this, Context, InCollection, LinearColorAttributeData, AttributeNameVal, GroupNameVal);
					}
				}
			}
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

void FSelectionToVertexListDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	const FDataflowVertexSelection& InVertexSelection = GetValue<FDataflowVertexSelection>(Context, &VertexSelection);
	SetValue(Context, InVertexSelection.AsArray(), &VertexList);
}

void FMultiplyTransformDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FTransform>(&OutTransform))
	{
		SetValue(Context,
			GetValue<FTransform>(Context, &InLeftTransform, FTransform::Identity)
			*GetValue<FTransform>(Context, &InRightTransform, FTransform::Identity)
			, &OutTransform);
	}
}

void FInvertTransformDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FTransform>(&OutTransform))
	{
		const FTransform InXf = GetValue<FTransform>(Context, &InTransform, FTransform::Identity);
		const FTransform OutXf = InXf.Inverse();
		SetValue(Context, OutXf, &OutTransform);
	}
}

void FBranchFloatDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<float>(&ReturnValue))
	{
		bool InCondition = GetValue<bool>(Context, &bCondition);

		if (InCondition)
		{
			const float InA = GetValue<float>(Context, &A, A);

			SetValue(Context, InA, &ReturnValue);
		}
		else
		{
			const float InB = GetValue<float>(Context, &B, B);

			SetValue(Context, InB, &ReturnValue);
		}
	}
}

void FBranchIntDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<int32>(&ReturnValue))
	{
		bool InCondition = GetValue<bool>(Context, &bCondition);

		if (InCondition)
		{
			const int32 InA = GetValue<int32>(Context, &A, A);

			SetValue(Context, InA, &ReturnValue);
		}
		else
		{
			const int32 InB = GetValue<int32>(Context, &B, B);

			SetValue(Context, InB, &ReturnValue);
		}
	}
}

void FBoundingSphereDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&BoundingSphere))
	{
		if (IsConnected(&Collection))
		{
			const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

			GeometryCollection::Facades::FBoundsFacade BoundsFacade(InCollection);
			const FSphere& BoundingSphereInCollectionSpace = BoundsFacade.GetBoundingSphereInCollectionSpace();

			SetValue(Context, BoundingSphereInCollectionSpace, &BoundingSphere);
			return;
		}

		SetValue(Context, FSphere(), &BoundingSphere);
	}
}

void FExpandBoundingSphereDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	FSphere InSphere = GetValue(Context, &BoundingSphere);

	if (Out->IsA(&Center))
	{
		SetValue(Context, InSphere.Center, &Center);
	}
	else if (Out->IsA(&Radius))
	{
		SetValue(Context, (float)InSphere.W, &Radius);
	}
	else if (Out->IsA(&Volume))
	{
		SetValue(Context, (float)InSphere.GetVolume(), &Volume);
	}
}

void FVisualizeTetrahedronsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA(&Vertices))
	{
		if (IsConnected(&Collection))
		{
			FManagedArrayCollection InCollection = GetValue(Context, &Collection);

			if (InCollection.HasAttribute("Vertex", FGeometryCollection::VerticesGroup) &&
				InCollection.HasAttribute("TransformToGeometryIndex", FGeometryCollection::TransformGroup) &&
				InCollection.HasAttribute("VertexStart", FGeometryCollection::GeometryGroup) &&
				InCollection.HasAttribute("VertexCount", FGeometryCollection::GeometryGroup))
			{
				GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(InCollection);
				if (TransformFacade.IsValid())
				{
					TArray<FTransform> CollectionSpaceTransforms = TransformFacade.ComputeCollectionSpaceTransforms();

					const TManagedArray<FVector3f>& Vertex = InCollection.GetAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
					const TManagedArray<int32>& TransformToGeometryIndex = InCollection.GetAttribute<int32>("TransformToGeometryIndex", FGeometryCollection::TransformGroup);
					const TManagedArray<int32>& VertexStartArr = InCollection.GetAttribute<int32>("VertexStart", FGeometryCollection::GeometryGroup);
					const TManagedArray<int32>& VertexCountArr = InCollection.GetAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup);

					TArray<FVector> VerticesInCollectionSpace;
					VerticesInCollectionSpace.AddUninitialized(Vertex.Num());

					for (int32 TransformIndex = 0; TransformIndex < CollectionSpaceTransforms.Num(); ++TransformIndex)
					{
						const FTransform CollectionSpaceTransform = CollectionSpaceTransforms[TransformIndex];
						const int32 GeoIndex = TransformToGeometryIndex[TransformIndex];
						if (VertexStartArr.IsValidIndex(GeoIndex))
						{
							const int32 VertexStart = VertexStartArr[GeoIndex];
							const int32 VertexCount = VertexCountArr[GeoIndex];

							for (int32 VertexIdx = VertexStart; VertexIdx < VertexStart + VertexCount; ++VertexIdx)
							{
								VerticesInCollectionSpace[VertexIdx] = CollectionSpaceTransform.TransformPosition((FVector)Vertex[VertexIdx]);
							}
						}
					}

					SetValue(Context, MoveTemp(VerticesInCollectionSpace), &Vertices);
					return;
				}
			}
		}

		SetValue(Context, TArray<FVector>(), &Vertices);
	}
}

void FPointsToCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		if (IsConnected(&Collection) && IsConnected(&Points))
		{
			FManagedArrayCollection InCollection = GetValue(Context, &Collection);
			const TArray<FVector> InPoints = GetValue(Context, &Points);

			const int32 NumTransforms = InCollection.NumElements(FGeometryCollection::TransformGroup);
			const int32 NumGeoms = InCollection.NumElements(FGeometryCollection::GeometryGroup);
			const int32 NumVertices = InCollection.NumElements(FGeometryCollection::VerticesGroup);

			// Add new element to groups
			InCollection.AddElements(1, FGeometryCollection::TransformGroup);
			InCollection.AddElements(1, FGeometryCollection::GeometryGroup);
			InCollection.AddElements(InPoints.Num(), FGeometryCollection::VerticesGroup);

			TManagedArray<FTransform>& Transform = InCollection.AddAttribute<FTransform>("Transform", FGeometryCollection::TransformGroup);
			TManagedArray<FString>& BoneName = InCollection.AddAttribute<FString>("BoneName", FGeometryCollection::TransformGroup);
			TManagedArray<FLinearColor>& BoneColor = InCollection.AddAttribute<FLinearColor>("BoneColor", FGeometryCollection::TransformGroup);
			TManagedArray<int32>& Parent = InCollection.AddAttribute<int32>("Parent", FGeometryCollection::TransformGroup);
			TManagedArray<TSet<int32>>& Children = InCollection.AddAttribute<TSet<int32>>("Children", FGeometryCollection::TransformGroup);
			TManagedArray<int32>& TransformToGeometryIndex = InCollection.AddAttribute<int32>("TransformToGeometryIndex", FGeometryCollection::TransformGroup);
			TManagedArray<FVector3f>& Vertex = InCollection.AddAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
			TManagedArray<int32>& BoneMap = InCollection.AddAttribute<int32>("BoneMap", FGeometryCollection::VerticesGroup);
			TManagedArray<int32>& TransformIndex = InCollection.AddAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);
			TManagedArray<FBox>& BoundingBox = InCollection.AddAttribute<FBox>("BoundingBox", FGeometryCollection::GeometryGroup);
			TManagedArray<int32>& VertexStart = InCollection.AddAttribute<int32>("VertexStart", FGeometryCollection::GeometryGroup);
			TManagedArray<int32>& VertexCount = InCollection.AddAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup);

			Transform[NumTransforms] = FTransform::Identity;
			BoneName[NumTransforms] = FString(TEXT("Points"));
			BoneColor[NumTransforms] = FLinearColor(0.02f, 0.01f, 0.1f, 1.0f);
			Parent[NumTransforms] = -1;
			Children[NumTransforms] = TSet<int32>();
			TransformToGeometryIndex[NumTransforms] = NumGeoms;

			for (int32 VertexIdx = 0; VertexIdx < InPoints.Num(); ++VertexIdx)
			{
				Vertex[NumVertices + VertexIdx] = (FVector3f)InPoints[VertexIdx];
			}

			TransformIndex[NumGeoms] = NumTransforms;
			VertexStart[NumGeoms] = NumVertices;
			VertexCount[NumGeoms] = InPoints.Num();


			GeometryCollection::Facades::FBoundsFacade BoundsFacade(InCollection);
			const FBox& BoundingBoxOfPoints = BoundsFacade.ComputeBoundingBox(InPoints);

			BoundingBox[NumGeoms] = BoundingBoxOfPoints;

			SetValue(Context, MoveTemp(InCollection), &Collection);
			return;
		}

		SetValue(Context, FManagedArrayCollection(), &Collection);
	}
}

void FCollectionToPointsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA(&Points))
	{
		if (IsConnected(&Collection))
		{
			FManagedArrayCollection InCollection = GetValue(Context, &Collection);

			if (InCollection.HasAttribute("Vertex", FGeometryCollection::VerticesGroup) &&
				InCollection.HasAttribute("TransformToGeometryIndex", FGeometryCollection::TransformGroup) &&
				InCollection.HasAttribute("VertexStart", FGeometryCollection::GeometryGroup) &&
				InCollection.HasAttribute("VertexCount", FGeometryCollection::GeometryGroup))
			{
				GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(InCollection);
				if (TransformFacade.IsValid())
				{
					TArray<FTransform> CollectionSpaceTransforms = TransformFacade.ComputeCollectionSpaceTransforms();

					const TManagedArray<FVector3f>& Vertex = InCollection.GetAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
					const TManagedArray<int32>& TransformToGeometryIndex = InCollection.GetAttribute<int32>("TransformToGeometryIndex", FGeometryCollection::TransformGroup);
					const TManagedArray<int32>& VertexStartArr = InCollection.GetAttribute<int32>("VertexStart", FGeometryCollection::GeometryGroup);
					const TManagedArray<int32>& VertexCountArr = InCollection.GetAttribute<int32>("VertexCount", FGeometryCollection::GeometryGroup);

					TArray<FVector> VerticesInCollectionSpace;
					VerticesInCollectionSpace.AddUninitialized(Vertex.Num());

					for (int32 TransformIndex = 0; TransformIndex < CollectionSpaceTransforms.Num(); ++TransformIndex)
					{
						const FTransform CollectionSpaceTransform = CollectionSpaceTransforms[TransformIndex];
						const int32 GeoIndex = TransformToGeometryIndex[TransformIndex];
						const int32 VertexStart = VertexStartArr[GeoIndex];
						const int32 VertexCount = VertexCountArr[GeoIndex];

						for (int32 VertexIdx = VertexStart; VertexIdx < VertexStart + VertexCount; ++VertexIdx)
						{
							VerticesInCollectionSpace[VertexIdx] = CollectionSpaceTransform.TransformPosition((FVector)Vertex[VertexIdx]);
						}
					}

					SetValue(Context, MoveTemp(VerticesInCollectionSpace), &Points);
					return;
				}
			}
		}

		SetValue(Context, TArray<FVector>(), &Points);
	}
}

void FSpheresToPointsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Points) ||
		Out->IsA(&Radii))
	{
		if (IsConnected(&Spheres))
		{
			const TArray<FSphere>& InSpheres = GetValue(Context, &Spheres);

			const int32 NumSpheres = InSpheres.Num();

			if (NumSpheres > 0)
			{
				TArray<FVector> OutPoints; OutPoints.AddUninitialized(NumSpheres);
				TArray<float> OutRadii; OutRadii.AddUninitialized(NumSpheres);

				for (int32 Idx = 0; Idx < NumSpheres; ++Idx)
				{
					OutPoints[Idx] = InSpheres[Idx].Center;
					OutRadii[Idx] = InSpheres[Idx].W;
				}

				SetValue(Context, MoveTemp(OutPoints), &Points);
				SetValue(Context, MoveTemp(OutRadii), &Radii);
				return;
			}
		}

		SetValue(Context, TArray<FVector>(), &Points);
		SetValue(Context, TArray<float>(), &Radii);
	}
}

void FCollectionInfoDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA(&MaxLevel))
	{
		if (IsConnected(&Collection))
		{
			FManagedArrayCollection InCollection = GetValue(Context, &Collection);

			Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(InCollection);
			if (HierarchyFacade.IsValid())
			{
				if (!HierarchyFacade.HasLevelAttribute())
				{
					HierarchyFacade.GenerateLevelAttribute();
				}

				int32 MaxLevelAttr = HierarchyFacade.ComputeLevelAttributeMax();

				SetValue(Context, MaxLevelAttr, &MaxLevel);
				return;
			}
		}

		SetValue(Context, -1, &MaxLevel);
	}
}

