// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionNodesPlugin.h"

#include "CoreMinimal.h"

#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowCategoryRegistry.h"
#include "Dataflow/CreateColorArrayFromFloatArrayNode.h"
#include "Dataflow/GeometryCollectionArrayNodes.h"
#include "Dataflow/GeometryCollectionAssetNodes.h"
#include "Dataflow/GeometryCollectionClusteringNodes.h"
#include "Dataflow/GeometryCollectionConversionNodes.h"
#include "Dataflow/GeometryCollectionDebugNodes.h"
#include "Dataflow/GeometryCollectionEditNodes.h"
#include "Dataflow/GeometryCollectionFieldNodes.h"
#include "Dataflow/GeometryCollectionFracturingNodes.h"
#include "Dataflow/GeometryCollectionMakeNodes.h"
#include "Dataflow/GeometryCollectionMaterialNodes.h"
#include "Dataflow/GeometryCollectionMaterialInterfaceNodes.h"
#include "Dataflow/GeometryCollectionMathNodes.h"
#include "Dataflow/GeometryCollectionMeshNodes.h"
#include "Dataflow/GeometryCollectionNodes.h"
#include "Dataflow/GeometryCollectionOverrideNodes.h"
#include "Dataflow/GeometryCollectionProcessingNodes.h"
#include "Dataflow/GeometryCollectionSelectionNodes.h"
#include "Dataflow/GeometryCollectionSamplingNodes.h"
#include "Dataflow/GeometryCollectionRemoveMeshOverlapsNode.h"
#include "Dataflow/GeometryCollectionSkeletalMeshToCollectionNode.h"
#include "Dataflow/GeometryCollectionSkeletonToCollectionNode.h"
#include "Dataflow/GeometryCollectionStaticMeshToCollectionNode.h"
#include "Dataflow/GeometryCollectionTextureNodes.h"
#include "Dataflow/GeometryCollectionTransferVertexAttributeNode.h"
#include "Dataflow/GeometryCollectionTriangleBoundaryIndicesNode.h"
#include "Dataflow/GeometryCollectionUtilityNodes.h"
#include "Dataflow/GeometryCollectionUVNodes.h"
#include "Dataflow/GeometryCollectionVerticesNodes.h"
#include "Dataflow/GeometryCollectionVertexScalarToVertexIndicesNode.h"
#include "Dataflow/SetVertexColorFromFloatArrayNode.h"
#include "Dataflow/SetVertexColorFromVertexIndicesNode.h"
#include "Dataflow/SetVertexColorFromVertexSelectionNode.h"
#include "Dataflow/MeshMedialSkeletonConversionNodes.h"
#include "Dataflow/MeshToSkeletalMeshTerminalNode.h"
#include "Dataflow/MeshWeightEditingNodes.h"
#include "Dataflow/SubdivideMeshDataflowNodes.h"
#include "Dataflow/DataflowNodeAndPinTypeColors.h"
#include "Dataflow/DataflowCategoryRegistry.h"
#include "GeometryCollection/GeometryCollectionObject.h"

#define LOCTEXT_NAMESPACE "DataflowNodes"


void IGeometryCollectionNodesPlugin::StartupModule()
{
	UE::Dataflow::GeometryCollectionEngineNodes();
	UE::Dataflow::GeometryCollectionEngineAssetNodes();
	UE::Dataflow::GeometryCollectionProcessingNodes();
	UE::Dataflow::GeometryCollectionSelectionNodes();
	UE::Dataflow::GeometryCollectionMeshNodes();
	UE::Dataflow::GeometryCollectionClusteringNodes();
	UE::Dataflow::GeometryCollectionFracturingNodes();
	UE::Dataflow::GeometryCollectionEditNodes();
	UE::Dataflow::GeometryCollectionUtilityNodes();
	UE::Dataflow::GeometryCollectionMaterialNodes();
	UE::Dataflow::GeometryCollectionFieldNodes();
	UE::Dataflow::GeometryCollectionOverrideNodes();
	UE::Dataflow::GeometryCollectionMakeNodes();
	UE::Dataflow::GeometryCollectionMathNodes();
	UE::Dataflow::GeometryCollectionConversionNodes();
	UE::Dataflow::GeometryCollectionVerticesNodes();
	UE::Dataflow::GeometryCollectionArrayNodes();
	UE::Dataflow::GeometryCollectionDebugNodes();
	UE::Dataflow::GeometryCollectionSamplingNodes();
	UE::Dataflow::GeometryCollectionRemoveMeshOverlapsNode();
	UE::Dataflow::RegisterGeometryCollectionUVNodes();
	UE::Dataflow::RegisterGeometryCollectionTextureNodes();
	UE::Dataflow::RegisterGeometryCollectionMaterialInterfaceNodes();
	UE::Dataflow::RegisterGeometryStaticMeshToCollectionNodes();
	UE::Dataflow::RegisterMeshMedialSkeletonConversionNodes();
	UE::Dataflow::RegisterMeshToSkeletalMeshTerminalNode();
	UE::Dataflow::RegisterMeshWeightEditingNodes();
	UE::Dataflow::RegisterSubdivideMeshDataflowNodes();
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCreateColorArrayFromFloatArrayDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGeometryCollectionVertexScalarToVertexIndicesNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetVertexColorFromFloatArrayDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetVertexColorFromVertexIndicesDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetVertexColorFromVertexSelectionDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSkeletalMeshToCollectionDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionToSkeletalMeshDataflowNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGeometryCollectionTransferVertexAttributeNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGeometryCollectionTransferVertexSkinWeightsNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGeometryCollectionSetKinematicVertexSelectionNode);
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FTriangleBoundaryIndicesNode);
	// Deprecated nodes
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSkeletonToCollectionDataflowNode);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Register Dataflow node and PinType colors
	UE::Dataflow::RegisterGeometryCollectionNodesColors();

	// register category for GeometryCollection asset
	UE_DATAFLOW_REGISTER_CATEGORY_FORASSET_TYPE("GeometryCollection", UGeometryCollection);
}

void IGeometryCollectionNodesPlugin::ShutdownModule()
{
}


IMPLEMENT_MODULE(IGeometryCollectionNodesPlugin, GeometryCollectionNodes)


#undef LOCTEXT_NAMESPACE
