// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionOverrideNodes.h"
#include "Dataflow/DataflowCore.h"

#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionOverrideNodes)

namespace UE::Dataflow
{
	void GeometryCollectionOverrideNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetFloatOverrideFromAssetDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetIntOverrideFromAssetDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetBoolOverrideFromAssetDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetStringOverrideFromAssetDataflowNode);
	}
}


void FGetFloatOverrideFromAssetDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Float) || Out->IsA(&FloatDefault) || Out->IsA(&IsOverriden))
	{
		const float DefaultValue = GetDefaultValue<float>(Context);
		float NewValue = DefaultValue;
		bool bIsOverriden = false;

		if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
		{
			const FString ValueFromAsset = GetValueFromAsset(Context, EngineContext->Owner);

			if (!ValueFromAsset.IsEmpty() && ValueFromAsset.IsNumeric())
			{
				NewValue = FCString::Atof(*ValueFromAsset);
				bIsOverriden = true;
			}
		}

		SetValue(Context, NewValue, &Float);
		SetValue(Context, DefaultValue, &FloatDefault);
		SetValue(Context, bIsOverriden, &IsOverriden);
	}
}

void FGetIntOverrideFromAssetDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Int) || Out->IsA(&IntDefault) || Out->IsA(&IsOverriden))
	{
		const int32 DefaultValue = GetDefaultValue<int32>(Context);
		int32 NewValue = DefaultValue;
		bool bIsOverriden = false;

		if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
		{
			const FString ValueFromAsset = GetValueFromAsset(Context, EngineContext->Owner);

			if (!ValueFromAsset.IsEmpty() && ValueFromAsset.IsNumeric())
			{
				NewValue = FCString::Atoi(*ValueFromAsset);
				bIsOverriden = true;
			}
		}

		SetValue(Context, NewValue, &Int);
		SetValue(Context, DefaultValue, &IntDefault);
		SetValue(Context, bIsOverriden, &IsOverriden);
	}
}

static bool StringToBool(const FString& InString, bool InDefault)
{
	bool Result = InDefault;

	if (!InString.IsEmpty())
	{
		if (InString.IsNumeric())
		{
			Result = FCString::Atoi(*InString) == 0 ? false : true;
		}
		else
		{
			Result = !InString.Compare("false") ? false : !InString.Compare("true") ? true : InDefault;
		}
	}

	return Result;
}

void FGetBoolOverrideFromAssetDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Bool) || Out->IsA(&BoolDefault) || Out->IsA(&IsOverriden))
	{
		const bool DefaultValue = StringToBool(GetDefaultValue<FString>(Context), false);
		bool NewValue = DefaultValue;
		bool bIsOverriden = false;

		if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
		{
			const FString ValueFromAsset = GetValueFromAsset(Context, EngineContext->Owner);

			if (!ValueFromAsset.IsEmpty())
			{
				NewValue = StringToBool(ValueFromAsset, false);
				bIsOverriden = true;
			}
		}

		SetValue(Context, NewValue, &Bool);
		SetValue(Context, DefaultValue, &BoolDefault);
		SetValue(Context, bIsOverriden, &IsOverriden);
	}
}

void FGetStringOverrideFromAssetDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&String) || Out->IsA(&StringDefault) || Out->IsA(&IsOverriden))
	{
		const FString DefaultValue = GetDefaultValue<FString>(Context);
		FString NewValue = DefaultValue;
		bool bIsOverriden = false;

		if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
		{
			const FString ValueFromAsset = GetValueFromAsset(Context, EngineContext->Owner);

			if (!ValueFromAsset.IsEmpty())
			{
				NewValue = ValueFromAsset;
				bIsOverriden = true;
			}
		}

		SetValue(Context, NewValue, &String);
		SetValue(Context, DefaultValue, &StringDefault);
		SetValue(Context, bIsOverriden, &IsOverriden);
	}
}
