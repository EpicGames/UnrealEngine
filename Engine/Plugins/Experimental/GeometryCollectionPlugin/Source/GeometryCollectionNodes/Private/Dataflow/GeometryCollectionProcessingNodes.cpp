// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionProcessingNodes.h"

#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionProcessingNodes)


namespace UE::Dataflow
{
	void GeometryCollectionProcessingNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCloseGeometryOnCollectionDataflowNode);
	}
}

void FCloseGeometryOnCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection))
	{
		DataType InCollection = GetValue<DataType>(Context, &Collection);

		//
		// @todo(dataflow) : Implemention that closes the geometry on the collection
		//

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}





