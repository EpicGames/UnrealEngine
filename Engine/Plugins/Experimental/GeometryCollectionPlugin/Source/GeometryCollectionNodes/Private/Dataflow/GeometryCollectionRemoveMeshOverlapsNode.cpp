// Copyright Epic Games, Inc. All Rights Reserved.


#include "Dataflow/GeometryCollectionRemoveMeshOverlapsNode.h"
#include "Dataflow/DataflowCore.h"

#include "Algo/Reverse.h"
#include "Operations/MeshBoolean.h"
#include "DynamicMesh/MeshTransforms.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionRemoveMeshOverlapsNode)

namespace UE::Dataflow
{

	void GeometryCollectionRemoveMeshOverlapsNode()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRemoveMeshOverlapsDataflowNode);
	}
}


FRemoveMeshOverlapsDataflowNode::FRemoveMeshOverlapsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&DynamicMeshes);
	RegisterOutputConnection(&DynamicMeshes, &DynamicMeshes);
	RegisterInputConnection(&PerMeshTransforms).SetCanHidePin(true).SetPinIsHidden(true);
}

void FRemoveMeshOverlapsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;
	
	if (Out->IsA(&DynamicMeshes) ||
		Out->IsA(&PerMeshTransforms))
	{
		const TArray<TObjectPtr<UDynamicMesh>>& InMeshes = GetValue(Context, &DynamicMeshes);
		const TArray<FTransform>& InPerMeshTransforms = GetValue(Context, &PerMeshTransforms);

		TArray<TObjectPtr<UDynamicMesh>> ResultMeshes;
		TArray<UE::Geometry::FAxisAlignedBox3d> MeshBounds;

		// Define ordering to apply boolean subtraction operations
		TArray<int32> Ordering = [&InMeshes, &ResultMeshes, this]() -> TArray<int32>
		{
			TArray<int32> Indices;
			Indices.SetNumUninitialized(InMeshes.Num());
			for (int32 Idx = 0; Idx < InMeshes.Num(); ++Idx)
			{
				Indices[Idx] = Idx;
			}
			bool bVolumeOrder = false, bReverseOrder = false;
			switch (SubtractOrder)
			{
			case EDataflowRemoveOverlapsMeshSortOrder::IncreasingVolume:
				bVolumeOrder = true;
				break;
			case EDataflowRemoveOverlapsMeshSortOrder::DecreasingVolume:
				bReverseOrder = bVolumeOrder = true;
				break;
			case EDataflowRemoveOverlapsMeshSortOrder::ReverseInputOrder:
				bReverseOrder = true;
				break;
			case EDataflowRemoveOverlapsMeshSortOrder::InputOrder:
				// defaults apply
				break;
			}
			if (bVolumeOrder)
			{
				TArray<float> Values;
				Values.SetNumUninitialized(InMeshes.Num());
				for (int32 Idx = 0; Idx < InMeshes.Num(); ++Idx)
				{
					float Value = 0;
					InMeshes[Idx]->ProcessMesh([&Value](const FDynamicMesh3& Mesh) -> void { Value = TMeshQueries<FDynamicMesh3>::GetVolumeNonWatertight(Mesh); });
					Values[Idx] = Value;
				}
				Indices.Sort([&Values](int32 A, int32 B) { return Values[A] < Values[B]; });
			}
			if (bReverseOrder)
			{
				Algo::Reverse(Indices);
			}
			return Indices;
		}();

		// Copy initial meshes to results array, bake transforms, and compute initial bounds
		ResultMeshes.SetNum(InMeshes.Num());
		MeshBounds.Init(UE::Geometry::FAxisAlignedBox3d::Empty(), InMeshes.Num());
		for (int32 MeshIdx = 0; MeshIdx < InMeshes.Num(); ++MeshIdx)
		{
			ResultMeshes[MeshIdx] = NewObject<UDynamicMesh>();
			ResultMeshes[MeshIdx]->EditMesh([&InMeshes, &InPerMeshTransforms, &MeshBounds, MeshIdx](FDynamicMesh3& Mesh) -> void
				{
					InMeshes[MeshIdx]->ProcessMesh([&Mesh](const FDynamicMesh3& InMesh) -> void
					{
						Mesh = InMesh;
					});
					if (InPerMeshTransforms.IsValidIndex(MeshIdx))
					{
						MeshTransforms::ApplyTransform(Mesh, InPerMeshTransforms[MeshIdx]);
					}
					MeshBounds[MeshIdx] = Mesh.GetBounds();
				});
		}
		// Compute mesh boolean subtractions to remove overlaps
		if (InMeshes.Num() > 1)
		{
			ParallelFor(InMeshes.Num() - 1, 
			[&Ordering, &ResultMeshes, &MeshBounds, &InMeshes, &InPerMeshTransforms, this](int32 FromOrderingIdx)
			{
				const int32 FromMeshIdx = Ordering[FromOrderingIdx];
				UE::Geometry::FDynamicMesh3& ToTrim = ResultMeshes[FromMeshIdx]->GetMeshRef();
				for (int32 VsOrderingIdx = FromOrderingIdx + 1; VsOrderingIdx < InMeshes.Num(); ++VsOrderingIdx)
				{
					const int32 VsMeshIdx = Ordering[VsOrderingIdx];
					if (MeshBounds[FromMeshIdx].Intersects(MeshBounds[VsMeshIdx]))
					{
						FTransform VsTransform = InPerMeshTransforms.IsValidIndex(VsMeshIdx) ? InPerMeshTransforms[VsMeshIdx] : FTransform::Identity;
						UE::Geometry::FMeshBoolean Boolean(
							&ToTrim, FTransform::Identity, 
							InMeshes[VsMeshIdx]->GetMeshPtr(), VsTransform,
							&ToTrim, UE::Geometry::FMeshBoolean::EBooleanOp::Difference);
						Boolean.bSimplifyAlongNewEdges = bSimplifyNewMeshEdges;
						Boolean.Compute();
					}
				}
			}, EParallelForFlags::Unbalanced);
		}

		SetValue(Context, ResultMeshes, &DynamicMeshes);
	}
}
