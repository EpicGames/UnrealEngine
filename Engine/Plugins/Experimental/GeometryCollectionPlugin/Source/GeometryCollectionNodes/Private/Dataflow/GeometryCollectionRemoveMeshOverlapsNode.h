// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowEngine.h"

#include "GeometryCollectionRemoveMeshOverlapsNode.generated.h"

class FGeometryCollection;
class UDynamicMesh;

// The mesh ordering for used when removing overlaps
UENUM()
enum class EDataflowRemoveOverlapsMeshSortOrder
{
	// Process in order of increasing volume; largest is unchanged
	IncreasingVolume,
	// Process in order of decreasing volume; smallest is unchanged
	DecreasingVolume,
	// Process in input ordering; last mesh is unchanged
	InputOrder,
	// Process in the reverse of the input ordering; first mesh is unchanged
	ReverseInputOrder
};

/**
 * Cut away any overlaps between input meshes, creating a set of non-overlapping meshes as output
 * For example, given meshes w/ order [A, B, C], non-overlapping meshes will be [(A-B)-C, B-C, C]
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FRemoveMeshOverlapsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRemoveMeshOverlapsDataflowNode, "RemoveMeshOverlaps", "GeometryCollection|Fracture", "")

public:
	FRemoveMeshOverlapsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	/** Dynamic Meshes to remove overlaps from */
	UPROPERTY(EditAnywhere, Category = "Meshes", meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "DynamicMeshes", DataflowIntrinsic))
	TArray<TObjectPtr<UDynamicMesh>> DynamicMeshes;

	/** Optional transforms for input meshes. Will be baked into output. If not provided, meshes will not be transformed. */
	UPROPERTY(EditAnywhere, Category = "Meshes", meta = (DataflowInput))
	TArray<FTransform> PerMeshTransforms;

	/** Ordering to use when applying subtraction operations. Later meshes will be subtracted from earlier meshes in the ordering. */
	UPROPERTY(EditAnywhere, Category = "Ordering")
	EDataflowRemoveOverlapsMeshSortOrder SubtractOrder = EDataflowRemoveOverlapsMeshSortOrder::InputOrder;

	/** Whether to simplify along new edges created by the mesh subtraction operations */
	UPROPERTY(EditAnywhere, Category = "Boolean")
	bool bSimplifyNewMeshEdges = true;
};

namespace UE::Dataflow
{
	void GeometryCollectionRemoveMeshOverlapsNode();
}

