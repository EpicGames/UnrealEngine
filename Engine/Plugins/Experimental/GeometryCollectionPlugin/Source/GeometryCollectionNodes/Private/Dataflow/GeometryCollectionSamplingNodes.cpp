// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionSamplingNodes.h"

#include "UDynamicMesh.h"
#include "Spatial/FastWinding.h"

#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowDebugDraw.h"
#if WITH_EDITOR
#include "Dataflow/DataflowRenderingViewMode.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionSamplingNodes)

#define LOCTEXT_NAMESPACE "GeometryCollectionSamplingNodes"

namespace UE::Dataflow
{
	void GeometryCollectionSamplingNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFilterPointSetWithMeshDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FUniformPointSamplingDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FNonUniformPointSamplingDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVertexWeightedPointSamplingDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMeshMedialSkeletonSamplingDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSimplifyMedialSkeletonDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSubdivideMedialSkeletonDataflowNode_v2);

		// Deprecated
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFilterPointSetWithMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FUniformPointSamplingDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FNonUniformPointSamplingDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FVertexWeightedPointSamplingDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMeshMedialSkeletonSamplingDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSimplifyMedialSkeletonDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSubdivideMedialSkeletonDataflowNode);
	}
}

// -------------------------------------------------------------------------------------------------------------------------------------------------

FFilterPointSetWithMeshDataflowNode::FFilterPointSetWithMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&TargetMesh);
	RegisterInputConnection(&SamplePoints);
	RegisterInputConnection(&bKeepInside).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&WindingThreshold).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinDistance).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxDistance).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bUseSignedDistance).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&SamplePoints);
}

void FFilterPointSetWithMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&SamplePoints))
	{
		TArray<FVector> InSamplePoints = GetValue(Context, &SamplePoints);
		const double InWindingThreshold = (double)GetValue(Context, &WindingThreshold);
		const bool bInKeepInside = GetValue(Context, &bKeepInside);
		const double InMinDistance = (double)GetValue(Context, &MinDistance);
		const double InMaxDistance = (double)GetValue(Context, &MaxDistance);
		const bool bInUseSignedDistance = GetValue(Context, &bUseSignedDistance);

		if (TObjectPtr<const UDynamicMesh> InTargetMesh = GetValue(Context, &TargetMesh))
		{
			using namespace UE::Geometry;
			InTargetMesh->ProcessMesh(
				[&InSamplePoints, InWindingThreshold, bInKeepInside, InMinDistance, InMaxDistance, bInUseSignedDistance, this]
				(const FDynamicMesh3& Mesh)
				{
					const bool bFilterWinding = bool(FilterMethod & (uint8)EFilterPointSetWithMeshDataflowMethodFlags::Winding);
					const bool bFilterMinDist = bool(FilterMethod & (uint8)EFilterPointSetWithMeshDataflowMethodFlags::MinDistance);
					const bool bFilterMaxDist = bool(FilterMethod & (uint8)EFilterPointSetWithMeshDataflowMethodFlags::MaxDistance);

					if (!bFilterWinding && !bFilterMinDist && !bFilterMaxDist)
					{
						// No filtering, early-out
						return;
					}

					// Clamp unsigned distances to 0
					double UseMinDistance = InMinDistance;
					double UseMaxDistance = InMaxDistance;
					if (!bInUseSignedDistance)
					{
						UseMinDistance = FMath::Max(0, UseMinDistance);
						UseMaxDistance = FMath::Max(0, UseMaxDistance);
					}

					if (bFilterMinDist && bFilterMaxDist)
					{
						// If Min > Max, impossible for points to pass the filter, so early-out
						if (UseMinDistance > UseMaxDistance)
						{
							InSamplePoints.Empty();
							return;
						}
					}

					const bool bNeedWinding = bFilterWinding || bInUseSignedDistance;
					const bool bNeedDistance = bFilterMinDist || bFilterMaxDist;
					const double MaxRelevantDistance = UE_DOUBLE_KINDA_SMALL_NUMBER + FMath::Max(
						bFilterMinDist ? FMath::Abs(UseMinDistance) : 0,
						bFilterMaxDist ? FMath::Abs(UseMaxDistance) : 0);

					//  set up AABBTree and FWNTree lists
					TMeshAABBTree3<FDynamicMesh3> Spatial(&Mesh);
					TFastWindingTree<FDynamicMesh3> FastWinding(&Spatial, bNeedWinding);

					// square threshold distances, but keep their signs, so we can do threshold testing w/out sqrt
					const double UseSignSqMinDist = FMath::CopySign(UseMinDistance * UseMinDistance, UseMinDistance);
					const double UseSignSqMaxDist = FMath::CopySign(UseMaxDistance * UseMaxDistance, UseMaxDistance);

					// Filter points 
					TArray<bool> KeepPoint;
					KeepPoint.SetNumUninitialized(InSamplePoints.Num());
					ParallelFor(KeepPoint.Num(),
						[&KeepPoint, &Spatial, &FastWinding, &InSamplePoints, 
						bFilterWinding, bFilterMinDist, bFilterMaxDist, bInUseSignedDistance, bInKeepInside,
						bNeedWinding, InWindingThreshold, bNeedDistance, UseSignSqMinDist, UseSignSqMaxDist, MaxRelevantDistance]
						(int32 PointIdx)
						{
							FVector Pt = InSamplePoints[PointIdx];

							bool bWindingInside = false;
							if (bNeedWinding)
							{
								bWindingInside = FastWinding.IsInside(Pt, InWindingThreshold);
								// test if we fail the winding filter
								if (bFilterWinding && bInKeepInside != bWindingInside)
								{
									KeepPoint[PointIdx] = false;
									return;
								}
							}
							double FoundDistSq = 0;
							if (bNeedDistance)
							{
								const int32 FoundTID = Spatial.FindNearestTriangle(Pt, FoundDistSq, UE::Geometry::IMeshSpatial::FQueryOptions(MaxRelevantDistance));
								if (FoundTID == INDEX_NONE)
								{
									// we have a max dist, lack of closest point -> we fail the max dist filter
									// or it's inside the shape w/ signed distances -> it's too far inside, fail the min filter
									if (bFilterMaxDist || (bInUseSignedDistance && bWindingInside))
									{
										KeepPoint[PointIdx] = false;
										return;
									}
									else
									{
										// point at least passes the min distance threshold
										FoundDistSq = UseSignSqMinDist;
									}
								}
								else
								{
									if (bInUseSignedDistance && bWindingInside)
									{
										// sign the squared distance
										FoundDistSq = -FoundDistSq;
									}
								}
								if (bFilterMinDist && FoundDistSq < UseSignSqMinDist)
								{
									KeepPoint[PointIdx] = false;
									return;
								}
								else if (bFilterMaxDist && FoundDistSq > UseSignSqMaxDist)
								{
									KeepPoint[PointIdx] = false;
									return;
								}
							}

							// passed all filters, keep the point
							KeepPoint[PointIdx] = true;
						}
					);

					// Move the points we're keeping to the front of the array, and trim the array
					int32 FoundPoints = 0;
					for (int32 Idx = 0; ; ++Idx)
					{
						while (Idx < InSamplePoints.Num() && !KeepPoint[Idx])
						{
							Idx++;
						}
						if (Idx < InSamplePoints.Num())
						{
							if (Idx != FoundPoints)
							{
								InSamplePoints[FoundPoints] = InSamplePoints[Idx];
							}
							FoundPoints++;
						}
						else
						{
							break;
						}
					}
					InSamplePoints.SetNum(FoundPoints);
				}
			);
		}
		SetValue(Context, MoveTemp(InSamplePoints), &SamplePoints);
	}
}

// -------------------------------------------------------------------------------------------------------------------------------------------------

FFilterPointSetWithMeshDataflowNode_v2::FFilterPointSetWithMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&TargetMesh);
	RegisterInputConnection(&SamplePoints);
	RegisterInputConnection(&bKeepInside).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&WindingThreshold).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinDistance).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxDistance).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bUseSignedDistance).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&SamplePoints);
}

void FFilterPointSetWithMeshDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&SamplePoints))
	{
		FDataflowPoints InSamplePoints = GetValue(Context, &SamplePoints);
		GeometryCollection::Facades::FPointsFacade PointFacade = InSamplePoints.GetPointsFacade();

		TArray<FVector> InSamplePointsArr = PointFacade.GetPointsAsArray();

		const double InWindingThreshold = (double)GetValue(Context, &WindingThreshold);
		const bool bInKeepInside = GetValue(Context, &bKeepInside);
		const double InMinDistance = (double)GetValue(Context, &MinDistance);
		const double InMaxDistance = (double)GetValue(Context, &MaxDistance);
		const bool bInUseSignedDistance = GetValue(Context, &bUseSignedDistance);

		if (TObjectPtr<const UDataflowMesh> InTargetDataflowMesh = GetValue(Context, &TargetMesh))
		{
			if (const FDynamicMesh3* DynMesh = InTargetDataflowMesh->GetDynamicMesh())
			{
				TObjectPtr<UDynamicMesh> InTargetMesh = NewObject<UDynamicMesh>();
				InTargetMesh->SetMesh(*DynMesh);

				if (InTargetMesh)
				{
					using namespace UE::Geometry;
					InTargetMesh->ProcessMesh(
						[&InSamplePointsArr, InWindingThreshold, bInKeepInside, InMinDistance, InMaxDistance, bInUseSignedDistance, this]
						(const FDynamicMesh3& Mesh)
						{
							const bool bFilterWinding = bool(FilterMethod & (uint8)EFilterPointSetWithMeshDataflowMethodFlags::Winding);
							const bool bFilterMinDist = bool(FilterMethod & (uint8)EFilterPointSetWithMeshDataflowMethodFlags::MinDistance);
							const bool bFilterMaxDist = bool(FilterMethod & (uint8)EFilterPointSetWithMeshDataflowMethodFlags::MaxDistance);

							if (!bFilterWinding && !bFilterMinDist && !bFilterMaxDist)
							{
								// No filtering, early-out
								return;
							}

							// Clamp unsigned distances to 0
							double UseMinDistance = InMinDistance;
							double UseMaxDistance = InMaxDistance;
							if (!bInUseSignedDistance)
							{
								UseMinDistance = FMath::Max(0, UseMinDistance);
								UseMaxDistance = FMath::Max(0, UseMaxDistance);
							}

							if (bFilterMinDist && bFilterMaxDist)
							{
								// If Min > Max, impossible for points to pass the filter, so early-out
								if (UseMinDistance > UseMaxDistance)
								{
									InSamplePointsArr.Empty();
									return;
								}
							}

							const bool bNeedWinding = bFilterWinding || bInUseSignedDistance;
							const bool bNeedDistance = bFilterMinDist || bFilterMaxDist;
							const double MaxRelevantDistance = UE_DOUBLE_KINDA_SMALL_NUMBER + FMath::Max(
								bFilterMinDist ? FMath::Abs(UseMinDistance) : 0,
								bFilterMaxDist ? FMath::Abs(UseMaxDistance) : 0);

							//  set up AABBTree and FWNTree lists
							TMeshAABBTree3<FDynamicMesh3> Spatial(&Mesh);
							TFastWindingTree<FDynamicMesh3> FastWinding(&Spatial, bNeedWinding);

							// square threshold distances, but keep their signs, so we can do threshold testing w/out sqrt
							const double UseSignSqMinDist = FMath::CopySign(UseMinDistance * UseMinDistance, UseMinDistance);
							const double UseSignSqMaxDist = FMath::CopySign(UseMaxDistance * UseMaxDistance, UseMaxDistance);

							// Filter points 
							TArray<bool> KeepPoint;
							KeepPoint.SetNumUninitialized(InSamplePointsArr.Num());
							ParallelFor(KeepPoint.Num(),
								[&KeepPoint, &Spatial, &FastWinding, &InSamplePointsArr,
								bFilterWinding, bFilterMinDist, bFilterMaxDist, bInUseSignedDistance, bInKeepInside,
								bNeedWinding, InWindingThreshold, bNeedDistance, UseSignSqMinDist, UseSignSqMaxDist, MaxRelevantDistance]
								(int32 PointIdx)
								{
									FVector Pt = InSamplePointsArr[PointIdx];

									bool bWindingInside = false;
									if (bNeedWinding)
									{
										bWindingInside = FastWinding.IsInside(Pt, InWindingThreshold);
										// test if we fail the winding filter
										if (bFilterWinding && bInKeepInside != bWindingInside)
										{
											KeepPoint[PointIdx] = false;
											return;
										}
									}
									double FoundDistSq = 0;
									if (bNeedDistance)
									{
										const int32 FoundTID = Spatial.FindNearestTriangle(Pt, FoundDistSq, UE::Geometry::IMeshSpatial::FQueryOptions(MaxRelevantDistance));
										if (FoundTID == INDEX_NONE)
										{
											// we have a max dist, lack of closest point -> we fail the max dist filter
											// or it's inside the shape w/ signed distances -> it's too far inside, fail the min filter
											if (bFilterMaxDist || (bInUseSignedDistance && bWindingInside))
											{
												KeepPoint[PointIdx] = false;
												return;
											}
											else
											{
												// point at least passes the min distance threshold
												FoundDistSq = UseSignSqMinDist;
											}
										}
										else
										{
											if (bInUseSignedDistance && bWindingInside)
											{
												// sign the squared distance
												FoundDistSq = -FoundDistSq;
											}
										}
										if (bFilterMinDist && FoundDistSq < UseSignSqMinDist)
										{
											KeepPoint[PointIdx] = false;
											return;
										}
										else if (bFilterMaxDist && FoundDistSq > UseSignSqMaxDist)
										{
											KeepPoint[PointIdx] = false;
											return;
										}
									}

									// passed all filters, keep the point
									KeepPoint[PointIdx] = true;
								}
							);

							// Move the points we're keeping to the front of the array, and trim the array
							int32 FoundPoints = 0;
							for (int32 Idx = 0; ; ++Idx)
							{
								while (Idx < InSamplePointsArr.Num() && !KeepPoint[Idx])
								{
									Idx++;
								}
								if (Idx < InSamplePointsArr.Num())
								{
									if (Idx != FoundPoints)
									{
										InSamplePointsArr[FoundPoints] = InSamplePointsArr[Idx];
									}
									FoundPoints++;
								}
								else
								{
									break;
								}
							}

							InSamplePointsArr.SetNum(FoundPoints);
						}
					);
				}
			}
		}

		FDataflowPoints OutPoints;
		GeometryCollection::Facades::FPointsFacade PointFacadeOutPoints = OutPoints.GetPointsFacade();
		PointFacadeOutPoints.AddPoints(InSamplePointsArr);

		SetValue(Context, MoveTemp(OutPoints), &SamplePoints);
	}
}

// -------------------------------------------------------------------------------------------------------------------------------------------------

void FUniformPointSamplingDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&SamplePoints) || 
		Out->IsA(&SampleTriangleIDs) ||
		Out->IsA(&SampleBarycentricCoords) ||
		Out->IsA(&NumSamplePoints))
	{
		TArray<FVector> OutPoints;
		TArray<FTransform> OutSamples;
		TArray<int32> OutTriangleIDs;
		TArray<FVector> OutBarycentricCoords;

		if (TObjectPtr<UDynamicMesh> InTargetMesh = GetValue(Context, &TargetMesh))
		{
			const UE::Geometry::FDynamicMesh3& InDynTargetMesh = InTargetMesh->GetMeshRef();

			if (InDynTargetMesh.VertexCount() > 0)
			{
				FFractureEngineSampling::ComputeUniformPointSampling(InDynTargetMesh,
					GetValue(Context, &SamplingRadius),
					GetValue(Context, &MaxNumSamples),
					GetValue(Context, &SubSampleDensity),
					GetValue(Context, &RandomSeed),
					OutSamples, 
					OutTriangleIDs, 
					OutBarycentricCoords);

				const int32 NumSamples = OutSamples.Num();

				OutPoints.AddUninitialized(NumSamples);

				for (int32 Idx = 0; Idx < NumSamples; ++Idx)
				{
					OutPoints[Idx] = OutSamples[Idx].GetTranslation();
				}
			}
		}

		SetValue(Context, MoveTemp(OutPoints), &SamplePoints);
		SetValue(Context, MoveTemp(OutTriangleIDs), &SampleTriangleIDs);
		SetValue(Context, MoveTemp(OutBarycentricCoords), &SampleBarycentricCoords);
		SetValue(Context, OutPoints.Num(), &NumSamplePoints);
	}
}

// -------------------------------------------------------------------------------------------------------------------------------------------------

FUniformPointSamplingDataflowNode_v2::FUniformPointSamplingDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&TargetMesh);
	RegisterInputConnection(&SamplingRadius).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxNumSamples).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SubSampleDensity).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RandomSeed).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&SamplePoints);
	RegisterOutputConnection(&SampleTriangleIDs);
	RegisterOutputConnection(&SampleBarycentricCoords);
	RegisterOutputConnection(&NumSamplePoints);
}

void FUniformPointSamplingDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&SamplePoints) ||
		Out->IsA(&SampleTriangleIDs) ||
		Out->IsA(&SampleBarycentricCoords) ||
		Out->IsA(&NumSamplePoints))
	{
		TArray<FVector> OutPoints;
		TArray<FTransform> OutSamples;
		TArray<int32> OutTriangleIDs;
		TArray<FVector> OutBarycentricCoords;

		if (TObjectPtr<const UDataflowMesh> InTargetDataflowMesh = GetValue(Context, &TargetMesh))
		{
			const FDynamicMesh3& DynMesh = InTargetDataflowMesh->GetDynamicMeshRef();

			const float InSamplingRadius = GetValue(Context, &SamplingRadius);
			const int32 InMaxNumSamples = GetValue(Context, &MaxNumSamples);
			const float InSubSampleDensity = GetValue(Context, &SubSampleDensity);
			const int32 InRandomSeed = GetValue(Context, &RandomSeed);

			if (DynMesh.VertexCount() > 0)
			{
				FFractureEngineSampling::ComputeUniformPointSampling(DynMesh,
					InSamplingRadius,
					InMaxNumSamples,
					InSubSampleDensity,
					InRandomSeed,
					OutSamples,
					OutTriangleIDs,
					OutBarycentricCoords);

				const int32 NumSamples = OutSamples.Num();

				OutPoints.AddUninitialized(NumSamples);

				for (int32 Idx = 0; Idx < NumSamples; ++Idx)
				{
					OutPoints[Idx] = OutSamples[Idx].GetTranslation();
				}
			}
		}

		FDataflowPoints NewPoints;
		GeometryCollection::Facades::FPointsFacade PointFacade = NewPoints.GetPointsFacade();
		PointFacade.AddPoints(OutPoints);

		SetValue(Context, MoveTemp(NewPoints), &SamplePoints);
		SetValue(Context, MoveTemp(OutTriangleIDs), &SampleTriangleIDs);
		SetValue(Context, MoveTemp(OutBarycentricCoords), &SampleBarycentricCoords);
		SetValue(Context, OutPoints.Num(), &NumSamplePoints);
	}
}

// -------------------------------------------------------------------------------------------------------------------------------------------------

void FNonUniformPointSamplingDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&SamplePoints) ||
		Out->IsA(&SampleRadii) ||
		Out->IsA(&SampleTriangleIDs) ||
		Out->IsA(&SampleBarycentricCoords) ||
		Out->IsA(&NumSamplePoints))
	{
		TArray<FVector> OutPoints;
		TArray<FTransform> OutSamples;
		TArray<float> OutSampleRadii;
		TArray<int32> OutTriangleIDs;
		TArray<FVector> OutBarycentricCoords;

		if (TObjectPtr<UDynamicMesh> InTargetMesh = GetValue(Context, &TargetMesh))
		{
			const UE::Geometry::FDynamicMesh3& InDynTargetMesh = InTargetMesh->GetMeshRef();

			if (InDynTargetMesh.VertexCount() > 0)
			{
				FFractureEngineSampling::ComputeNonUniformPointSampling(InDynTargetMesh,
					GetValue(Context, &SamplingRadius),
					GetValue(Context, &MaxNumSamples),
					GetValue(Context, &SubSampleDensity),
					GetValue(Context, &RandomSeed),
					GetValue(Context, &MaxSamplingRadius),
					SizeDistribution,
					GetValue(Context, &SizeDistributionPower),
					OutSamples,
					OutSampleRadii,
					OutTriangleIDs, 
					OutBarycentricCoords);

				const int32 NumSamples = OutSamples.Num();

				OutPoints.AddUninitialized(NumSamples);

				for (int32 Idx = 0; Idx < NumSamples; ++Idx)
				{
					OutPoints[Idx] = OutSamples[Idx].GetTranslation();
				}
			}
		}

		SetValue(Context, MoveTemp(OutPoints), &SamplePoints);
		SetValue(Context, MoveTemp(OutSampleRadii), &SampleRadii);
		SetValue(Context, MoveTemp(OutTriangleIDs), &SampleTriangleIDs);
		SetValue(Context, MoveTemp(OutBarycentricCoords), &SampleBarycentricCoords);
		SetValue(Context, OutPoints.Num(), &NumSamplePoints);
	}
}

// -------------------------------------------------------------------------------------------------------------------------------------------------

FNonUniformPointSamplingDataflowNode_v2::FNonUniformPointSamplingDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&TargetMesh);
	RegisterInputConnection(&SamplingRadius).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxNumSamples).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SubSampleDensity).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RandomSeed).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxSamplingRadius).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SizeDistributionPower).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&SamplePoints);
	RegisterOutputConnection(&SampleRadii);
	RegisterOutputConnection(&SampleTriangleIDs);
	RegisterOutputConnection(&SampleBarycentricCoords);
	RegisterOutputConnection(&NumSamplePoints);
}

void FNonUniformPointSamplingDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&SamplePoints) ||
		Out->IsA(&SampleRadii) ||
		Out->IsA(&SampleTriangleIDs) ||
		Out->IsA(&SampleBarycentricCoords) ||
		Out->IsA(&NumSamplePoints))
	{
		TArray<FVector> OutPoints;
		TArray<FTransform> OutSamples;
		TArray<float> OutSampleRadii;
		TArray<int32> OutTriangleIDs;
		TArray<FVector> OutBarycentricCoords;

		if (TObjectPtr<const UDataflowMesh> InTargetDataflowMesh = GetValue(Context, &TargetMesh))
		{
			const FDynamicMesh3& DynMesh = InTargetDataflowMesh->GetDynamicMeshRef();

			if (DynMesh.VertexCount() > 0)
			{
				FFractureEngineSampling::ComputeNonUniformPointSampling(DynMesh,
					GetValue(Context, &SamplingRadius),
					GetValue(Context, &MaxNumSamples),
					GetValue(Context, &SubSampleDensity),
					GetValue(Context, &RandomSeed),
					GetValue(Context, &MaxSamplingRadius),
					SizeDistribution,
					GetValue(Context, &SizeDistributionPower),
					OutSamples,
					OutSampleRadii,
					OutTriangleIDs,
					OutBarycentricCoords);

				const int32 NumSamples = OutSamples.Num();

				OutPoints.AddUninitialized(NumSamples);

				for (int32 Idx = 0; Idx < NumSamples; ++Idx)
				{
					OutPoints[Idx] = OutSamples[Idx].GetTranslation();
				}
			}
		}

		FDataflowPoints NewPoints;
		GeometryCollection::Facades::FPointsFacade PointFacade = NewPoints.GetPointsFacade();
		PointFacade.AddPoints(OutPoints);

		SetValue(Context, MoveTemp(NewPoints), &SamplePoints);
		SetValue(Context, MoveTemp(OutSampleRadii), &SampleRadii);
		SetValue(Context, MoveTemp(OutTriangleIDs), &SampleTriangleIDs);
		SetValue(Context, MoveTemp(OutBarycentricCoords), &SampleBarycentricCoords);
		SetValue(Context, OutPoints.Num(), &NumSamplePoints);
	}
}

// -------------------------------------------------------------------------------------------------------------------------------------------------

void FVertexWeightedPointSamplingDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&SamplePoints) ||
		Out->IsA(&SampleRadii) ||
		Out->IsA(&SampleTriangleIDs) ||
		Out->IsA(&SampleBarycentricCoords) ||
		Out->IsA(&NumSamplePoints))
	{
		TArray<FVector> OutPoints;
		TArray<FTransform> OutSamples;
		TArray<float> OutSampleRadii;
		TArray<int32> OutTriangleIDs;
		TArray<FVector> OutBarycentricCoords;

		if (TObjectPtr<UDynamicMesh> InTargetMesh = GetValue(Context, &TargetMesh))
		{
			const UE::Geometry::FDynamicMesh3& InDynTargetMesh = InTargetMesh->GetMeshRef();

			if (InDynTargetMesh.VertexCount() > 0)
			{
				if (IsConnected(&VertexWeights))
				{
					FFractureEngineSampling::ComputeVertexWeightedPointSampling(InDynTargetMesh,
						GetValue(Context, &VertexWeights),
						GetValue(Context, &SamplingRadius),
						GetValue(Context, &MaxNumSamples),
						GetValue(Context, &SubSampleDensity),
						GetValue(Context, &RandomSeed),
						GetValue(Context, &MaxSamplingRadius),
						SizeDistribution,
						GetValue(Context, &SizeDistributionPower),
						WeightMode,
						bInvertWeights,
						OutSamples,
						OutSampleRadii,
						OutTriangleIDs,
						OutBarycentricCoords);

					const int32 NumSamples = OutSamples.Num();

					OutPoints.AddUninitialized(NumSamples);

					for (int32 Idx = 0; Idx < NumSamples; ++Idx)
					{
						OutPoints[Idx] = OutSamples[Idx].GetTranslation();
					}

				}
			}
		}

		SetValue(Context, MoveTemp(OutPoints), &SamplePoints);
		SetValue(Context, MoveTemp(OutSampleRadii), &SampleRadii);
		SetValue(Context, MoveTemp(OutTriangleIDs), &SampleTriangleIDs);
		SetValue(Context, MoveTemp(OutBarycentricCoords), &SampleBarycentricCoords);
		SetValue(Context, OutPoints.Num(), &NumSamplePoints);
	}
}

// -------------------------------------------------------------------------------------------------------------------------------------------------

FVertexWeightedPointSamplingDataflowNode_v2::FVertexWeightedPointSamplingDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&TargetMesh);
	RegisterInputConnection(&VertexWeights);
	RegisterInputConnection(&SamplingRadius).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxNumSamples).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SubSampleDensity).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RandomSeed).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxSamplingRadius).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SizeDistributionPower).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&SamplePoints);
	RegisterOutputConnection(&SampleRadii);
	RegisterOutputConnection(&SampleTriangleIDs);
	RegisterOutputConnection(&SampleBarycentricCoords);
	RegisterOutputConnection(&NumSamplePoints);
}

void FVertexWeightedPointSamplingDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&SamplePoints) ||
		Out->IsA(&SampleRadii) ||
		Out->IsA(&SampleTriangleIDs) ||
		Out->IsA(&SampleBarycentricCoords) ||
		Out->IsA(&NumSamplePoints))
	{
		TArray<FVector> OutPoints;
		TArray<FTransform> OutSamples;
		TArray<float> OutSampleRadii;
		TArray<int32> OutTriangleIDs;
		TArray<FVector> OutBarycentricCoords;

		if (TObjectPtr<const UDataflowMesh> InTargetDataflowMesh = GetValue(Context, &TargetMesh))
		{
			const FDynamicMesh3& DynMesh = InTargetDataflowMesh->GetDynamicMeshRef();

			const int32 NumVertices = DynMesh.VertexCount();
			if (NumVertices > 0)
			{
				TArray<float> InVertexWeights;

				if (IsConnected(&VertexWeights))
				{
					InVertexWeights = GetValue(Context, &VertexWeights);
				}
				else
				{
					InVertexWeights.Init(1.0, NumVertices);
				}

				FFractureEngineSampling::ComputeVertexWeightedPointSampling(DynMesh,
					InVertexWeights,
					GetValue(Context, &SamplingRadius),
					GetValue(Context, &MaxNumSamples),
					GetValue(Context, &SubSampleDensity),
					GetValue(Context, &RandomSeed),
					GetValue(Context, &MaxSamplingRadius),
					SizeDistribution,
					GetValue(Context, &SizeDistributionPower),
					WeightMode,
					bInvertWeights,
					OutSamples,
					OutSampleRadii,
					OutTriangleIDs,
					OutBarycentricCoords);

				const int32 NumSamples = OutSamples.Num();

				OutPoints.AddUninitialized(NumSamples);

				for (int32 Idx = 0; Idx < NumSamples; ++Idx)
				{
					OutPoints[Idx] = OutSamples[Idx].GetTranslation();
				}
			}
		}

		FDataflowPoints NewPoints;
		GeometryCollection::Facades::FPointsFacade PointFacade = NewPoints.GetPointsFacade();
		PointFacade.AddPoints(OutPoints);

		SetValue(Context, MoveTemp(NewPoints), &SamplePoints);
		SetValue(Context, MoveTemp(OutSampleRadii), &SampleRadii);
		SetValue(Context, MoveTemp(OutTriangleIDs), &SampleTriangleIDs);
		SetValue(Context, MoveTemp(OutBarycentricCoords), &SampleBarycentricCoords);
		SetValue(Context, OutPoints.Num(), &NumSamplePoints);
	}
}

// -------------------------------------------------------------------------------------------------------------------------------------------------

FMeshMedialSkeletonSamplingDataflowNode::FMeshMedialSkeletonSamplingDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&TargetMesh);
	RegisterInputConnection(&MaxSpheres).SetCanHidePin(true);
	RegisterInputConnection(&MinClusterErrorToSplit).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinClusterSizeToKeep).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bSplitClustersIfEdgesIntersectSurface).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ErrorScaleNearEdgeSurfaceIntersection).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&PosErrorWt).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&MedialSkeleton);
	RegisterOutputConnection(&MedialSpheres);
}

void FMeshMedialSkeletonSamplingDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&MedialSkeleton) ||
		Out->IsA(&MedialSpheres))
	{
		FDataflowMedialSkeleton OutMedialSkeleton;
		UE::Geometry::MedialAxis::FMedialSkeleton& Skeleton = OutMedialSkeleton.Skeleton;

		if (TObjectPtr<const UDynamicMesh> InTargetMesh = GetValue(Context, &TargetMesh))
		{
			const UE::Geometry::FDynamicMesh3& InDynTargetMesh = InTargetMesh->GetMeshRef();

			if (InDynTargetMesh.TriangleCount() > 0)
			{
				UE::Geometry::TMeshAABBTree3<FDynamicMesh3> Spatial(&InDynTargetMesh);
				UE::Geometry::TFastWindingTree<FDynamicMesh3> FWTree(&Spatial);
				UE::Geometry::MedialAxis::FSkeletonViaSampling SkelSample;
				SkelSample.MaxSpheres = GetValue(Context, &MaxSpheres);
				SkelSample.MinClusterErrorToSplit = GetValue(Context, &MinClusterErrorToSplit);
				SkelSample.MinClusterSizeToKeep = GetValue(Context, &MinClusterSizeToKeep);
				SkelSample.bSplitClustersIfEdgesIntersectSurface = GetValue(Context, &bSplitClustersIfEdgesIntersectSurface);
				SkelSample.ErrorScaleNearEdgeSurfaceIntersection = GetValue(Context, &ErrorScaleNearEdgeSurfaceIntersection);
				SkelSample.PosErrorWt = GetValue(Context, &PosErrorWt);

				Skeleton = SkelSample.ComputeSkeleton(Spatial, 1e-4, &FWTree);
			}
		}
		else
		{
			Context.Error(LOCTEXT("MedialSkeletonSamplingNullMeshError", "MeshMedialSkeletonSampling: TargetMesh input is null"), this, Out);
		}

		TArray<FSphere> Spheres = Skeleton.Spheres;
		SetValue(Context, MoveTemp(OutMedialSkeleton), &MedialSkeleton);
		SetValue(Context, MoveTemp(Spheres), &MedialSpheres);
	}
}

// -------------------------------------------------------------------------------------------------------------------------------------------------

FMeshMedialSkeletonSamplingDataflowNode_v2::FMeshMedialSkeletonSamplingDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&TargetMesh);
	RegisterInputConnection(&MaxSpheres).SetCanHidePin(true);
	RegisterInputConnection(&MinClusterErrorToSplit).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinClusterSizeToKeep).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bSplitClustersIfEdgesIntersectSurface).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ErrorScaleNearEdgeSurfaceIntersection).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&PosErrorWt).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&MedialSkeleton);
	RegisterOutputConnection(&MedialSpheres);
}

void FMeshMedialSkeletonSamplingDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&MedialSkeleton) ||
		Out->IsA(&MedialSpheres))
	{
		FDataflowMedialSkeleton OutMedialSkeleton;
		UE::Geometry::MedialAxis::FMedialSkeleton& Skeleton = OutMedialSkeleton.Skeleton;

		if (TObjectPtr<const UDataflowMesh> InTargetDataflowMesh = GetValue(Context, &TargetMesh))
		{
			const FDynamicMesh3& InDynTargetMesh = InTargetDataflowMesh->GetDynamicMeshRef();

			if (InDynTargetMesh.TriangleCount() > 0)
			{
				UE::Geometry::TMeshAABBTree3<FDynamicMesh3> Spatial(&InDynTargetMesh);
				UE::Geometry::TFastWindingTree<FDynamicMesh3> FWTree(&Spatial);
				UE::Geometry::MedialAxis::FSkeletonViaSampling SkelSample;
				SkelSample.MaxSpheres = GetValue(Context, &MaxSpheres);
				SkelSample.MinClusterErrorToSplit = GetValue(Context, &MinClusterErrorToSplit);
				SkelSample.MinClusterSizeToKeep = GetValue(Context, &MinClusterSizeToKeep);
				SkelSample.bSplitClustersIfEdgesIntersectSurface = GetValue(Context, &bSplitClustersIfEdgesIntersectSurface);
				SkelSample.ErrorScaleNearEdgeSurfaceIntersection = GetValue(Context, &ErrorScaleNearEdgeSurfaceIntersection);
				SkelSample.PosErrorWt = GetValue(Context, &PosErrorWt);

				Skeleton = SkelSample.ComputeSkeleton(Spatial, 1e-4, &FWTree);
			}
		}
		else
		{
			Context.Error(LOCTEXT("MedialSkeletonSamplingNullMeshError", "MeshMedialSkeletonSampling: TargetMesh input is null"), this, Out);
		}

		TArray<FSphere> Spheres = Skeleton.Spheres;
		SetValue(Context, MoveTemp(OutMedialSkeleton), &MedialSkeleton);
		SetValue(Context, MoveTemp(Spheres), &MedialSpheres);
	}
}

// -------------------------------------------------------------------------------------------------------------------------------------------------

FSimplifyMedialSkeletonDataflowNode::FSimplifyMedialSkeletonDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MedialSkeleton);
	RegisterInputConnection(&TargetMesh);
	RegisterInputConnection(&MinSpheres).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&QEMErrorThresholdSqrt).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&EdgeLengthThreshold).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SphereRadiusThreshold).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SphereOverlapThreshold).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bOnlySimplifySurfaces).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ClusterSkeletonDistanceWt).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bPreventEdgeSurfaceIntersections).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bSplitThinTriEdges).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SplitThinTriEdgeAngleThresholdDeg).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SplitThinTriEdgePosErrorWt).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&MedialSkeleton);
	RegisterOutputConnection(&MedialSpheres);
}

void FSimplifyMedialSkeletonDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&MedialSkeleton) || Out->IsA(&MedialSpheres))
	{
		FDataflowMedialSkeleton OutMedialSkeleton = GetValue(Context, &MedialSkeleton);
		UE::Geometry::MedialAxis::FMedialSkeleton& Skeleton = OutMedialSkeleton.Skeleton;

		if (TObjectPtr<const UDynamicMesh> InTargetMesh = GetValue(Context, &TargetMesh))
		{
			const UE::Geometry::FDynamicMesh3& InDynTargetMesh = InTargetMesh->GetMeshRef();
			bool bIsMeshCompatible = Skeleton.IsCompatibleWithMesh(InDynTargetMesh);
			bool bIsNonEmpty = Skeleton.Spheres.Num() > 0;
			if (bIsMeshCompatible && bIsNonEmpty)
			{
				UE::Geometry::TMeshAABBTree3<FDynamicMesh3> Spatial(&InDynTargetMesh);

				UE::Geometry::MedialAxis::FSkeletonSimplifier Simplifier;
				Simplifier.MinSpheres = GetValue(Context, &MinSpheres);
				double UseQEMThresholdSqrt = GetValue(Context, &QEMErrorThresholdSqrt);
				Simplifier.QEMErrorThreshold = UseQEMThresholdSqrt * UseQEMThresholdSqrt;
				Simplifier.EdgeLengthThreshold = GetValue(Context, &EdgeLengthThreshold);
				Simplifier.SphereRadiusThreshold = GetValue(Context, &SphereRadiusThreshold);
				Simplifier.SphereOverlapThreshold = GetValue(Context, &SphereOverlapThreshold);
				Simplifier.bOnlySimplifySurfaces = GetValue(Context, &bOnlySimplifySurfaces);
				Simplifier.ClusterSkeletonDistanceWt = GetValue(Context, &ClusterSkeletonDistanceWt);
				Simplifier.bPreventEdgeSurfaceIntersections = GetValue(Context, &bPreventEdgeSurfaceIntersections);
				Simplifier.ClusterRegularizeWeight = ClusterRegularizeWeight;
				Simplifier.bSplitThinTriEdges = GetValue(Context, &bSplitThinTriEdges);
				Simplifier.SplitThinTriEdgeAngleThresholdDeg = GetValue(Context, &SplitThinTriEdgeAngleThresholdDeg);
				Simplifier.SplitThinTriEdgePosErrorWt = GetValue(Context, &SplitThinTriEdgePosErrorWt);

				Simplifier.Simplify(Skeleton, Spatial);
			}
			else
			{
				if (!bIsMeshCompatible)
				{
					Context.Error(LOCTEXT("SimplifyMedialSkeletonIncompatibleMesh", "SimplifyMedialSkeleton: Incompatible mesh provided. Simplify expects a mesh matching what was used to generate the skeleton."), this, Out);
				}
				if (!bIsNonEmpty)
				{
					Context.Warning(LOCTEXT("SimplifyMedialSkeletonEmptySkeleton", "SimplifyMedialSkeleton: Empty skeleton has nothing to simplify."), this, Out);
				}
			}
		}
		else
		{
			Context.Error(LOCTEXT("SimplifyMedialSkeletonNullMeshError", "SimplifyMedialSkeleton: TargetMesh input is null"), this, Out);
		}

		TArray<FSphere> Spheres = Skeleton.Spheres;
		SetValue(Context, MoveTemp(OutMedialSkeleton), &MedialSkeleton);
		SetValue(Context, MoveTemp(Spheres), &MedialSpheres);
	}
}

// -------------------------------------------------------------------------------------------------------------------------------------------------

FSimplifyMedialSkeletonDataflowNode_v2::FSimplifyMedialSkeletonDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MedialSkeleton);
	RegisterInputConnection(&TargetMesh);
	RegisterInputConnection(&MinSpheres).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&QEMErrorThresholdSqrt).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&EdgeLengthThreshold).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SphereRadiusThreshold).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SphereOverlapThreshold).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bOnlySimplifySurfaces).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ClusterSkeletonDistanceWt).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bPreventEdgeSurfaceIntersections).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bSplitThinTriEdges).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SplitThinTriEdgeAngleThresholdDeg).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SplitThinTriEdgePosErrorWt).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&MedialSkeleton);
	RegisterOutputConnection(&MedialSpheres);
}

void FSimplifyMedialSkeletonDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&MedialSkeleton) || Out->IsA(&MedialSpheres))
	{
		FDataflowMedialSkeleton OutMedialSkeleton = GetValue(Context, &MedialSkeleton);
		UE::Geometry::MedialAxis::FMedialSkeleton& Skeleton = OutMedialSkeleton.Skeleton;

		if (TObjectPtr<const UDataflowMesh>InTargetDataflowMesh = GetValue(Context, &TargetMesh))
		{
			const FDynamicMesh3& InDynTargetMesh = InTargetDataflowMesh->GetDynamicMeshRef();

			bool bIsMeshCompatible = Skeleton.IsCompatibleWithMesh(InDynTargetMesh);
			bool bIsNonEmpty = Skeleton.Spheres.Num() > 0;
			if (bIsMeshCompatible && bIsNonEmpty)
			{
				UE::Geometry::TMeshAABBTree3<FDynamicMesh3> Spatial(&InDynTargetMesh);

				UE::Geometry::MedialAxis::FSkeletonSimplifier Simplifier;
				Simplifier.MinSpheres = GetValue(Context, &MinSpheres);
				double UseQEMThresholdSqrt = GetValue(Context, &QEMErrorThresholdSqrt);
				Simplifier.QEMErrorThreshold = UseQEMThresholdSqrt * UseQEMThresholdSqrt;
				Simplifier.EdgeLengthThreshold = GetValue(Context, &EdgeLengthThreshold);
				Simplifier.SphereRadiusThreshold = GetValue(Context, &SphereRadiusThreshold);
				Simplifier.SphereOverlapThreshold = GetValue(Context, &SphereOverlapThreshold);
				Simplifier.bOnlySimplifySurfaces = GetValue(Context, &bOnlySimplifySurfaces);
				Simplifier.ClusterSkeletonDistanceWt = GetValue(Context, &ClusterSkeletonDistanceWt);
				Simplifier.bPreventEdgeSurfaceIntersections = GetValue(Context, &bPreventEdgeSurfaceIntersections);
				Simplifier.ClusterRegularizeWeight = ClusterRegularizeWeight;
				Simplifier.bSplitThinTriEdges = GetValue(Context, &bSplitThinTriEdges);
				Simplifier.SplitThinTriEdgeAngleThresholdDeg = GetValue(Context, &SplitThinTriEdgeAngleThresholdDeg);
				Simplifier.SplitThinTriEdgePosErrorWt = GetValue(Context, &SplitThinTriEdgePosErrorWt);

				Simplifier.Simplify(Skeleton, Spatial);
			}
			else
			{
				if (!bIsMeshCompatible)
				{
					Context.Error(LOCTEXT("SimplifyMedialSkeletonIncompatibleMesh", "SimplifyMedialSkeleton: Incompatible mesh provided. Simplify expects a mesh matching what was used to generate the skeleton."), this, Out);
				}
				if (!bIsNonEmpty)
				{
					Context.Warning(LOCTEXT("SimplifyMedialSkeletonEmptySkeleton", "SimplifyMedialSkeleton: Empty skeleton has nothing to simplify."), this, Out);
				}
			}
		}
		else
		{
			Context.Error(LOCTEXT("SimplifyMedialSkeletonNullMeshError", "SimplifyMedialSkeleton: TargetMesh input is null"), this, Out);
		}

		TArray<FSphere> Spheres = Skeleton.Spheres;
		SetValue(Context, MoveTemp(OutMedialSkeleton), &MedialSkeleton);
		SetValue(Context, MoveTemp(Spheres), &MedialSpheres);
	}
}

// -------------------------------------------------------------------------------------------------------------------------------------------------

FSubdivideMedialSkeletonDataflowNode::FSubdivideMedialSkeletonDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MedialSkeleton);
	RegisterInputConnection(&TargetMesh);
	RegisterInputConnection(&TargetEdgeLengthRadiusFraction).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RadiusReference).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinTargetEdgeLength).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxTargetEdgeLength).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bSubdivideOnSurfaces).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ReassignClusterPosErrorWt).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bProjectNewMedialSpheres).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&MedialSkeleton);
	RegisterOutputConnection(&MedialSpheres);
}

void FSubdivideMedialSkeletonDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&MedialSkeleton) || Out->IsA(&MedialSpheres))
	{
		FDataflowMedialSkeleton OutMedialSkeleton = GetValue(Context, &MedialSkeleton);
		UE::Geometry::MedialAxis::FMedialSkeleton& Skeleton = OutMedialSkeleton.Skeleton;

		if (TObjectPtr<const UDynamicMesh> InTargetMesh = GetValue(Context, &TargetMesh))
		{
			const UE::Geometry::FDynamicMesh3& InDynTargetMesh = InTargetMesh->GetMeshRef();
			bool bIsMeshCompatible = Skeleton.IsCompatibleWithMesh(InDynTargetMesh);
			bool bIsNonEmpty = Skeleton.Spheres.Num() > 0;
			if (bIsMeshCompatible && bIsNonEmpty)
			{
				const double RadiusFraction = GetValue(Context, &TargetEdgeLengthRadiusFraction);
				const ESubdivideEdgeRadiusReference RadRef = GetValue(Context, &RadiusReference);
				const double MinLen = GetValue(Context, &MinTargetEdgeLength);
				const double MaxLen = GetValue(Context, &MaxTargetEdgeLength);

				if (MinLen > 0 && MaxLen > 0 && MinLen > MaxLen)
				{
					Context.Warning(LOCTEXT("SubdivideMinMaxInverted", "SubdivideMedialSkeleton: MinTargetEdgeLength exceeds MaxTargetEdgeLength; Max will take precedence."), this, Out);
				}

				// If all parameters are zero, skip subdivision
				if (RadiusFraction > 0 || MinLen > 0 || MaxLen > 0)
				{
					UE::Geometry::TMeshAABBTree3<FDynamicMesh3> Spatial(&InDynTargetMesh);

					UE::Geometry::MedialAxis::FSkeletonSubdivider Subdivider;
					Subdivider.bSubdivideOnSurfaces = GetValue(Context, &bSubdivideOnSurfaces);
					Subdivider.ReassignClusterPosErrorWt = GetValue(Context, &ReassignClusterPosErrorWt);
					Subdivider.bReprojectMedialSpheres = GetValue(Context, &bProjectNewMedialSpheres);
					Subdivider.GetTargetEdgeLength = [RadiusFraction, RadRef, MinLen, MaxLen](const FSphere& A, const FSphere& B) -> double
					{
						double Radius;
						switch (RadRef)
						{
						default:
						case ESubdivideEdgeRadiusReference::Larger:
							Radius = FMath::Max(A.W, B.W);
							break;
						case ESubdivideEdgeRadiusReference::Smaller:
							Radius = FMath::Min(A.W, B.W);
							break;
						case ESubdivideEdgeRadiusReference::Average:
							Radius = (A.W + B.W) * 0.5;
							break;
						}
						double TargetLen = RadiusFraction * Radius;
						if (MinLen > 0)
						{
							TargetLen = FMath::Max(TargetLen, MinLen);
						}
						if (MaxLen > 0)
						{
							// Use MaxLen as the target if nothing else has set one
							TargetLen = (TargetLen > 0) ? FMath::Min(TargetLen, MaxLen) : MaxLen;
						}
						return TargetLen;
					};

					const UE::Geometry::TFastWindingTree<FDynamicMesh3>* FWTreePtr = nullptr;
					TOptional<UE::Geometry::TFastWindingTree<FDynamicMesh3>> FWTree;
					if (Subdivider.bReprojectMedialSpheres)
					{
						FWTree.Emplace(&Spatial);
						FWTreePtr = &FWTree.GetValue();
					}
					Subdivider.Subdivide(Skeleton, Spatial, UE_DOUBLE_KINDA_SMALL_NUMBER, FWTreePtr);
				}
			}
			else
			{
				if (!bIsMeshCompatible)
				{
					Context.Error(LOCTEXT("SubdivideMedialSkeletonIncompatibleMesh", "SubdivideMedialSkeleton: Incompatible mesh provided. Subdivide expects a mesh matching what was used to generate the skeleton."), this, Out);
				}
				if (!bIsNonEmpty)
				{
					Context.Warning(LOCTEXT("SubdivideMedialSkeletonEmptySkeleton", "SubdivideMedialSkeleton: Empty skeleton has nothing to subdivide."), this, Out);
				}
			}
		}
		else
		{
			Context.Error(LOCTEXT("SubdivideMedialSkeletonNullMeshError", "SubdivideMedialSkeleton: TargetMesh input is null"), this, Out);
		}

		TArray<FSphere> Spheres = Skeleton.Spheres;
		SetValue(Context, MoveTemp(OutMedialSkeleton), &MedialSkeleton);
		SetValue(Context, MoveTemp(Spheres), &MedialSpheres);
	}
}

// -------------------------------------------------------------------------------------------------------------------------------------------------

FSubdivideMedialSkeletonDataflowNode_v2::FSubdivideMedialSkeletonDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MedialSkeleton);
	RegisterInputConnection(&TargetMesh);
	RegisterInputConnection(&TargetEdgeLengthRadiusFraction).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RadiusReference).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinTargetEdgeLength).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxTargetEdgeLength).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bSubdivideOnSurfaces).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ReassignClusterPosErrorWt).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bProjectNewMedialSpheres).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&MedialSkeleton);
	RegisterOutputConnection(&MedialSpheres);
}

void FSubdivideMedialSkeletonDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&MedialSkeleton) || Out->IsA(&MedialSpheres))
	{
		FDataflowMedialSkeleton OutMedialSkeleton = GetValue(Context, &MedialSkeleton);
		UE::Geometry::MedialAxis::FMedialSkeleton& Skeleton = OutMedialSkeleton.Skeleton;

		if (TObjectPtr<const UDataflowMesh>InTargetDataflowMesh = GetValue(Context, &TargetMesh))
		{
			const FDynamicMesh3& InDynTargetMesh = InTargetDataflowMesh->GetDynamicMeshRef();

			bool bIsMeshCompatible = Skeleton.IsCompatibleWithMesh(InDynTargetMesh);
			bool bIsNonEmpty = Skeleton.Spheres.Num() > 0;
			if (bIsMeshCompatible && bIsNonEmpty)
			{
				const double RadiusFraction = GetValue(Context, &TargetEdgeLengthRadiusFraction);
				const ESubdivideEdgeRadiusReference RadRef = GetValue(Context, &RadiusReference);
				const double MinLen = GetValue(Context, &MinTargetEdgeLength);
				const double MaxLen = GetValue(Context, &MaxTargetEdgeLength);

				if (MinLen > 0 && MaxLen > 0 && MinLen > MaxLen)
				{
					Context.Warning(LOCTEXT("SubdivideMinMaxInverted", "SubdivideMedialSkeleton: MinTargetEdgeLength exceeds MaxTargetEdgeLength; Max will take precedence."), this, Out);
				}

				// If all parameters are zero, skip subdivision
				if (RadiusFraction > 0 || MinLen > 0 || MaxLen > 0)
				{
					UE::Geometry::TMeshAABBTree3<FDynamicMesh3> Spatial(&InDynTargetMesh);

					UE::Geometry::MedialAxis::FSkeletonSubdivider Subdivider;
					Subdivider.bSubdivideOnSurfaces = GetValue(Context, &bSubdivideOnSurfaces);
					Subdivider.ReassignClusterPosErrorWt = GetValue(Context, &ReassignClusterPosErrorWt);
					Subdivider.bReprojectMedialSpheres = GetValue(Context, &bProjectNewMedialSpheres);
					Subdivider.GetTargetEdgeLength = [RadiusFraction, RadRef, MinLen, MaxLen](const FSphere& A, const FSphere& B) -> double
						{
							double Radius;
							switch (RadRef)
							{
							default:
							case ESubdivideEdgeRadiusReference::Larger:
								Radius = FMath::Max(A.W, B.W);
								break;
							case ESubdivideEdgeRadiusReference::Smaller:
								Radius = FMath::Min(A.W, B.W);
								break;
							case ESubdivideEdgeRadiusReference::Average:
								Radius = (A.W + B.W) * 0.5;
								break;
							}
							double TargetLen = RadiusFraction * Radius;
							if (MinLen > 0)
							{
								TargetLen = FMath::Max(TargetLen, MinLen);
							}
							if (MaxLen > 0)
							{
								// Use MaxLen as the target if nothing else has set one
								TargetLen = (TargetLen > 0) ? FMath::Min(TargetLen, MaxLen) : MaxLen;
							}
							return TargetLen;
						};

					const UE::Geometry::TFastWindingTree<FDynamicMesh3>* FWTreePtr = nullptr;
					TOptional<UE::Geometry::TFastWindingTree<FDynamicMesh3>> FWTree;
					if (Subdivider.bReprojectMedialSpheres)
					{
						FWTree.Emplace(&Spatial);
						FWTreePtr = &FWTree.GetValue();
					}
					Subdivider.Subdivide(Skeleton, Spatial, UE_DOUBLE_KINDA_SMALL_NUMBER, FWTreePtr);
				}
			}
			else
			{
				if (!bIsMeshCompatible)
				{
					Context.Error(LOCTEXT("SubdivideMedialSkeletonIncompatibleMesh", "SubdivideMedialSkeleton: Incompatible mesh provided. Subdivide expects a mesh matching what was used to generate the skeleton."), this, Out);
				}
				if (!bIsNonEmpty)
				{
					Context.Warning(LOCTEXT("SubdivideMedialSkeletonEmptySkeleton", "SubdivideMedialSkeleton: Empty skeleton has nothing to subdivide."), this, Out);
				}
			}
		}
		else
		{
			Context.Error(LOCTEXT("SubdivideMedialSkeletonNullMeshError", "SubdivideMedialSkeleton: TargetMesh input is null"), this, Out);
		}

		TArray<FSphere> Spheres = Skeleton.Spheres;
		SetValue(Context, MoveTemp(OutMedialSkeleton), &MedialSkeleton);
		SetValue(Context, MoveTemp(Spheres), &MedialSpheres);
	}
}

// -------------------------------------------------------------------------------------------------------------------------------------------------

#undef LOCTEXT_NAMESPACE

