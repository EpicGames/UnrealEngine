// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionSelectionNodes.h"
#include "Dataflow/DataflowCore.h"

#if WITH_EDITOR
#include "Dataflow/DataflowRenderingViewMode.h"
#endif

#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowSimpleDebugDrawMesh.h"

#include "Engine/StaticMesh.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionEngineUtility.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealTypePrivate.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "StaticMeshAttributes.h"
#include "DynamicMeshEditor.h"
#include "Operations/MeshBoolean.h"

#include "EngineGlobals.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "Voronoi/Voronoi.h"
#include "PlanarCut.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "FractureEngineClustering.h"
#include "FractureEngineSelection.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "Dataflow/GeometryCollectionUtils.h"
#include "GeometryCollection/Facades/PointsFacade.h"
#include "Dataflow/DataflowUtils.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionSelectionNodes)

#define LOCTEXT_NAMESPACE "DataflowGeometryCollectionSelectionNodes"

namespace UE::Dataflow
{
	namespace SelectionNodes::Private
	{
		static void DebugDrawBox(const FBox& InBox, const FTransform& InBoxTransform, IDataflowDebugDrawInterface& DataflowRenderingInterface)
		{
			DataflowRenderingInterface.SetLineWidth(1.0);
			DataflowRenderingInterface.SetWireframe(true);
			DataflowRenderingInterface.SetWorldPriority();
			DataflowRenderingInterface.SetColor(FLinearColor::Red);

			const FVector TransformedCenter = InBox.GetCenter() + InBoxTransform.GetTranslation();
			const FVector ScaledExtent = InBox.GetExtent() * InBoxTransform.GetScale3D();
			DataflowRenderingInterface.DrawBox(ScaledExtent, InBoxTransform.GetRotation(), TransformedCenter, 1.0);
		}

		static void DebugDrawSphere(const FSphere& InSphere, IDataflowDebugDrawInterface& DataflowRenderingInterface)
		{
			DataflowRenderingInterface.SetLineWidth(1.0);
			DataflowRenderingInterface.SetWireframe(true);
			DataflowRenderingInterface.SetWorldPriority();
			DataflowRenderingInterface.SetColor(FLinearColor::Red);

			DataflowRenderingInterface.DrawSphere(InSphere.Center, InSphere.W);
		}

		static void DebugDrawPlane(const FTransform& InPlaneTransform, float Size, int32 VtxPerEdge, IDataflowDebugDrawInterface& DataflowRenderingInterface)
		{
			DataflowRenderingInterface.SetLineWidth(1.0);
			DataflowRenderingInterface.SetShaded(false);
			DataflowRenderingInterface.SetWireframe(true);
			DataflowRenderingInterface.SetWorldPriority();
			DataflowRenderingInterface.SetColor(FLinearColor::Red);

			FSimpleDebugDrawMesh Mesh;
			Mesh.MakeRectangleMesh(FVector::ZeroVector, Size, Size, VtxPerEdge, VtxPerEdge);
			Mesh.TransformVertices(InPlaneTransform);
			DataflowRenderingInterface.DrawMesh(Mesh);
		}
	}

	void GeometryCollectionSelectionNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionInfoDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionRootDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionFromIndexArrayDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionParentDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionChildrenDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionSiblingsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionLevelDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionTargetLevelDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionContactDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionLeafDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionClusterDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionClusterDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionBySizeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionByVolumeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSelectFloatArrayIndicesInRangeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSelectInternalFacesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSelectTransformStringDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSetTransformStringValueDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetNumElementsInSelectionDataflowNode);

		// deprecated
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionInvertDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionFaceSelectionInvertDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionSetOperationDataflowNode);

		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionAllDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionNoneDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionRandomDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionCustomDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionCustomDataflowNode_v2); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionByPercentageDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionInBoxDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionInSphereDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionByFloatAttrDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionVertexSelectionByBoxDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionVertexSelectionBySphereDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionVertexSelectionByPlaneDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionTransformSelectionByIntAttrDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionVertexSelectionCustomDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionFaceSelectionCustomDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSelectionConvertDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionVertexSelectionByPercentageDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionVertexSelectionSetOperationDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSelectionByAttrDataflowNode); //
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGeometrySelectionToVertexSelectionDataflowNode); //

		// generic input nodes
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSelectionSetOperationDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSelectionInvertDataflowNode);

		// New nodes
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSelectionCustomDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSelectionConvertDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSelectionAllDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSelectionNoneDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSelectionRandomDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSelectionByPercentageDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSelectionByAttributeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSelectionByPrimitiveDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSelectionByMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCollectionSelectionToAttributeDataflowNode);
	}
}


void FCollectionTransformSelectionAllDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectAll();

		FDataflowTransformSelection NewTransformSelection;
		NewTransformSelection.InitializeFromCollection(InCollection, false);
		NewTransformSelection.SetFromArray(SelectionArr);

		SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}


void FCollectionTransformSelectionSetOperationDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		const FDataflowTransformSelection& InTransformSelectionA = GetValue<FDataflowTransformSelection>(Context, &TransformSelectionA);
		const FDataflowTransformSelection& InTransformSelectionB = GetValue<FDataflowTransformSelection>(Context, &TransformSelectionB);

		FDataflowTransformSelection NewTransformSelection;

		if (InTransformSelectionA.Num() == InTransformSelectionB.Num())
		{
			if (Operation == ESetOperationEnum::Dataflow_SetOperation_AND)
			{
				InTransformSelectionA.AND(InTransformSelectionB, NewTransformSelection);
			}
			else if (Operation == ESetOperationEnum::Dataflow_SetOperation_OR)
			{
				InTransformSelectionA.OR(InTransformSelectionB, NewTransformSelection);
			}
			else if (Operation == ESetOperationEnum::Dataflow_SetOperation_XOR)
			{
				InTransformSelectionA.XOR(InTransformSelectionB, NewTransformSelection);
			}
			else if (Operation == ESetOperationEnum::Dataflow_SetOperation_Subtract)
			{
				InTransformSelectionA.Subtract(InTransformSelectionB, NewTransformSelection);
			}
		}
		else
		{
			// ERROR: INPUT TRANSFORMSELECTIONS HAVE DIFFERENT NUMBER OF ELEMENTS
			FString ErrorStr = "Input TransformSelections have different number of elements.";
			UE_LOGF(LogTemp, Error, "[Dataflow ERROR] %ls", *ErrorStr);
		}

		SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
	}
}

namespace {
	struct BoneInfo {
		int32 BoneIndex;
		int32 Level;
	};
}

static void ExpandRecursive(const int32 BoneIndex, int32 Level, const TManagedArray<TSet<int32>>& Children, TArray<BoneInfo>& BoneHierarchy)
{
	BoneHierarchy.Add({ BoneIndex, Level });

	TSet<int32> ChildrenSet = Children[BoneIndex];
	if (ChildrenSet.Num() > 0)
	{
		for (auto& Child : ChildrenSet)
		{
			ExpandRecursive(Child, Level + 1, Children, BoneHierarchy);
		}
	}
}

static void BuildHierarchicalOutput(const TManagedArray<int32>& Parents, 
	const TManagedArray<TSet<int32>>& Children, 
	const TManagedArray<FString>& BoneNames,
	const FDataflowTransformSelection& TransformSelection, 
	FString& OutputStr)
{
	TArray<BoneInfo> BoneHierarchy;

	int32 NumElements = Parents.Num();
	for (int32 Index = 0; Index < NumElements; ++Index)
	{
		if (Parents[Index] == FGeometryCollection::Invalid)
		{
			ExpandRecursive(Index, 0, Children, BoneHierarchy);
		}
	}

	// Get level max
	int32 LevelMax = -1;
	int32 BoneNameLengthMax = -1;
	for (int32 Idx = 0; Idx < BoneHierarchy.Num(); ++Idx)
	{
		if (BoneHierarchy[Idx].Level > LevelMax)
		{
			LevelMax = BoneHierarchy[Idx].Level;
		}

		int32 BoneNameLength = BoneNames[Idx].Len();
		if (BoneNameLength > BoneNameLengthMax)
		{
			BoneNameLengthMax = BoneNameLength;
		}
	}

	const int32 BoneIndexWidth = 2 + LevelMax * 2 + 6;
	const int32 BoneNameWidth = BoneNameLengthMax + 2;
	const int32 SelectedWidth = 10;

	for (int32 Idx = 0; Idx < BoneHierarchy.Num(); ++Idx)
	{
		FString BoneIndexStr, BoneNameStr;
		BoneIndexStr.Reserve(BoneIndexWidth);
		BoneNameStr.Reserve(BoneNameWidth);

		if (BoneHierarchy[Idx].Level == 0)
		{
			BoneIndexStr.Appendf(TEXT("[%d]"), BoneHierarchy[Idx].BoneIndex);
		}
		else
		{
			BoneIndexStr.Appendf(TEXT(" |"));
			for (int32 Idx1 = 0; Idx1 < BoneHierarchy[Idx].Level; ++Idx1)
			{
				BoneIndexStr.Appendf(TEXT("--"));
			}
			BoneIndexStr.Appendf(TEXT("[%d]"), BoneHierarchy[Idx].BoneIndex);
		}
		BoneIndexStr = BoneIndexStr.RightPad(BoneIndexWidth);

		BoneNameStr.Appendf(TEXT("%s"), *BoneNames[Idx]);
		BoneNameStr = BoneNameStr.RightPad(BoneNameWidth);

		OutputStr.Appendf(TEXT("%s%s%s\n\n"), *BoneIndexStr, *BoneNameStr, (TransformSelection.IsSelected(BoneHierarchy[Idx].BoneIndex) ? TEXT("Selected") : TEXT("---")));
	}

}


void FCollectionTransformSelectionInfoDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FString>(&String))
	{
		const FDataflowTransformSelection& InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		FString OutputStr;

		OutputStr.Appendf(TEXT("\n----------------------------------------\n"));
		OutputStr.Appendf(TEXT("Number of Elements: %d\n"), InTransformSelection.Num());

		// Hierarchical display
		if (InCollection.HasGroup(FGeometryCollection::TransformGroup) &&
			InCollection.HasAttribute("Parent", FGeometryCollection::TransformGroup) &&
			InCollection.HasAttribute("Children", FGeometryCollection::TransformGroup) &&
			InCollection.HasAttribute("BoneName", FGeometryCollection::TransformGroup))
		{
			if (InTransformSelection.Num() == InCollection.NumElements(FGeometryCollection::TransformGroup))
			{
				const TManagedArray<int32>& Parents = InCollection.GetAttribute<int32>("Parent", FGeometryCollection::TransformGroup);
				const TManagedArray<TSet<int32>>& Children = InCollection.GetAttribute<TSet<int32>>("Children", FGeometryCollection::TransformGroup);
				const TManagedArray<FString>& BoneNames = InCollection.GetAttribute<FString>("BoneName", FGeometryCollection::TransformGroup);

				BuildHierarchicalOutput(Parents, Children, BoneNames, InTransformSelection, OutputStr);
			}
			else
			{
				// ERROR: TransformSelection doesn't match the Collection
				FString ErrorStr = "TransformSelection doesn't match the Collection.";
				UE_LOGF(LogTemp, Error, "[Dataflow ERROR] %ls", *ErrorStr);
			}
		}
		else
		// Simple display
		{
			for (int32 Idx = 0; Idx < InTransformSelection.Num(); ++Idx)
			{
				OutputStr.Appendf(TEXT("%4d: %s\n"), Idx, (InTransformSelection.IsSelected(Idx) ? TEXT("Selected") : TEXT("---")));
			}
		}

		OutputStr.Appendf(TEXT("----------------------------------------\n"));

		SetValue(Context, MoveTemp(OutputStr), &String);
	}
}


void FCollectionTransformSelectionNoneDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectNone();

		FDataflowTransformSelection NewTransformSelection;
		NewTransformSelection.InitializeFromCollection(InCollection, false);
		NewTransformSelection.SetFromArray(SelectionArr);

		SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}


void FCollectionTransformSelectionInvertDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		FDataflowTransformSelection InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);

		InTransformSelection.Invert();

		SetValue(Context, MoveTemp(InTransformSelection), &TransformSelection);
	}
}


void FCollectionTransformSelectionRandomDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		float RandomSeedVal = GetValue<float>(Context, &RandomSeed);
		float RandomThresholdVal = GetValue<float>(Context, &RandomThreshold);

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectRandom(bDeterministic, RandomSeedVal, RandomThresholdVal);

		FDataflowTransformSelection NewTransformSelection;
		NewTransformSelection.InitializeFromCollection(InCollection, false);
		NewTransformSelection.SetFromArray(SelectionArr);

		SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}


void FCollectionTransformSelectionRootDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectRootBones();

		FDataflowTransformSelection NewTransformSelection;
		NewTransformSelection.InitializeFromCollection(InCollection, false);
		NewTransformSelection.SetFromArray(SelectionArr);

		SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}


void FCollectionTransformSelectionCustomDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		if (InCollection.HasGroup(FGeometryCollection::TransformGroup))
		{
			FDataflowTransformSelection NewTransformSelection;
			NewTransformSelection.InitializeFromCollection(InCollection, false);

			const FString InBoneIndices = GetValue<FString>(Context, &BoneIndicies);

			TArray<FString> Indices;
			InBoneIndices.ParseIntoArray(Indices, TEXT(" "), true);

			for (FString IndexStr : Indices)
			{
				if (IndexStr.IsNumeric())
				{
					int32 Index = FCString::Atoi(*IndexStr);
					if (NewTransformSelection.IsValidIndex(Index))
					{
						NewTransformSelection.SetSelected(Index);
					}
					else
					{
						// ERROR: INVALID INDEX
						Context.Error(FString::Printf(
							TEXT("Index %d is not a valid vertex index for transform group size %d."),
							Index, NewTransformSelection.Num()),
							this);
					}
				}
			}

			SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
		}
		else
		{
			SetValue(Context, FDataflowTransformSelection(), &TransformSelection);
		}
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

FCollectionTransformSelectionCustomDataflowNode_v2::FCollectionTransformSelectionCustomDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&BoneIndices);

	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&TransformSelection);
}

void FCollectionTransformSelectionCustomDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

		if (InCollection.HasGroup(FGeometryCollection::TransformGroup))
		{
			FDataflowTransformSelection NewTransformSelection;
			NewTransformSelection.InitializeFromCollection(InCollection, false);

			const FString InBoneIndices = GetValue(Context, &BoneIndices);

			TArray<int32> Indices;

			using namespace UE::Dataflow::Utils;

			EErrorCode ErrorCode = ParseIndicesStr(InBoneIndices, Indices);

			if (ErrorCode == EErrorCode::None)
			{
				NewTransformSelection.SetSelected(Indices);
			}
			else
			{
				if (ErrorCode == EErrorCode::InvalidChars)
				{
					SetError(Context, &TransformSelection, TEXT("Invalid character(s) specified in list"));
				}
				else if (ErrorCode == EErrorCode::InvalidFormatInSegment)
				{
					SetError(Context, &TransformSelection, TEXT("Invalid format in segment"));
				}
			}

			SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
		}
		else
		{
			SetValue(Context, FDataflowTransformSelection(), &TransformSelection);
		}
	}
	else if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

void FCollectionTransformSelectionFromIndexArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

		if (InCollection.HasGroup(FGeometryCollection::TransformGroup))
		{
			const TArray<int32>& InBoneIndices = GetValue(Context, &BoneIndices);

			FDataflowTransformSelection NewTransformSelection;
			NewTransformSelection.InitializeFromCollection(InCollection, false);
			for (int32 SelectedIdx : InBoneIndices)
			{
				if (NewTransformSelection.IsValidIndex(SelectedIdx))
				{
					NewTransformSelection.SetSelected(SelectedIdx);
				}
				else
				{
					Context.Error(FString::Printf(
							TEXT("Index %d is not a valid vertex index for transform group size %d."),
							SelectedIdx, NewTransformSelection.Num()),
							this);
				}
			}

			SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
		}
		else
		{
			SetValue(Context, FDataflowTransformSelection(), &TransformSelection);
		}
	}
	else if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}


void FCollectionTransformSelectionParentDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		FDataflowTransformSelection InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		TArray<int32> SelectionArr = InTransformSelection.AsArray();
		TransformSelectionFacade.SelectParent(SelectionArr);

		InTransformSelection.SetFromArray(SelectionArr);
		
		SetValue(Context, MoveTemp(InTransformSelection), &TransformSelection);
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}


void FCollectionTransformSelectionByPercentageDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		FDataflowTransformSelection InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);

		int32 InPercentage = GetValue<int32>(Context, &Percentage);
		float InRandomSeed = GetValue<float>(Context, &RandomSeed);

		TArray<int32> SelectionArr = InTransformSelection.AsArray();

		GeometryCollection::Facades::FCollectionTransformSelectionFacade::SelectByPercentage(SelectionArr, InPercentage, bDeterministic, (int32)InRandomSeed);

		InTransformSelection.SetFromArray(SelectionArr);
		SetValue(Context, MoveTemp(InTransformSelection), &TransformSelection);
	}
}


void FCollectionTransformSelectionChildrenDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		FDataflowTransformSelection InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		TArray<int32> SelectionArr = InTransformSelection.AsArray();

		TransformSelectionFacade.SelectChildren(SelectionArr);
		InTransformSelection.SetFromArray(SelectionArr);

		SetValue(Context, MoveTemp(InTransformSelection), &TransformSelection);
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}


void FCollectionTransformSelectionSiblingsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		FDataflowTransformSelection InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		TArray<int32> SelectionArr = InTransformSelection.AsArray();

		TransformSelectionFacade.SelectSiblings(SelectionArr);
		InTransformSelection.SetFromArray(SelectionArr);

		SetValue(Context, MoveTemp(InTransformSelection), &TransformSelection);
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}


void FCollectionTransformSelectionLevelDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&TransformSelection) || Out->IsA(&Collection))
	{
		FManagedArrayCollection OutCollection = GetValue(Context, &Collection);
		FDataflowTransformSelection InTransformSelection = GetValue(Context, &TransformSelection);

		// make sure there's a level attribute
		Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(OutCollection);
		if (!HierarchyFacade.HasLevelAttribute())
		{
			HierarchyFacade.GenerateLevelAttribute();
		}

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(OutCollection);
		TArray<int32> SelectionArr = InTransformSelection.AsArray();

		TransformSelectionFacade.SelectLevel(SelectionArr);
		InTransformSelection.SetFromArray(SelectionArr);

		SetValue(Context, MoveTemp(InTransformSelection), &TransformSelection);
		SetValue(Context, MoveTemp(OutCollection), &Collection);
	}
}


void FCollectionTransformSelectionTargetLevelDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&TransformSelection) || Out->IsA(&Collection))
	{
		FManagedArrayCollection OutCollection = GetValue(Context, &Collection);

		// make sure there's a level attribute
		Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(OutCollection);
		if (!HierarchyFacade.HasLevelAttribute())
		{
			HierarchyFacade.GenerateLevelAttribute();
		}

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(OutCollection);

		const int32 InTargetLevel = GetValue(Context, &TargetLevel);

		const TArray<int32> AllAtLevel = TransformSelectionFacade.GetBonesExactlyAtLevel(InTargetLevel, bSkipEmbedded);

		FDataflowTransformSelection NewTransformSelection;
		NewTransformSelection.InitializeFromCollection(OutCollection, false);
		NewTransformSelection.SetFromArray(AllAtLevel);

		SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
		SetValue(Context, MoveTemp(OutCollection), &Collection);
	}
}


void FCollectionTransformSelectionContactDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		FDataflowTransformSelection InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		TArray<int32> SelectionArr = InTransformSelection.AsArray();

		TransformSelectionFacade.SelectContact(SelectionArr, bAllowContactInParentLevels);
		InTransformSelection.SetFromArray(SelectionArr);

		SetValue(Context, MoveTemp(InTransformSelection), &TransformSelection);
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}


void FCollectionTransformSelectionLeafDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectLeaf();

		FDataflowTransformSelection NewTransformSelection;
		NewTransformSelection.InitializeFromCollection(InCollection, false);
		NewTransformSelection.SetFromArray(SelectionArr);

		SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}


void FCollectionTransformSelectionClusterDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		// this node used to use SelectCluster() but this was buggy and woudl select the leaves instead
		// for this reason this node is now deprecated and we need to keep it doing what it sued to : SelectLeaf()
		// version 2 of the node properly use the right way 
		const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectLeaf(); // used to be buggy SelectCluster() - see comment above 

		FDataflowTransformSelection NewTransformSelection;
		NewTransformSelection.InitializeFromCollection(InCollection, false);
		NewTransformSelection.SetFromArray(SelectionArr);

		SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
	}
	else if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

void FCollectionTransformSelectionClusterDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectCluster(); 

		FDataflowTransformSelection NewTransformSelection;
		NewTransformSelection.InitializeFromCollection(InCollection, false);
		NewTransformSelection.SetFromArray(SelectionArr);

		SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
	}
	else if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

void FCollectionTransformSelectionBySizeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		float InSizeMin = GetValue<float>(Context, &SizeMin);
		float InSizeMax = GetValue<float>(Context, &SizeMax);
		bool bInsideRange = RangeSetting == ERangeSettingEnum::Dataflow_RangeSetting_InsideRange;

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectBySize(InSizeMin, InSizeMax, bInclusive, bInsideRange, bUseRelativeSize);

		FDataflowTransformSelection NewTransformSelection;
		NewTransformSelection.InitializeFromCollection(InCollection, false);
		NewTransformSelection.SetFromArray(SelectionArr);

		SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////

void FCollectionTransformSelectionByVolumeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		float InVolumeMin = GetValue<float>(Context, &VolumeMin);
		float InVolumeMax = GetValue<float>(Context, &VolumeMax);
		bool bInsideRange = RangeSetting == ERangeSettingEnum::Dataflow_RangeSetting_InsideRange;

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectByVolume(InVolumeMin, InVolumeMax, bInclusive, bInsideRange);

		FDataflowTransformSelection NewTransformSelection;
		NewTransformSelection.InitializeFromCollection(InCollection, false);
		NewTransformSelection.SetFromArray(SelectionArr);

		SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

void FCollectionTransformSelectionInBoxDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const FBox& InBox = GetValue<FBox>(Context, &Box);
		const FTransform& InTransform = GetValue<FTransform>(Context, &Transform);

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);

		TArray<int32> SelectionArr;
		if (Type == ESelectSubjectTypeEnum::Dataflow_SelectSubjectType_Vertices)
		{
			SelectionArr = TransformSelectionFacade.SelectVerticesInBox(InBox, InTransform, bAllVerticesMustContainedInBox);
		}
		else if (Type == ESelectSubjectTypeEnum::Dataflow_SelectSubjectType_BoundingBox)
		{
			SelectionArr = TransformSelectionFacade.SelectBoundingBoxInBox(InBox, InTransform);
		}
		else if (Type == ESelectSubjectTypeEnum::Dataflow_SelectSubjectType_Centroid)
		{
			SelectionArr = TransformSelectionFacade.SelectCentroidInBox(InBox, InTransform);
		}

		FDataflowTransformSelection NewTransformSelection;
		NewTransformSelection.InitializeFromCollection(InCollection, false);
		NewTransformSelection.SetFromArray(SelectionArr);

		SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

#if WITH_EDITOR

bool FCollectionTransformSelectionInBoxDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FCollectionTransformSelectionInBoxDataflowNode::DebugDraw(UE::Dataflow::FContext& Context,	IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		const FBox& InBox = GetValue(Context, &Box);
		const FTransform& InTransform = GetValue(Context, &Transform);

		UE::Dataflow::SelectionNodes::Private::DebugDrawBox(InBox, InTransform, DataflowRenderingInterface);
	}
}

#endif

//////////////////////////////////////////////////////////////////////////////////////////////////

FCollectionVertexSelectionByBoxDataflowNode::FCollectionVertexSelectionByBoxDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Box);
	RegisterInputConnection(&BoxTransform);

	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&VertexSelection);
}

void FCollectionVertexSelectionByBoxDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA(&VertexSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const FBox& InBox = GetValue(Context, &Box);
		const FTransform& InBoxTransform = GetValue(Context, &BoxTransform);

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		TArray<bool> SelectedVertices;
		TransformSelectionFacade.SelectVerticesInTransformedBox(InBox, InBoxTransform, SelectedVertices);

		FDataflowVertexSelection OutVertexSelection;
		OutVertexSelection.InitializeFromCollection(InCollection, false);
		OutVertexSelection.SetFromArray(SelectedVertices);

		SetValue(Context, MoveTemp(OutVertexSelection), &VertexSelection);
	}
}

#if WITH_EDITOR

bool FCollectionVertexSelectionByBoxDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FCollectionVertexSelectionByBoxDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		const FBox& InBox = GetValue(Context, &Box);
		const FTransform& InBoxTransform = GetValue(Context, &BoxTransform);
		UE::Dataflow::SelectionNodes::Private::DebugDrawBox(InBox, InBoxTransform, DataflowRenderingInterface);
	}
}

#endif

//////////////////////////////////////////////////////////////////////////////////////////////////

FCollectionVertexSelectionBySphereDataflowNode::FCollectionVertexSelectionBySphereDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Sphere);

	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&VertexSelection);
}

void FCollectionVertexSelectionBySphereDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA(&VertexSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const FSphere& InSphere = GetValue(Context, &Sphere);

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		TArray<bool> SelectedVertices;
		TransformSelectionFacade.SelectVerticesInSphere(InSphere, SelectedVertices);

		FDataflowVertexSelection OutVertexSelection;
		OutVertexSelection.InitializeFromCollection(InCollection, false);
		OutVertexSelection.SetFromArray(SelectedVertices);

		SetValue(Context, MoveTemp(OutVertexSelection), &VertexSelection);
	}
}

#if WITH_EDITOR

bool FCollectionVertexSelectionBySphereDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FCollectionVertexSelectionBySphereDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		const FSphere& InSphere = GetValue(Context, &Sphere);
		UE::Dataflow::SelectionNodes::Private::DebugDrawSphere(InSphere, DataflowRenderingInterface);
	}
}

#endif

//////////////////////////////////////////////////////////////////////////////////////////////////

FCollectionVertexSelectionByPlaneDataflowNode::FCollectionVertexSelectionByPlaneDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&PlaneTransform);

	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&VertexSelection);
}

void FCollectionVertexSelectionByPlaneDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA(&VertexSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const FTransform& InPlaneTransform = GetValue(Context, &PlaneTransform);

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		TArray<bool> SelectedVertices;

		const FPlane InPlane(InPlaneTransform.GetLocation(), InPlaneTransform.GetUnitAxis(EAxis::Z));
		TransformSelectionFacade.SelectVerticesOnPlaneSide(InPlane, bPositiveSide, SelectedVertices);

		FDataflowVertexSelection OutVertexSelection;
		OutVertexSelection.InitializeFromCollection(InCollection, false);
		OutVertexSelection.SetFromArray(SelectedVertices);

		SetValue(Context, MoveTemp(OutVertexSelection), &VertexSelection);
	}
}

#if WITH_EDITOR

bool FCollectionVertexSelectionByPlaneDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FCollectionVertexSelectionByPlaneDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		const FTransform& InPlaneTransform = GetValue(Context, &PlaneTransform);
		UE::Dataflow::SelectionNodes::Private::DebugDrawPlane(InPlaneTransform, PlaneSize, PlaneResolution, DataflowRenderingInterface);
	}
}

#endif


//////////////////////////////////////////////////////////////////////////////////////////////////

void FCollectionTransformSelectionInSphereDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const FSphere& InSphere = GetValue<FSphere>(Context, &Sphere);
		const FTransform& InTransform = GetValue<FTransform>(Context, &Transform);

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);

		TArray<int32> SelectionArr;
		if (Type == ESelectSubjectTypeEnum::Dataflow_SelectSubjectType_Vertices)
		{
			SelectionArr = TransformSelectionFacade.SelectVerticesInSphere(InSphere, InTransform, bAllVerticesMustContainedInSphere);
		}
		else if (Type == ESelectSubjectTypeEnum::Dataflow_SelectSubjectType_BoundingBox)
		{
			SelectionArr = TransformSelectionFacade.SelectBoundingBoxInSphere(InSphere, InTransform);
		}
		else if (Type == ESelectSubjectTypeEnum::Dataflow_SelectSubjectType_Centroid)
		{
			SelectionArr = TransformSelectionFacade.SelectCentroidInSphere(InSphere, InTransform);
		}

		FDataflowTransformSelection NewTransformSelection;
		NewTransformSelection.InitializeFromCollection(InCollection, false);
		NewTransformSelection.SetFromArray(SelectionArr);

		SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

#if WITH_EDITOR

bool FCollectionTransformSelectionInSphereDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FCollectionTransformSelectionInSphereDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		const FSphere& InSphere = GetValue<FSphere>(Context, &Sphere);
		const FTransform& InTransform = GetValue<FTransform>(Context, &Transform);

		DataflowRenderingInterface.SetLineWidth(1.0);
		DataflowRenderingInterface.SetWireframe(true);
		DataflowRenderingInterface.SetWorldPriority();
		DataflowRenderingInterface.SetColor(FLinearColor::Red);

		const FVector TransformedCenter = InSphere.Center + InTransform.GetTranslation();
		const double ScaledRadius = InSphere.W * InTransform.GetScale3D().GetMax();
		DataflowRenderingInterface.DrawSphere(TransformedCenter, ScaledRadius);
	}
}

#endif

//////////////////////////////////////////////////////////////////////////////////////////////////

void FCollectionTransformSelectionByFloatAttrDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		float InMin = GetValue<float>(Context, &Min);
		float InMax = GetValue<float>(Context, &Max);
		bool bInsideRange = RangeSetting == ERangeSettingEnum::Dataflow_RangeSetting_InsideRange;

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectByFloatAttribute(GroupName, AttrName, InMin, InMax, bInclusive, bInsideRange);

		FDataflowTransformSelection NewTransformSelection;
		NewTransformSelection.InitializeFromCollection(InCollection, false);
		NewTransformSelection.SetFromArray(SelectionArr);

		SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

void FSelectFloatArrayIndicesInRangeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Indices))
	{
		const TArray<float>& InValues = GetValue(Context, &Values);
		float InMin = GetValue(Context, &Min);
		float InMax = GetValue(Context, &Max);
		bool bInsideRange = RangeSetting == ERangeSettingEnum::Dataflow_RangeSetting_InsideRange;

		TArray<int32> OutIndices;
		for (int32 Idx = 0; Idx < InValues.Num(); ++Idx)
		{
			const float FloatValue = InValues[Idx];

			if (bInsideRange && FloatValue > Min && FloatValue < Max)
			{
				OutIndices.Add(Idx);
			}
			else if (!bInsideRange && (FloatValue < Min || FloatValue > Max))
			{
				OutIndices.Add(Idx);
			}
			else if (bInclusive && (FloatValue == Min || FloatValue == Max))
			{
				OutIndices.Add(Idx);
			}
		}

		SetValue(Context, MoveTemp(OutIndices), &Indices);
	}
}

void FCollectionTransformSelectionByIntAttrDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowTransformSelection>(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		int32 InMin = GetValue<int32>(Context, &Min);
		int32 InMax = GetValue<int32>(Context, &Max);
		bool bInsideRange = RangeSetting == ERangeSettingEnum::Dataflow_RangeSetting_InsideRange;

		GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
		const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectByIntAttribute(GroupName, AttrName, InMin, InMax, bInclusive, bInsideRange);

		FDataflowTransformSelection NewTransformSelection;
		NewTransformSelection.InitializeFromCollection(InCollection, false);
		NewTransformSelection.SetFromArray(SelectionArr);

		SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}


void FCollectionVertexSelectionCustomDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowVertexSelection>(&VertexSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		if (InCollection.HasGroup(FGeometryCollection::VerticesGroup))
		{
			FDataflowVertexSelection NewVertexSelection;
			NewVertexSelection.InitializeFromCollection(InCollection, false);

			const FString InVertexIndices = GetValue<FString>(Context, &VertexIndicies);

			TArray<FString> Indices;
			InVertexIndices.ParseIntoArray(Indices, TEXT(" "), true);

			for (FString IndexStr : Indices)
			{
				if (IndexStr.IsNumeric())
				{
					int32 Index = FCString::Atoi(*IndexStr);
					if (NewVertexSelection.IsValidIndex(Index))
					{
						NewVertexSelection.SetSelected(Index);
					}
					else
					{
						// ERROR: INVALID INDEX
						Context.Error(FString::Printf(
							TEXT("Index %d is not a valid vertex index for vertex group size %d."),
							Index, NewVertexSelection.Num()),
							this);
					}
				}
			}

			SetValue(Context, MoveTemp(NewVertexSelection), &VertexSelection);
		}
		else
		{
			SetValue(Context, FDataflowVertexSelection(), &VertexSelection);
		}
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}


void FCollectionFaceSelectionCustomDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowFaceSelection>(&FaceSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		if (InCollection.HasGroup(FGeometryCollection::FacesGroup))
		{
			FDataflowFaceSelection NewFaceSelection;
			NewFaceSelection.InitializeFromCollection(InCollection, false);

			const FString InFaceIndices = GetValue<FString>(Context, &FaceIndicies);

			TArray<FString> Indices;
			InFaceIndices.ParseIntoArray(Indices, TEXT(" "), true);

			for (FString& IndexStr : Indices)
			{
				if (IndexStr.IsNumeric())
				{
					const int32 Index = FCString::Atoi(*IndexStr);
					if (NewFaceSelection.IsValidIndex(Index))
					{
						NewFaceSelection.SetSelected(Index);
					}
					else
					{
						Context.Error(FString::Printf(
							TEXT("Index %d is not a valid vertex index for face group size %d."),
							Index, NewFaceSelection.Num()),
							this);
					}
				}
			}

			SetValue(Context, MoveTemp(NewFaceSelection), &FaceSelection);
		}
		else
		{
			SetValue(Context, FDataflowFaceSelection(), &FaceSelection);
		}
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}


void FCollectionSelectionConvertDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&TransformSelection))
	{
		if (IsConnected(&VertexSelection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
			const FDataflowVertexSelection& InVertexSelection = GetValue(Context, &VertexSelection);

			GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
			const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertVertexSelectionToTransformSelection(InVertexSelection.AsArray(), bAllElementsMustBeSelected);

			FDataflowTransformSelection NewTransformSelection;
			NewTransformSelection.InitializeFromCollection(InCollection, false);
			NewTransformSelection.SetFromArray(SelectionArr);

			SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
		}
		else if (IsConnected(&FaceSelection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
			const FDataflowFaceSelection& InFaceSelection = GetValue(Context, &FaceSelection);

			GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
			const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertFaceSelectionToTransformSelection(InFaceSelection.AsArray(), bAllElementsMustBeSelected);

			FDataflowTransformSelection NewTransformSelection;
			NewTransformSelection.InitializeFromCollection(InCollection, false);
			NewTransformSelection.SetFromArray(SelectionArr);

			SetValue(Context, MoveTemp(NewTransformSelection), &TransformSelection);
		}
		else
		{
			// Passthrough
			SafeForwardInput(Context, &TransformSelection, &TransformSelection);
		}
	}
	else if (Out->IsA(&FaceSelection))
	{
		if (IsConnected(&VertexSelection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
			const FDataflowVertexSelection& InVertexSelection = GetValue(Context, &VertexSelection);

			GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
			const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertVertexSelectionToFaceSelection(InVertexSelection.AsArray(), bAllElementsMustBeSelected);

			FDataflowFaceSelection NewFaceSelection;
			NewFaceSelection.InitializeFromCollection(InCollection, false);
			NewFaceSelection.SetFromArray(SelectionArr);

			SetValue(Context, MoveTemp(NewFaceSelection), &FaceSelection);
		}
		else if (IsConnected(&TransformSelection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
			const FDataflowTransformSelection& InTransformSelection = GetValue(Context, &TransformSelection);

			GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
			const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertTransformSelectionToFaceSelection(InTransformSelection.AsArray());

			FDataflowFaceSelection NewFaceSelection;
			NewFaceSelection.InitializeFromCollection(InCollection, false);
			NewFaceSelection.SetFromArray(SelectionArr);

			SetValue(Context, MoveTemp(NewFaceSelection), &FaceSelection);
		}
		else
		{
			// Passthrough
			SafeForwardInput(Context, &FaceSelection, &FaceSelection);
		}
	}
	else if (Out->IsA(&VertexSelection))
	{
		if (IsConnected(&FaceSelection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
			const FDataflowFaceSelection& InFaceSelection = GetValue(Context, &FaceSelection);

			GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
			const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertFaceSelectionToVertexSelection(InFaceSelection.AsArray());

			FDataflowVertexSelection NewVertexSelection;
			NewVertexSelection.InitializeFromCollection(InCollection, false);
			NewVertexSelection.SetFromArray(SelectionArr);

			SetValue(Context, MoveTemp(NewVertexSelection), &VertexSelection);
		}
		else if (IsConnected(&TransformSelection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
			const FDataflowTransformSelection& InTransformSelection = GetValue(Context, &TransformSelection);

			GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
			const TArray<int32>& SelectionArr = TransformSelectionFacade.ConvertTransformSelectionToVertexSelection(InTransformSelection.AsArray());

			FDataflowVertexSelection NewVertexSelection;
			NewVertexSelection.InitializeFromCollection(InCollection, false);
			NewVertexSelection.SetFromArray(SelectionArr);

			SetValue(Context, MoveTemp(NewVertexSelection), &VertexSelection);
		}
		else
		{
			// Passthrough
			SafeForwardInput(Context, &VertexSelection, &VertexSelection);
		}
	}
	else if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}


void FCollectionFaceSelectionInvertDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowFaceSelection>(&FaceSelection))
	{
		FDataflowFaceSelection InFaceSelection = GetValue<FDataflowFaceSelection>(Context, &FaceSelection);

		InFaceSelection.Invert();

		SetValue<FDataflowFaceSelection>(Context, MoveTemp(InFaceSelection), &FaceSelection);
	}
}


void FCollectionVertexSelectionByPercentageDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowVertexSelection>(&VertexSelection))
	{
		FDataflowVertexSelection InVertexSelection = GetValue<FDataflowVertexSelection>(Context, &VertexSelection);

		int32 InPercentage = GetValue<int32>(Context, &Percentage);
		float InRandomSeed = GetValue<float>(Context, &RandomSeed);

		TArray<int32> SelectionArr = InVertexSelection.AsArray();

		GeometryCollection::Facades::FCollectionTransformSelectionFacade::SelectByPercentage(SelectionArr, InPercentage, bDeterministic, (int32)InRandomSeed);

		InVertexSelection.SetFromArray(SelectionArr);
		SetValue(Context, MoveTemp(InVertexSelection), &VertexSelection);
	}
}

void FCollectionVertexSelectionSetOperationDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowVertexSelection>(&VertexSelection))
	{
		const FDataflowVertexSelection& InVertexSelectionA = GetValue<FDataflowVertexSelection>(Context, &VertexSelectionA);
		const FDataflowVertexSelection& InVertexSelectionB = GetValue<FDataflowVertexSelection>(Context, &VertexSelectionB);

		FDataflowVertexSelection NewVertexSelection;

		if (InVertexSelectionA.Num() == InVertexSelectionB.Num())
		{
			if (Operation == ESetOperationEnum::Dataflow_SetOperation_AND)
			{
				InVertexSelectionA.AND(InVertexSelectionB, NewVertexSelection);
			}
			else if (Operation == ESetOperationEnum::Dataflow_SetOperation_OR)
			{
				InVertexSelectionA.OR(InVertexSelectionB, NewVertexSelection);
			}
			else if (Operation == ESetOperationEnum::Dataflow_SetOperation_XOR)
			{
				InVertexSelectionA.XOR(InVertexSelectionB, NewVertexSelection);
			}
			else if (Operation == ESetOperationEnum::Dataflow_SetOperation_Subtract)
			{
				InVertexSelectionA.Subtract(InVertexSelectionB, NewVertexSelection);
			}
		}
		else
		{
			// ERROR: INPUT TRANSFORMSELECTIONS HAVE DIFFERENT NUMBER OF ELEMENTS
			FString ErrorStr = "Input VertexSelections have different number of elements.";
			UE_LOGF(LogTemp, Error, "[Dataflow ERROR] %ls", *ErrorStr);
		}

		SetValue(Context, MoveTemp(NewVertexSelection), &VertexSelection);
	}
}

static void CreateSelectionFromAttr(const FManagedArrayCollection& InCollection,
	const FName InGroup,
	const FName InAttribute,
	const FString InValue,
	const ESelectionByAttrOperation InOperation,
	FDataflowSelection& OutSelection)
{
	const FManagedArrayCollection::EArrayType ArrayType = InCollection.GetAttributeType(InAttribute, InGroup);
	const int32 NumElements = InCollection.NumElements(InGroup);

	const bool bIsMinOrMaxOperation = (InOperation == ESelectionByAttrOperation::Maximum) || (InOperation == ESelectionByAttrOperation::Minimum);

	if (ArrayType == FManagedArrayCollection::EArrayType::FFloatType)
	{
		const TManagedArray<float>* const Array = InCollection.FindAttributeTyped<float>(InAttribute, InGroup);
		if (InValue.IsNumeric() || bIsMinOrMaxOperation)
		{
			if (InOperation == ESelectionByAttrOperation::Maximum)
			{
				float MaxValue = TNumericLimits<float>::Lowest();
				int32 MaxIndex = INDEX_NONE;
				for (int32 Idx = 0; Idx < NumElements; ++Idx)
				{
					const float ValueAtIdx = (*Array)[Idx];
					if (ValueAtIdx >= MaxValue)
					{
						MaxValue = ValueAtIdx;
						MaxIndex = Idx;
					}
				}
				if (MaxIndex != INDEX_NONE)
				{
					OutSelection.SetSelected(MaxIndex);
				}
			}
			else if (InOperation == ESelectionByAttrOperation::Minimum)
			{
				float MinValue = TNumericLimits<float>::Max();
				int32 MinIndex = INDEX_NONE;
				for (int32 Idx = 0; Idx < NumElements; ++Idx)
				{
					const float ValueAtIdx = (*Array)[Idx];
					if (ValueAtIdx <= MinValue)
					{
						MinValue = ValueAtIdx;
						MinIndex = Idx;
					}
				}
				if (MinIndex != INDEX_NONE)
				{
					OutSelection.SetSelected(MinIndex);
				}
			}
			else
			{
				float FloatValue = FCString::Atof(*InValue);

				for (int32 Idx = 0; Idx < NumElements; ++Idx)
				{
					if ((InOperation == ESelectionByAttrOperation::Equal && (*Array)[Idx] == FloatValue) ||
						(InOperation == ESelectionByAttrOperation::NotEqual && (*Array)[Idx] != FloatValue) ||
						(InOperation == ESelectionByAttrOperation::Greater && (*Array)[Idx] > FloatValue) ||
						(InOperation == ESelectionByAttrOperation::GreaterOrEqual && (*Array)[Idx] >= FloatValue) ||
						(InOperation == ESelectionByAttrOperation::Smaller && (*Array)[Idx] < FloatValue) ||
						(InOperation == ESelectionByAttrOperation::SmallerOrEqual && (*Array)[Idx] <= FloatValue))
					{
						OutSelection.SetSelected(Idx);
					}
				}
			}
		}
		else
		{
			// Error: Invalid Value specified
			return;
		}
	}
	else if (ArrayType == FManagedArrayCollection::EArrayType::FInt32Type)
	{
		const TManagedArray<int32>* const Array = InCollection.FindAttributeTyped<int32>(InAttribute, InGroup);
		if (InValue.IsNumeric() || bIsMinOrMaxOperation)
		{
			if (InOperation == ESelectionByAttrOperation::Maximum)
			{
				int32 MaxValue = TNumericLimits<int32>::Lowest();
				int32 MaxIndex = INDEX_NONE;
				for (int32 Idx = 0; Idx < NumElements; ++Idx)
				{
					const int32 ValueAtIdx = (*Array)[Idx];
					if (ValueAtIdx >= MaxValue)
					{
						MaxValue = ValueAtIdx;
						MaxIndex = Idx;
					}
				}
				if (MaxIndex != INDEX_NONE)
				{
					OutSelection.SetSelected(MaxIndex);
				}
			}
			else if (InOperation == ESelectionByAttrOperation::Minimum)
			{
				int32 MinValue = TNumericLimits<int32>::Max();
				int32 MinIndex = INDEX_NONE;
				for (int32 Idx = 0; Idx < NumElements; ++Idx)
				{
					const int32 ValueAtIdx = (*Array)[Idx];
					if (ValueAtIdx <= MinValue)
					{
						MinValue = ValueAtIdx;
						MinIndex = Idx;
					}
				}
				if (MinIndex != INDEX_NONE)
				{
					OutSelection.SetSelected(MinIndex);
				}
			}
			else
			{
				float IntValue = FCString::Atoi(*InValue);

				for (int32 Idx = 0; Idx < NumElements; ++Idx)
				{
					if ((InOperation == ESelectionByAttrOperation::Equal && (*Array)[Idx] == IntValue) ||
						(InOperation == ESelectionByAttrOperation::NotEqual && (*Array)[Idx] != IntValue) ||
						(InOperation == ESelectionByAttrOperation::Greater && (*Array)[Idx] > IntValue) ||
						(InOperation == ESelectionByAttrOperation::GreaterOrEqual && (*Array)[Idx] >= IntValue) ||
						(InOperation == ESelectionByAttrOperation::Smaller && (*Array)[Idx] < IntValue) ||
						(InOperation == ESelectionByAttrOperation::SmallerOrEqual && (*Array)[Idx] <= IntValue))
					{
						OutSelection.SetSelected(Idx);
					}
				}
			}
		}
		else
		{
			// Error: Invalid Value specified
			return;
		}
	}
	else if (ArrayType == FManagedArrayCollection::EArrayType::FStringType)
	{
		const TManagedArray<FString>* const Array = InCollection.FindAttributeTyped<FString>(InAttribute, InGroup);

		for (int32 Idx = 0; Idx < NumElements; ++Idx)
		{
			if ((InOperation == ESelectionByAttrOperation::Equal && (*Array)[Idx] == InValue) ||
				(InOperation == ESelectionByAttrOperation::NotEqual && !((*Array)[Idx] == InValue)))
			{
				OutSelection.SetSelected(Idx);
			}
		}
	}
	else if (ArrayType == FManagedArrayCollection::EArrayType::FBoolType)
	{
		const TManagedArray<bool>* const Array = InCollection.FindAttributeTyped<bool>(InAttribute, InGroup);
		bool BoolValue = false;
		if (InValue.IsNumeric())
		{
			float FloatValue = FCString::Atof(*InValue);

			if (FloatValue > 0.f)
			{
				BoolValue = true;
			}
		}
		else
		{
			if (InValue == FString("true") || InValue == FString("True"))
			{
				BoolValue = true;
			}
		}

		for (int32 Idx = 0; Idx < NumElements; ++Idx)
		{
			if ((InOperation == ESelectionByAttrOperation::Equal && (*Array)[Idx] == BoolValue) ||
				(InOperation == ESelectionByAttrOperation::NotEqual && !((*Array)[Idx] == BoolValue)))
			{
				OutSelection.SetSelected(Idx);
			}
		}
	}
}

void FCollectionSelectionByAttrDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&VertexSelection) ||
		Out->IsA(&FaceSelection) ||
		Out->IsA(&TransformSelection) ||
		Out->IsA(&GeometrySelection) ||
		Out->IsA(&MaterialSelection) ||
		Out->IsA(&CurveSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		FCollectionAttributeKey InAttributeKey = GetValue(Context, &AttributeKey);
		FName GroupName = UE::Dataflow::Private::GetAttributeFromEnumAsName(Group);
		FName AttributeName = FName(Attribute);
		if (IsConnected(&AttributeKey))
		{
			GroupName = FName(InAttributeKey.Group);
			AttributeName = FName(InAttributeKey.Attribute);
		}

		if (InCollection.HasGroup(GroupName))
		{
			if (InCollection.HasAttribute(AttributeName, GroupName))
			{
				const int32 GroupSize = InCollection.NumElements(GroupName);

				FDataflowSelection NewGenericSelection;
				NewGenericSelection.Initialize(GroupSize, false);

				CreateSelectionFromAttr(InCollection,
					GroupName,
					AttributeName,
					Value,
					Operation,
					NewGenericSelection);

				using namespace UE::Dataflow::Private;

				FDataflowVertexSelection OutVertexSelection;
				if (GroupName == UE::Dataflow::Private::GetAttributeFromEnumAsName(ESelectionByAttrGroup::Vertices))
				{
					OutVertexSelection.Initialize(NewGenericSelection);
				}
				SetValue(Context, MoveTemp(OutVertexSelection), &VertexSelection);

				FDataflowFaceSelection OutFaceSelection;
				if (GroupName == UE::Dataflow::Private::GetAttributeFromEnumAsName(ESelectionByAttrGroup::Faces))
				{
					OutFaceSelection.Initialize(NewGenericSelection);
				}
				SetValue(Context, MoveTemp(OutFaceSelection), &FaceSelection);

				FDataflowTransformSelection OutTransformSelection;
				if (GroupName == UE::Dataflow::Private::GetAttributeFromEnumAsName(ESelectionByAttrGroup::Transform))
				{
					OutTransformSelection.Initialize(NewGenericSelection);
				}
				SetValue(Context, MoveTemp(OutTransformSelection), &TransformSelection);

				FDataflowGeometrySelection OutGeometrySelection;
				if (GroupName == UE::Dataflow::Private::GetAttributeFromEnumAsName(ESelectionByAttrGroup::Geometry))
				{
					OutGeometrySelection.Initialize(NewGenericSelection);
				}
				SetValue(Context, MoveTemp(OutGeometrySelection), &GeometrySelection);

				FDataflowMaterialSelection OutMaterialSelection;
				if (GroupName == UE::Dataflow::Private::GetAttributeFromEnumAsName(ESelectionByAttrGroup::Material))
				{
					OutMaterialSelection.Initialize(NewGenericSelection);
				}
				SetValue(Context, MoveTemp(OutMaterialSelection), &MaterialSelection);
				
				FDataflowCurveSelection OutCurveSelection;
				if (GroupName == UE::Dataflow::Private::GetAttributeFromEnumAsName(ESelectionByAttrGroup::Curves))
				{
					OutCurveSelection.Initialize(NewGenericSelection);
				}
				SetValue(Context, MoveTemp(OutCurveSelection), &CurveSelection);

				return;
			}
		}

		SetValue(Context, FDataflowVertexSelection(), &VertexSelection);
		SetValue(Context, FDataflowFaceSelection(), &FaceSelection);
		SetValue(Context, FDataflowTransformSelection(), &TransformSelection);
		SetValue(Context, FDataflowGeometrySelection(), &GeometrySelection);
		SetValue(Context, FDataflowMaterialSelection(), &MaterialSelection);
		SetValue(Context, FDataflowCurveSelection(), &CurveSelection);
	}
	else if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

void FGeometrySelectionToVertexSelectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FDataflowVertexSelection>(&VertexSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const int32 NumGeometries = InCollection.NumElements(FGeometryCollection::GeometryGroup);
		
		FDataflowVertexSelection InVertexSelection;
		InVertexSelection.InitializeFromCollection(InCollection, false);
		const TManagedArray<int32>* VertexStart = InCollection.FindAttributeTyped<int32>("VertexStart", FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>* VertexCount = InCollection.FindAttributeTyped<int32>("VertexCount", FGeometryCollection::GeometryGroup);
		TArray<int32> InGeometryIndexArray;
		if (IsConnected(&GeometrySelection))
		{
			InGeometryIndexArray = GetValue<FDataflowGeometrySelection>(Context, &GeometrySelection).AsArray();
		}
		else
		{
			TArray<FString> Indices;
			GeometryIndices.ParseIntoArray(Indices, TEXT(" "), true);
			for (FString IndexStr : Indices)
			{
				if (IndexStr.IsNumeric())
				{
					int32 Index = FCString::Atoi(*IndexStr);
					if (Index >= 0 && Index < NumGeometries)
					{
						InGeometryIndexArray.Add(Index);
					}
					else
					{
						// ERROR: INVALID INDEX
						FString ErrorStr = "Invalid geometry index found.";
						UE_LOGF(LogTemp, Error, "[Dataflow ERROR] %ls", *ErrorStr);
					}
				}
			}
		}
		if (VertexStart && VertexCount)
		{
			TArray<int32> VertexIndices;
			for (int32 GeometryIdx : InGeometryIndexArray)
			{
				if (ensure(VertexStart->IsValidIndex(GeometryIdx)))
				{
					const int32 Start = (*VertexStart)[GeometryIdx];
					const int32 Count = (*VertexCount)[GeometryIdx];
					for (int32 VertexIdx = Start; VertexIdx < Start + Count; ++VertexIdx)
					{
						VertexIndices.Add(VertexIdx);
					}
				}
			}
			InVertexSelection.SetFromArray(VertexIndices);
		}
		SetValue(Context, MoveTemp(InVertexSelection), &VertexSelection);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCollectionSelectionSetOperationDataflowNode::FCollectionSelectionSetOperationDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	static const FName TypeDependencyGroup("Main");
	RegisterInputConnection(&SelectionA)
		.SetTypeDependencyGroup(TypeDependencyGroup);
	RegisterInputConnection(&SelectionB)
		.SetTypeDependencyGroup(TypeDependencyGroup);
	RegisterOutputConnection(&Selection, &SelectionA)
		.SetTypeDependencyGroup(TypeDependencyGroup);
}

void FCollectionSelectionSetOperationDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Selection))
	{
		const FDataflowSelection& InSelectionA = GetValue(Context, &SelectionA);
		const FDataflowSelection& InSelectionB = GetValue(Context, &SelectionB);
		ensure(FindInput(&SelectionA)->GetType() == FindInput(&SelectionB)->GetType());
		ensure(FindOutput(&Selection)->GetType() == FindInput(&SelectionA)->GetType());

		FDataflowSelection OutSelection;

		if (InSelectionA.Num() == InSelectionB.Num())
		{
			switch (Operation)
			{
			case ESetOperationEnum::Dataflow_SetOperation_AND:
				InSelectionA.AND(InSelectionB, OutSelection);
				break;
			case ESetOperationEnum::Dataflow_SetOperation_OR:
				InSelectionA.OR(InSelectionB, OutSelection);
				break;
			case ESetOperationEnum::Dataflow_SetOperation_XOR:
				InSelectionA.XOR(InSelectionB, OutSelection);
				break;
			case ESetOperationEnum::Dataflow_SetOperation_Subtract:
				InSelectionA.Subtract(InSelectionB, OutSelection);
				break;
			}
		}
		else
		{
			SetError(Context, &Selection, LOCTEXT("CollectionSelectionSetOperationInputSelectionsMismatch", "Input selections have different number of elements.").ToString());
		}

		SetValue(Context, MoveTemp(OutSelection), &Selection);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCollectionSelectionInvertDataflowNode::FCollectionSelectionInvertDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	static const FName TypeDependencyGroup("Main");
	RegisterInputConnection(&Selection)
		.SetTypeDependencyGroup(TypeDependencyGroup);
	RegisterOutputConnection(&Selection, &Selection)
		.SetTypeDependencyGroup(TypeDependencyGroup);
}

void FCollectionSelectionInvertDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Selection))
	{
		ensure(FindOutput(&Selection)->GetType() == FindInput(&Selection)->GetType());

		FDataflowSelection InSelection = GetValue(Context, &Selection);
		InSelection.Invert();
		SetValue(Context, MoveTemp(InSelection), &Selection);
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCollectionSelectInternalFacesDataflowNode::FCollectionSelectInternalFacesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&TransformSelection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&FaceSelection);
}

void FCollectionSelectInternalFacesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	static const FName InternalAttributeName(TEXT("Internal"));

	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA(&FaceSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const FDataflowTransformSelection& InTransformSelection = GetValue(Context, &TransformSelection);

		struct FFaceRange
		{
			int32 Start;
			int32 Count;
		};

		const TManagedArray<int32>* TransformIndexFromGeometry = InCollection.FindAttribute<int32>("TransformIndex", FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>* FaceStart = InCollection.FindAttribute<int32>("FaceStart", FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>* FaceCount = InCollection.FindAttribute<int32>("FaceCount", FGeometryCollection::GeometryGroup);
		const TManagedArray<bool>* InternalFaces = InCollection.FindAttribute<bool>(InternalAttributeName, FGeometryCollection::FacesGroup);

		const int32 TotalNumFaces = InCollection.NumElements(FGeometryCollection::FacesGroup);

		FDataflowFaceSelection OutFaceSelection;
		OutFaceSelection.InitializeFromCollection(InCollection, false);

		if (TransformIndexFromGeometry && FaceStart && FaceCount && InternalFaces)
		{
			TArray<FFaceRange> FaceRanges;
			if (IsConnected(&TransformSelection))
			{
				for (int32 GeoIdx = 0; GeoIdx < TransformIndexFromGeometry->Num(); ++GeoIdx)
				{
					const int32 TransformIndex = (*TransformIndexFromGeometry)[GeoIdx];
					if (InTransformSelection.IsSelected(TransformIndex))
					{
						FaceRanges.Emplace( 
							FFaceRange { 
								.Start=(*FaceStart)[GeoIdx],
								.Count=(*FaceCount)[GeoIdx] 
							});
					}
				}
			}
			else
			{
				FaceRanges.Emplace(
					FFaceRange{ 
						.Start = 0,
						.Count = TotalNumFaces
					});
			}

			for (const FFaceRange& FaceRange : FaceRanges)
			{
				for (int32 Idx = 0; Idx < FaceRange.Count; ++Idx)
				{
					const int32 FaceIndex = (FaceRange.Start + Idx);
					if ((*InternalFaces)[FaceIndex])
					{
						OutFaceSelection.SetSelected(FaceRange.Start + Idx);
					}
				}
			}
		}
		SetValue(Context, OutFaceSelection, &FaceSelection);
	}
}

/////////////////////////////////////////////////////////////////////////////////

FCollectionSelectTransformStringDataflowNode::FCollectionSelectTransformStringDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&SearchText);
	RegisterInputConnection(&Attribute).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&TransformSelection);
}

void FCollectionSelectTransformStringDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA(&TransformSelection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const FString& InSearchText = GetValue(Context, &SearchText);
		const FString& InAttributeName = GetValue(Context, &Attribute);

		FDataflowTransformSelection OutSelection;
		OutSelection.InitFromArray(InCollection, {});

		const TManagedArray<FString>* StringAttribute = InCollection.FindAttributeTyped<FString>(FName(InAttributeName), FTransformCollection::TransformGroup);
		if (StringAttribute)
		{
			const int32 Num = StringAttribute->Num();
			for (int32 Index = 0; Index < Num; Index++)
			{
				const FString& Value = (*StringAttribute)[Index];

				bool bSelect = false;
				switch (Method)
				{
				case EDataflowCollectionSelectionByNameMethod::Exact:
					bSelect = (Value == InSearchText);
					break;
				case EDataflowCollectionSelectionByNameMethod::StartsWith:
					bSelect = Value.StartsWith(InSearchText, ESearchCase::CaseSensitive);
					break;
				case EDataflowCollectionSelectionByNameMethod::EndsWith:
					bSelect = Value.EndsWith(InSearchText, ESearchCase::CaseSensitive);
					break;
				case EDataflowCollectionSelectionByNameMethod::Contains:
					bSelect = Value.Contains(InSearchText, ESearchCase::CaseSensitive);
					break;
				}
				if (bSelect)
				{
					OutSelection.SetSelected(Index);
				}
				else
				{
					OutSelection.SetNotSelected(Index);
				}
			}
		}
		SetValue(Context, OutSelection, &TransformSelection);
	}
}

//////////////////////////////////////////////////////////////////////////////////////
FCollectionSetTransformStringValueDataflowNode::FCollectionSetTransformStringValueDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&TextToSet);
	RegisterInputConnection(&TransformSelection);
	RegisterInputConnection(&Attribute).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Collection, &Collection);
}

void FCollectionSetTransformStringValueDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection CollectionToUpdate = GetValue(Context, &Collection);
		FString InTextToSet = GetValue(Context, &TextToSet);
		const FString& InAttributeName = GetValue(Context, &Attribute);

		FDataflowTransformSelection InSelection = GetValue(Context, &TransformSelection);
		if (!IsConnected(&TransformSelection))
		{
			InSelection.InitFromArray(CollectionToUpdate, {});
			InSelection.Invert();
		}

		const FString StringFormat = InTextToSet
			.Replace(TEXT("{Current}"), TEXT("{0}"))
			.Replace(TEXT("{Index}"), TEXT("{1}"));

		// make sure we have the right format
		// todo : add the ability to choose the attribute ?
		TManagedArray<FString>* StringAttribute = CollectionToUpdate.FindAttributeTyped<FString>(FName(InAttributeName), FTransformCollection::TransformGroup);
		if (StringAttribute)
		{
			const int32 Num = StringAttribute->Num();
			int32 SelectionIndex = 0;
			for (int32 Index = 0; Index < Num; Index++)
			{
				if (InSelection.IsSelected(Index))
				{
					const FString& CurrentValue = (*StringAttribute)[Index];
					(*StringAttribute)[Index] = FString::Format(*StringFormat, { CurrentValue, SelectionIndex });
					SelectionIndex++;
				}
			}
		}
		SetValue(Context, MoveTemp(CollectionToUpdate), &Collection);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FGetNumElementsInSelectionDataflowNode::FGetNumElementsInSelectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	static const FName MainTypeGroup("Main");

	// Inputs
	RegisterInputConnection(&Selection)
		.SetTypeDependencyGroup(MainTypeGroup);

	// Outputs
	RegisterOutputConnection(&Selection, &Selection)
		.SetTypeDependencyGroup(MainTypeGroup);

	RegisterOutputConnection(&NumElements);
	RegisterOutputConnection(&NumSelectedElements);
}

void FGetNumElementsInSelectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Selection))
	{
		SafeForwardInput(Context, &Selection, &Selection);
	}
	else if (Out->IsA(&NumElements))
	{
		if (IsConnected(&Selection))
		{
			const FDataflowInput* SelectionInput = FindInput(&Selection);
			if (SelectionInput)
			{
				const FDataflowSelection& InSelection = GetValue(Context, &Selection);

				SetValue(Context, InSelection.Num(), &NumElements);

				return;
			}
		}

		SetValue(Context, 0, &NumElements);
	}
	else if (Out->IsA(&NumSelectedElements))
	{
		if (IsConnected(&Selection))
		{
			const FDataflowInput* SelectionInput = FindInput(&Selection);
			if (SelectionInput)
			{
				const FDataflowSelection& InSelection = GetValue(Context, &Selection);

				SetValue(Context, InSelection.NumSelected(), &NumSelectedElements);

				return;
			}
		}

		SetValue(Context, 0, &NumSelectedElements);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

namespace UE::Dataflow::Selection::Private
{
	void ConvertIndicesStrToIntArray(const FString& InIndices, TArray<int32>& OutIndicesArray)
	{
		TArray<FString> IndicesArray;
		InIndices.ParseIntoArray(IndicesArray, TEXT(" "), true);

		for (FString IndexStr : IndicesArray)
		{
			if (IndexStr.IsNumeric())
			{
				OutIndicesArray.Add(FCString::Atoi(*IndexStr));
			}
		}
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FCollectionSelectionCustomDataflowNode::FCollectionSelectionCustomDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Indices);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Selection);
}

void FCollectionSelectionCustomDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Selection))
	{
		if (IsConnected(&Collection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

			const FDataflowOutput* SelectionOutput = FindOutput(&Selection);
			if (SelectionOutput)
			{
				const FName GroupName = FDataflowSelection::GetSelectionTypeFromCollection(*SelectionOutput);
				if (GroupName != NAME_None && InCollection.HasGroup(GroupName))
				{
					FDataflowSelection NewSelection(GroupName);
					NewSelection.Initialize(InCollection.NumElements(GroupName), false);

					const FString InIndices = GetValue(Context, &Indices);

					TArray<int32> IndicesArray;

					using namespace UE::Dataflow::Utils;

					EErrorCode ErrorCode = ParseIndicesStr(InIndices, IndicesArray);

					if (ErrorCode == EErrorCode::None)
					{
						if (!NewSelection.SetSelectedWithCheck(IndicesArray))
						{
							SetError(Context, &Selection, LOCTEXT("CollectionSelectionCustomInvalidIndex", "Invalid index specified.").ToString());
						}
					}
					else
					{
						if (ErrorCode == EErrorCode::InvalidChars)
						{
							SetError(Context, &Selection, LOCTEXT("CollectionSelectionCustomInvalidChars", "Invalid character(s) specified in list.").ToString());
						}
						else if (ErrorCode == EErrorCode::InvalidFormatInSegment)
						{
							SetError(Context, &Selection, LOCTEXT("CollectionSelectionCustomInvalidFormat", "Invalid format in segment.").ToString());
						}
					}

					SetValue(Context, MoveTemp(NewSelection), &Selection);
					return;
				}
			}
		}

		SetValue(Context, FDataflowSelection(), &Selection);
	}
	else if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FCollectionSelectionConvertDataflowNode_v2::FCollectionSelectionConvertDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Selection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Selection, &Selection);
}

void FCollectionSelectionConvertDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Selection))
	{
		if (IsConnected(&Collection) && IsConnected(&Selection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
			GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);

			const FDataflowInput* SelectionInput = FindInput(&Selection);
			const FDataflowOutput* SelectionOutput = FindOutput(&Selection);
			if (SelectionInput && SelectionOutput)
			{
				const FName InGroupName = FDataflowSelection::GetSelectionTypeFromCollection(*SelectionInput);
				const FName OutGroupName = FDataflowSelection::GetSelectionTypeFromCollection(*SelectionOutput);

				if (InGroupName != NAME_None && InCollection.HasGroup(InGroupName) &&
					OutGroupName != NAME_None && InCollection.HasGroup(OutGroupName))
				{
					const FDataflowSelection& InSelection = GetValue(Context, &Selection);
					FDataflowSelection OutSelection(OutGroupName);
					TArray<int32> SelectionArray;

					if (OutGroupName == FDataflowTransformSelection::TransformGroupName)
					{
						if (InGroupName == FDataflowVertexSelection::VerticesGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertVertexSelectionToTransformSelection(InSelection.AsArray(), bAllElementsMustBeSelected);
						}
						else if (InGroupName == FDataflowFaceSelection::FacesGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertFaceSelectionToTransformSelection(InSelection.AsArray(), bAllElementsMustBeSelected);
						}
						else if (InGroupName == FDataflowGeometrySelection::GeometryGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertGeometrySelectionToTransformSelection(InSelection.AsArray());
						}
						else if (InGroupName == FDataflowCurveSelection::CurveGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertCurveSelectionToTransformSelection(InSelection.AsArray(), bAllElementsMustBeSelected);
						}
						else
						{
							SetError(Context, &Selection, LOCTEXT("CollectionSelectionConvertInvalidConversion", "Invalid conversion specified.").ToString());

						}
					}
					else if (OutGroupName == FDataflowFaceSelection::FacesGroupName)
					{
						if (InGroupName == FDataflowVertexSelection::VerticesGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertVertexSelectionToFaceSelection(InSelection.AsArray(), bAllElementsMustBeSelected);
						}
						else if (InGroupName == FDataflowTransformSelection::TransformGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertTransformSelectionToFaceSelection(InSelection.AsArray());
						}
						else if (InGroupName == FDataflowGeometrySelection::GeometryGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertGeometrySelectionToFaceSelection(InSelection.AsArray());
						}
						else if (InGroupName == FDataflowCurveSelection::CurveGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertCurveSelectionToFaceSelection(InSelection.AsArray());
						}
						else
						{
							SetError(Context, &Selection, LOCTEXT("CollectionSelectionConvertInvalidConversion", "Invalid conversion specified.").ToString());
						}
					}
					else if (OutGroupName == FDataflowVertexSelection::VerticesGroupName)
					{
						if (InGroupName == FDataflowFaceSelection::FacesGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertFaceSelectionToVertexSelection(InSelection.AsArray());
						}
						else if (InGroupName == FDataflowTransformSelection::TransformGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertTransformSelectionToVertexSelection(InSelection.AsArray());
						}
						else if (InGroupName == FDataflowGeometrySelection::GeometryGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertGeometrySelectionToVertexSelection(InSelection.AsArray());
						}
						else if (InGroupName == FDataflowCurveSelection::CurveGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertCurveSelectionToVertexSelection(InSelection.AsArray());
						}
						else
						{
							SetError(Context, &Selection, LOCTEXT("CollectionSelectionConvertInvalidConversion", "Invalid conversion specified.").ToString());
						}
					}
					else if (OutGroupName == FDataflowGeometrySelection::GeometryGroupName)
					{
						if (InGroupName == FDataflowFaceSelection::FacesGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertFaceSelectionToGeometrySelection(InSelection.AsArray(), bAllElementsMustBeSelected);
						}
						else if (InGroupName == FDataflowTransformSelection::TransformGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertTransformSelectionToGeometrySelection(InSelection.AsArray());
						}
						else if (InGroupName == FDataflowVertexSelection::VerticesGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertVertexSelectionToGeometrySelection(InSelection.AsArray(), bAllElementsMustBeSelected);
						}
						else if (InGroupName == FDataflowCurveSelection::CurveGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertCurveSelectionToGeometrySelection(InSelection.AsArray(), bAllElementsMustBeSelected);
						}
						else
						{
							SetError(Context, &Selection, LOCTEXT("CollectionSelectionConvertInvalidConversion", "Invalid conversion specified.").ToString());
						}
					}
					else if (OutGroupName == FDataflowCurveSelection::CurveGroupName)
					{
						if (InGroupName == FDataflowFaceSelection::FacesGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertFaceSelectionToCurveSelection(InSelection.AsArray(), bAllElementsMustBeSelected);
						}
						else if (InGroupName == FDataflowTransformSelection::TransformGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertTransformSelectionToCurveSelection(InSelection.AsArray());
						}
						else if (InGroupName == FDataflowVertexSelection::VerticesGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertVertexSelectionToCurveSelection(InSelection.AsArray(), bAllElementsMustBeSelected);
						}
						else if (InGroupName == FDataflowGeometrySelection::GeometryGroupName)
						{
							SelectionArray = TransformSelectionFacade.ConvertGeometrySelectionToCurveSelection(InSelection.AsArray());
						}
						else
						{
							SetError(Context, &Selection, LOCTEXT("CollectionSelectionConvertInvalidConversion", "Invalid conversion specified.").ToString());
						}
					}

					OutSelection.Initialize(InCollection.NumElements(OutGroupName), false);

					OutSelection.SetFromArray(SelectionArray);

					SetValue(Context, MoveTemp(OutSelection), &Selection);
					return;
				}
			}
		}

		SetValue(Context, FDataflowSelection(), &Selection);
	}
	else if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FCollectionSelectionAllDataflowNode::FCollectionSelectionAllDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Selection);
}

void FCollectionSelectionAllDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Selection))
	{
		if (IsConnected(&Collection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

			const FDataflowOutput* SelectionOutput = FindOutput(&Selection);
			if (SelectionOutput)
			{
				const FName OutGroupName = FDataflowSelection::GetSelectionTypeFromCollection(*SelectionOutput);
				if (OutGroupName != NAME_None && InCollection.HasGroup(OutGroupName))
				{
					FDataflowSelection OutSelection(OutGroupName);
					OutSelection.Initialize(InCollection.NumElements(OutGroupName), true);

					SetValue(Context, MoveTemp(OutSelection), &Selection);
					return;
				}
				else
				{
					SetError(Context, &Selection, LOCTEXT("CollectionSelectionAllGroupDoesntExist", "Selected group type doesn't exist in the collection.").ToString());
				}
			}
		}

		SetValue(Context, FDataflowSelection(), &Selection);
	}
	else if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FCollectionSelectionNoneDataflowNode::FCollectionSelectionNoneDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Selection);
}

void FCollectionSelectionNoneDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Selection))
	{
		if (IsConnected(&Collection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

			const FDataflowOutput* SelectionOutput = FindOutput(&Selection);
			if (SelectionOutput)
			{
				const FName OutGroupName = FDataflowSelection::GetSelectionTypeFromCollection(*SelectionOutput);
				if (OutGroupName != NAME_None && InCollection.HasGroup(OutGroupName))
				{
					FDataflowSelection OutSelection(OutGroupName);
					OutSelection.Initialize(InCollection.NumElements(OutGroupName), false);

					SetValue(Context, MoveTemp(OutSelection), &Selection);
					return;
				}
				else
				{
					SetError(Context, &Selection, LOCTEXT("CollectionSelectionNoneGroupDoesntExist", "Selected group type doesn't exist in the collection.").ToString());
				}
			}
		}

		SetValue(Context, FDataflowSelection(), &Selection);
	}
	else if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FCollectionSelectionRandomDataflowNode::FCollectionSelectionRandomDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&RandomSeed);
	RegisterInputConnection(&RandomThreshold);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Selection);
}

void FCollectionSelectionRandomDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Selection))
	{
		if (IsConnected(&Collection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

			const FDataflowOutput* SelectionOutput = FindOutput(&Selection);
			if (SelectionOutput)
			{
				const FName OutGroupName = FDataflowSelection::GetSelectionTypeFromCollection(*SelectionOutput);
				if (OutGroupName != NAME_None && InCollection.HasGroup(OutGroupName))
				{
					const int32 NumElems = InCollection.NumElements(OutGroupName);

					FDataflowSelection OutSelection(OutGroupName);
					OutSelection.Initialize(NumElems, false);

					const int32 InRandomSeed = GetValue(Context, &RandomSeed);
					const float InRandomThreshold = GetValue(Context, &RandomThreshold);

					GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
					const TArray<int32>& SelectionArray = TransformSelectionFacade.SelectRandom(NumElems, bDeterministic, InRandomSeed, InRandomThreshold);

					OutSelection.SetFromArray(SelectionArray);

					SetValue(Context, MoveTemp(OutSelection), &Selection);
					return;
				}
				else
				{
					SetError(Context, &Selection, LOCTEXT("CollectionSelectionRandomGroupDoesntExist", "Selected group type doesn't exist in the collection.").ToString());
				}
			}
		}

		SetValue(Context, FDataflowSelection(), &Selection);
	}
	else if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FCollectionSelectionByPercentageDataflowNode::FCollectionSelectionByPercentageDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	static const FName TypeDependencyGroup("Main");

	RegisterInputConnection(&Selection)
		.SetTypeDependencyGroup(TypeDependencyGroup);
	RegisterInputConnection(&Percentage);
	RegisterInputConnection(&RandomSeed);
	RegisterOutputConnection(&Selection, &Selection)
		.SetTypeDependencyGroup(TypeDependencyGroup);
}

void FCollectionSelectionByPercentageDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Selection))
	{
		const FDataflowSelection& InSelection = GetValue(Context, &Selection);

		const FDataflowOutput* SelectionOutput = FindOutput(&Selection);
		if (SelectionOutput)
		{
			const FName OutGroupName = FDataflowSelection::GetSelectionTypeFromCollection(*SelectionOutput);
			if (OutGroupName != NAME_None)
			{
				FDataflowSelection OutSelection(OutGroupName);
				OutSelection.Initialize(InSelection);

				const int32 InPercentage = GetValue(Context, &Percentage);
				const int32 InRandomSeed = GetValue(Context, &RandomSeed);

				TArray<int32> SelectionArray = OutSelection.AsArray();

				GeometryCollection::Facades::FCollectionTransformSelectionFacade::SelectByPercentage(SelectionArray, InPercentage, bDeterministic, InRandomSeed);

				OutSelection.SetFromArray(SelectionArray);

				SetValue(Context, MoveTemp(OutSelection), &Selection);
				return;
			}
		}

		SetValue(Context, FDataflowSelection(), &Selection);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FCollectionSelectionByAttributeDataflowNode::FCollectionSelectionByAttributeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Selection);
}

void FCollectionSelectionByAttributeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Selection))
	{
		if (IsConnected(&Collection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
			const FName AttributeName = FName(Attribute);

			const FDataflowOutput* SelectionOutput = FindOutput(&Selection);
			if (SelectionOutput)
			{
				const FName OutGroupName = FDataflowSelection::GetSelectionTypeFromCollection(*SelectionOutput);
				if (OutGroupName != NAME_None && InCollection.HasGroup(OutGroupName))
				{
					if (InCollection.HasAttribute(AttributeName, OutGroupName))
					{
						const int32 NumElems = InCollection.NumElements(OutGroupName);

						FDataflowSelection OutSelection(OutGroupName);
						OutSelection.Initialize(NumElems, false);

						CreateSelectionFromAttr(InCollection,
							OutGroupName,
							AttributeName,
							Value,
							Operation,
							OutSelection);

						SetValue(Context, MoveTemp(OutSelection), &Selection);
						return;
					}
				}
			}
		}

		SetValue(Context, FDataflowSelection(), &Selection);
	}
	else if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

FCollectionSelectionByPrimitiveDataflowNode::FCollectionSelectionByPrimitiveDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
	, Transform(FTransform::Identity)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Primitive);
	RegisterInputConnection(&Transform);

	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Selection);
}

void FCollectionSelectionByPrimitiveDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Selection))
	{
		if (IsConnected(&Collection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
			const FTransform& InTransform = GetValue(Context, &Transform);

			const FDataflowOutput* SelectionOutput = FindOutput(&Selection);
			if (SelectionOutput)
			{
				const FName OutGroupName = FDataflowSelection::GetSelectionTypeFromCollection(*SelectionOutput);

				if (OutGroupName != NAME_None && 
					InCollection.HasGroup(FGeometryCollection::VerticesGroup) && 
					InCollection.HasGroup(OutGroupName))
				{
					GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);

					const FDataflowPrimitiveTypesStorage InPrimitive = GetValue(Context, &Primitive);

					TBitArray<> VertexSelection; 
					VertexSelection.Init(false, InCollection.NumElements(FGeometryCollection::VerticesGroup));
					if (IsConnected(&Primitive))
					{
						if (const FBox* InBox = InPrimitive.TryGet<FBox>())
						{
							TransformSelectionFacade.SelectVerticesInTransformedBox(*InBox, InTransform, VertexSelection);
						}
						else if (const FSphere* InSphere = InPrimitive.TryGet<FSphere>())
						{
							FSphere TransformedSphere = InSphere->TransformBy(InTransform);
							TransformSelectionFacade.SelectVerticesInSphere(TransformedSphere, VertexSelection);
						}
						else if (const FDataflowPlane* InPlane = InPrimitive.TryGet<FDataflowPlane>())
						{
							FDataflowPlane TransformedPlane = InPlane->GetTransformed(InTransform);
							TransformSelectionFacade.SelectVerticesOnPlaneSide(TransformedPlane.AsPlane(), bPositiveSide, VertexSelection);
						}
					}
					FDataflowSelection OutSelection(OutGroupName);
					OutSelection.Initialize(InCollection.NumElements(OutGroupName), false);

					TArray<int32> IntArray;
					IntArray.Reserve(VertexSelection.Num());
					for (int32 Idx = 0; Idx < VertexSelection.Num(); ++Idx)
					{
						if (VertexSelection[Idx])
						{
							IntArray.Add(Idx);
						}
					}

					if (OutGroupName == FDataflowVertexSelection::VerticesGroupName)
					{
						OutSelection.Initialize(VertexSelection);
					}
					else if (OutGroupName == FDataflowFaceSelection::FacesGroupName)
					{
						TArray<int32> SelectionArray = TransformSelectionFacade.ConvertVertexSelectionToFaceSelection(IntArray, bAllElementsMustBeSelected);
						OutSelection.SetFromArray(SelectionArray);
					}
					else if (OutGroupName == FDataflowTransformSelection::TransformGroupName)
					{
						TArray<int32> SelectionArray = TransformSelectionFacade.ConvertVertexSelectionToTransformSelection(IntArray, bAllElementsMustBeSelected);
						OutSelection.SetFromArray(SelectionArray);
					}
					else if (OutGroupName == FDataflowGeometrySelection::GeometryGroupName)
					{
						TArray<int32> SelectionArray = TransformSelectionFacade.ConvertVertexSelectionToGeometrySelection(IntArray, bAllElementsMustBeSelected);
						OutSelection.SetFromArray(SelectionArray);
					}
					else if (OutGroupName == FDataflowCurveSelection::CurveGroupName)
					{
						TBitArray<> OutCurveSelection;
						TransformSelectionFacade.ConvertVertexSelectionToCurveSelection(VertexSelection, bAllElementsMustBeSelected, OutCurveSelection);
						OutSelection.Initialize(OutCurveSelection);
					}
					else
					{
						SetError(Context, &Selection, LOCTEXT("CollectionSelectionByPrimitiveSelectedTypeNotSupported", "Selected type is not supported.").ToString());
					}

					SetValue(Context, MoveTemp(OutSelection), &Selection);					
					return;
				}
				else
				{
					SetError(Context, &Selection, LOCTEXT("CollectionSelectionByPrimitiveGroupDoesntExist", "Selected group type doesn't exist in the collection.").ToString());
				}
			}
		}

		SetValue(Context, FDataflowSelection(), &Selection);
	}
	else if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

#if WITH_EDITOR
bool FCollectionSelectionByPrimitiveDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FCollectionSelectionByPrimitiveDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		if (IsConnected(&Primitive))
		{
			const FTransform& InTransform = GetValue(Context, &Transform);
			const FDataflowPrimitiveTypesStorage InPrimitive = GetValue(Context, &Primitive);
			if (const FBox* InBox = InPrimitive.TryGet<FBox>())
			{
				UE::Dataflow::SelectionNodes::Private::DebugDrawBox(*InBox, InTransform, DataflowRenderingInterface);
			}
			else if (const FSphere* InSphere = InPrimitive.TryGet<FSphere>())
			{
				FSphere TransformedSphere = InSphere->TransformBy(InTransform);
				UE::Dataflow::SelectionNodes::Private::DebugDrawSphere(TransformedSphere, DataflowRenderingInterface);
			}
			else if (const FDataflowPlane* InPlane = InPrimitive.TryGet<FDataflowPlane>())
			{
				FDataflowPlane TransformedPlane = InPlane->GetTransformed(InTransform);
				UE::Dataflow::SelectionNodes::Private::DebugDrawPlane(TransformedPlane.AsTransform(), PlaneSize, VertexPerEdge, DataflowRenderingInterface);
			}
		}
	}
}
#endif

/* ----------------------------------------------------------------------------------------------------------------------- */

FCollectionSelectionByMeshDataflowNode::FCollectionSelectionByMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&bKeepInside).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&WindingThreshold).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinDistance).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxDistance).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bUseSignedDistance).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Selection);
}

void FCollectionSelectionByMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Selection))
	{
		if (IsConnected(&Collection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
			const bool bInKeepInside = GetValue(Context, &bKeepInside);
			const double InWindingThreshold = (double)GetValue(Context, &WindingThreshold);
			const double InMinDistance = (double)GetValue(Context, &MinDistance);
			const double InMaxDistance = (double)GetValue(Context, &MaxDistance);
			const bool bInUseSignedDistance = GetValue(Context, &bUseSignedDistance);

			const FDataflowOutput* SelectionOutput = FindOutput(&Selection);
			if (SelectionOutput)
			{
				const FName OutGroupName = FDataflowSelection::GetSelectionTypeFromCollection(*SelectionOutput);

				if (OutGroupName != NAME_None && InCollection.HasGroup(OutGroupName))
				{
					GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);

					if (TObjectPtr<const UDynamicMesh> InMesh = GetValue(Context, &Mesh))
					{
						TArray<FVector3f> VerticesInCollectionSpace;

						GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(InCollection);
						MeshFacade.GetVerticesInCollectionSpace(VerticesInCollectionSpace);

						TArray<FVector> InSamplePoints = TArray<FVector>(VerticesInCollectionSpace);

						TArray<bool> KeepPoints;
						KeepPoints.SetNumZeroed(InSamplePoints.Num());

						using namespace UE::Geometry;
						InMesh->ProcessMesh(
							[&InSamplePoints, &KeepPoints, InWindingThreshold, bInKeepInside, InMinDistance, InMaxDistance, bInUseSignedDistance, this]
							(const FDynamicMesh3& DynMesh)
							{
								const bool bFilterWinding = bool(FilterMethod == ESelectionByMeshMethodFlags::Winding);
								const bool bFilterMinDist = bool(FilterMethod == ESelectionByMeshMethodFlags::MinDistance);
								const bool bFilterMaxDist = bool(FilterMethod == ESelectionByMeshMethodFlags::MaxDistance);

								if (!bFilterWinding && !bFilterMinDist && !bFilterMaxDist)
								{
									// No filtering, early-out
									return;
								}

								// Clamp unsigned distances to 0
								double UseMinDistance = InMinDistance;
								double UseMaxDistance = InMaxDistance;
								if (!bInUseSignedDistance)
								{
									UseMinDistance = FMath::Max(0, UseMinDistance);
									UseMaxDistance = FMath::Max(0, UseMaxDistance);
								}

								if (bFilterMinDist && bFilterMaxDist)
								{
									// If Min > Max, impossible for points to pass the filter, so early-out
									if (UseMinDistance > UseMaxDistance)
									{
										InSamplePoints.Empty();
										KeepPoints.Empty();
										return;
									}
								}

								const bool bNeedWinding = bFilterWinding || bInUseSignedDistance;
								const bool bNeedDistance = bFilterMinDist || bFilterMaxDist;
								const double MaxRelevantDistance = UE_DOUBLE_KINDA_SMALL_NUMBER + FMath::Max(
									bFilterMinDist ? FMath::Abs(UseMinDistance) : 0,
									bFilterMaxDist ? FMath::Abs(UseMaxDistance) : 0);

								//  set up AABBTree and FWNTree lists
								TMeshAABBTree3<FDynamicMesh3> Spatial(&DynMesh);
								TFastWindingTree<FDynamicMesh3> FastWinding(&Spatial, bNeedWinding);

								// square threshold distances, but keep their signs, so we can do threshold testing w/out sqrt
								const double UseSignSqMinDist = FMath::CopySign(UseMinDistance * UseMinDistance, UseMinDistance);
								const double UseSignSqMaxDist = FMath::CopySign(UseMaxDistance * UseMaxDistance, UseMaxDistance);

								// Filter points 
								ParallelFor(KeepPoints.Num(),
									[&KeepPoints, &Spatial, &FastWinding, &InSamplePoints,
									bFilterWinding, bFilterMinDist, bFilterMaxDist, bInUseSignedDistance, bInKeepInside,
									bNeedWinding, InWindingThreshold, bNeedDistance, UseSignSqMinDist, UseSignSqMaxDist, MaxRelevantDistance]
									(int32 PointIdx)
									{
										FVector Pt = InSamplePoints[PointIdx];

										bool bWindingInside = false;
										if (bNeedWinding)
										{
											bWindingInside = FastWinding.IsInside(Pt, InWindingThreshold);
											// test if we fail the winding filter
											if (bFilterWinding && bInKeepInside != bWindingInside)
											{
												KeepPoints[PointIdx] = false;
												return;
											}
										}
										double FoundDistSq = 0;
										if (bNeedDistance)
										{
											const int32 FoundTID = Spatial.FindNearestTriangle(Pt, FoundDistSq, UE::Geometry::IMeshSpatial::FQueryOptions(MaxRelevantDistance));
											if (FoundTID == INDEX_NONE)
											{
												// we have a max dist, lack of closest point -> we fail the max dist filter
												// or it's inside the shape w/ signed distances -> it's too far inside, fail the min filter
												if (bFilterMaxDist || (bInUseSignedDistance && bWindingInside))
												{
													KeepPoints[PointIdx] = false;
													return;
												}
												else
												{
													// point at least passes the min distance threshold
													FoundDistSq = UseSignSqMinDist;
												}
											}
											else
											{
												if (bInUseSignedDistance && bWindingInside)
												{
													// sign the squared distance
													FoundDistSq = -FoundDistSq;
												}
											}
											if (bFilterMinDist && FoundDistSq < UseSignSqMinDist)
											{
												KeepPoints[PointIdx] = false;
												return;
											}
											else if (bFilterMaxDist && FoundDistSq > UseSignSqMaxDist)
											{
												KeepPoints[PointIdx] = false;
												return;
											}
										}

										// passed all filters, keep the point
										KeepPoints[PointIdx] = true;
									}
								);
							}
						);

						if (KeepPoints.Num())
						{
							FDataflowSelection OutSelection(OutGroupName);
							OutSelection.Initialize(InCollection.NumElements(OutGroupName), false);

							if (OutGroupName == FDataflowVertexSelection::VerticesGroupName)
							{
								OutSelection.SetFromArray(KeepPoints);
							}
							else if (OutGroupName == FDataflowFaceSelection::FacesGroupName)
							{
								TArray<int32> SelectionArray = TransformSelectionFacade.ConvertVertexSelectionToFaceSelection(TArray<int32>(KeepPoints), bAllElementsMustBeSelected);
								OutSelection.SetFromArray(SelectionArray);
							}
							else if (OutGroupName == FDataflowTransformSelection::TransformGroupName)
							{
								TArray<int32> SelectionArray = TransformSelectionFacade.ConvertVertexSelectionToTransformSelection(TArray<int32>(KeepPoints), bAllElementsMustBeSelected);
								OutSelection.SetFromArray(SelectionArray);
							}
							else if (OutGroupName == FDataflowGeometrySelection::GeometryGroupName)
							{
								TArray<int32> SelectionArray = TransformSelectionFacade.ConvertVertexSelectionToGeometrySelection(TArray<int32>(KeepPoints), bAllElementsMustBeSelected);
								OutSelection.SetFromArray(SelectionArray);
							}
							else if (OutGroupName == FDataflowCurveSelection::CurveGroupName)
							{
								TArray<int32> SelectionArray = TransformSelectionFacade.ConvertVertexSelectionToCurveSelection(TArray<int32>(KeepPoints), bAllElementsMustBeSelected);
								OutSelection.SetFromArray(SelectionArray);
							}
							else
							{
								SetError(Context, &Selection, LOCTEXT("CollectionSelectionByMeshSelectedTypeNotSupported", "Selected type is not supported.").ToString());
							}

							SetValue(Context, MoveTemp(OutSelection), &Selection);
							return;
						}
					}
				}
				else
				{
					SetError(Context, &Selection, LOCTEXT("CollectionSelectionByMeshGroupDoesntExist", "Selected group type doesn't exist in the collection.").ToString());
				}
			}
		}

		SetValue(Context, FDataflowSelection(), &Selection);
	}
	else if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

namespace UE::Dataflow::Private
{
	template<typename AttributeType, typename = void>
	struct THasLexFromString : std::false_type {};

	template<typename AttributeType>
	struct THasLexFromString<AttributeType, std::void_t<decltype(LexFromString(DeclVal<AttributeType&>(), DeclVal<const TCHAR*>()))>>
		: std::true_type {};

	template<typename AttributeType>
	void SetAttributeFromString(FManagedArrayCollection& Collection, const FName& AttributeName, const FName& GroupName, 
		const FString& SelectionString, const FString& DefaultString, const TArray<int32>& SelectedElements)
	{
		if constexpr (THasLexFromString<AttributeType>::value)
		{
			AttributeType SelectionValue{};
			LexFromString(SelectionValue, *SelectionString);
		
			AttributeType DefaultValue{};
			LexFromString(DefaultValue, *DefaultString);
		
			const bool bHasAttribute = Collection.HasAttribute(AttributeName, GroupName);
			if (TManagedArray<AttributeType>* AttributeArray = Collection.FindOrAddAttributeTyped<AttributeType>(AttributeName, GroupName))
			{
				if (!bHasAttribute)
				{
					AttributeArray->Fill(DefaultValue);
				}
				if (!SelectedElements.IsEmpty())
				{
					for (int32 ElementIndex : SelectedElements)
					{
						if (AttributeArray->IsValidIndex(ElementIndex))
						{
							(*AttributeArray)[ElementIndex] = SelectionValue;
						}
					}
				}
			}
		}
	}
	
	FManagedArrayCollection::EArrayType GetAttributeArrayType(const ESelectionToAttributeTypeFlags& TypeFlag)
	{
		FManagedArrayCollection::EArrayType ArrayType = FManagedArrayCollection::EArrayType::FFloatType;
		switch (TypeFlag)
		{
			case ESelectionToAttributeTypeFlags::Int:
				ArrayType = FManagedArrayCollection::EArrayType::FInt32Type;
				break;
			case ESelectionToAttributeTypeFlags::Bool:
				ArrayType = FManagedArrayCollection::EArrayType::FBoolType;
				break;
			case ESelectionToAttributeTypeFlags::Float:
				ArrayType = FManagedArrayCollection::EArrayType::FFloatType;
				break;
			case ESelectionToAttributeTypeFlags::Double:
				ArrayType = FManagedArrayCollection::EArrayType::FDoubleType;
				break;
			case ESelectionToAttributeTypeFlags::Vec2f:
				ArrayType = FManagedArrayCollection::EArrayType::FVector2DType;
				break;
			case ESelectionToAttributeTypeFlags::Vec3f:
				ArrayType = FManagedArrayCollection::EArrayType::FVectorType;
				break;
			case ESelectionToAttributeTypeFlags::Vec4f:
				ArrayType = FManagedArrayCollection::EArrayType::FVector4fType;
				break;
			case ESelectionToAttributeTypeFlags::Vec2i:
				ArrayType = FManagedArrayCollection::EArrayType::FIntVector2Type;
				break;
			case ESelectionToAttributeTypeFlags::Vec3i:
				ArrayType = FManagedArrayCollection::EArrayType::FIntVectorType;
				break;
			case ESelectionToAttributeTypeFlags::Vec4i:
				ArrayType = FManagedArrayCollection::EArrayType::FIntVector4Type;
				break;
			case ESelectionToAttributeTypeFlags::Color:
				ArrayType = FManagedArrayCollection::EArrayType::FLinearColorType;
				break;
			case ESelectionToAttributeTypeFlags::Quat:
				ArrayType = FManagedArrayCollection::EArrayType::FQuatType;
				break;
			case ESelectionToAttributeTypeFlags::String:
				ArrayType = FManagedArrayCollection::EArrayType::FStringType;
				break;
			case ESelectionToAttributeTypeFlags::Name:
				ArrayType = FManagedArrayCollection::EArrayType::FNameType;
				break;
			case ESelectionToAttributeTypeFlags::Transform:
				ArrayType = FManagedArrayCollection::EArrayType::FTransformType;
				break;
			case ESelectionToAttributeTypeFlags::Matrix:
				ArrayType = FManagedArrayCollection::EArrayType::FPMatrix33dType;
				break;
			case ESelectionToAttributeTypeFlags::Box:
				ArrayType = FManagedArrayCollection::EArrayType::FBoxType;
				break;
			default:
				break;
		}
		return ArrayType;
	}
}

FCollectionSelectionToAttributeDataflowNode::FCollectionSelectionToAttributeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Selection);

	RegisterInputConnection(&Attribute).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&DefaultValue).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Value).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Attribute, &Attribute);
}

void FCollectionSelectionToAttributeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		if (IsConnected(&Collection) && IsConnected(&Selection))
		{
			FManagedArrayCollection InCollection = GetValue(Context, &Collection);
			FDataflowSelection InSelection = GetValue(Context, &Selection);

			const FDataflowInput* SelectionInput = FindInput(&Selection);
			if (SelectionInput)
			{
				const FName GroupName = FDataflowSelection::GetSelectionTypeFromCollection(*SelectionInput);
				if (GroupName != NAME_None && InCollection.HasGroup(GroupName))
				{
					if (InCollection.NumElements(GroupName) == InSelection.Num())
					{
						const FName AttributeName = FName(GetValue(Context, &Attribute));
						
						const FString SelectionString = GetValue(Context, &Value);
						const FString DefaultString = GetValue(Context, &DefaultValue);
						
						const FManagedArrayCollection::EArrayType ArrayType = InCollection.HasAttribute(AttributeName, GroupName) ? 
							InCollection.GetAttributeType(AttributeName, GroupName) : UE::Dataflow::Private::GetAttributeArrayType(AttributeType);
						
						switch (ArrayType)
						{
#define MANAGED_ARRAY_TYPE(CppType, EnumName) \
							case EManagedArrayType::F##EnumName##Type: \
								UE::Dataflow::Private::SetAttributeFromString<CppType>(InCollection, AttributeName, GroupName, SelectionString, DefaultString, InSelection.AsArray()); \
							break;
#include "GeometryCollection/ManagedArrayTypeValues.inl"
#undef MANAGED_ARRAY_TYPE
							default: break;
						}
			
						SetValue(Context, MoveTemp(InCollection), &Collection);
						return;
					}
				}
			}

			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		SetValue(Context, FManagedArrayCollection(), &Collection);
	}
	else if (Out->IsA(&Attribute))
	{
		SafeForwardInput(Context, &Attribute, &Attribute);
	}
}

/* ----------------------------------------------------------------------------------------------------------------------- */

#undef LOCTEXT_NAMESPACE
