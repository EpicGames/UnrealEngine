// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionSkeletalMeshToCollectionNode.h"

#include "Animation/Skeleton.h"
#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowElement.h"
#include "Dataflow/DataflowSkeletalMeshNodes.h"
#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "Materials/MaterialInterface.h"

#if WITH_EDITOR
#include "Dataflow/DataflowRenderingViewMode.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionSkeletalMeshToCollectionNode)

void FSkeletalMeshToCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection))
	{
		FGeometryCollection OutCollection;
		TObjectPtr<const USkeletalMesh> InSkeletalMesh = GetValue<TObjectPtr<const USkeletalMesh>>(Context, &SkeletalMesh);
		if (InSkeletalMesh)
		{
			FGeometryCollectionEngineConversion::AppendSkeletalMesh(InSkeletalMesh, 0, FTransform::Identity, &OutCollection, /*bReindexMaterials = */ true, bImportTransformOnly);
		}
		SetValue(Context, FManagedArrayCollection(OutCollection), &Collection);
	}
}

////////////////////////////////////////////////////////////////////////////

FCollectionToSkeletalMeshDataflowNode::FCollectionToSkeletalMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Materials);

	RegisterOutputConnection(&SkeletalMesh);
	RegisterOutputConnection(&Skeleton);
}

void FCollectionToSkeletalMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
if (Out->IsA(&SkeletalMesh) || Out->IsA(&Skeleton))
	{
		USkeletalMesh* OutSkeletalMesh = NewObject<USkeletalMesh>();
		USkeleton* OutSkeleton = NewObject<USkeleton>();
		
		if (OutSkeletalMesh && OutSkeleton)
		{
			// make a copy so that we can ensure the level array is created
			FManagedArrayCollection InCollection = GetValue(Context, &Collection);
			const TArray<TObjectPtr<UMaterialInterface>>& InMaterials = GetValue(Context, &Materials);

			Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(InCollection);
			if (HierarchyFacade.IsValid())
			{
				HierarchyFacade.GenerateLevelAttribute();
			}

			FGeometryCollectionEngineConversion::ConvertCollectionToSkeletalMesh(InCollection, InMaterials, *OutSkeletalMesh, *OutSkeleton);
		}
		
		SetValue(Context, TObjectPtr<const USkeletalMesh>(OutSkeletalMesh), &SkeletalMesh);
		SetValue(Context, TObjectPtr<const USkeleton>(OutSkeleton), &Skeleton);
	}
}

#if WITH_EDITOR
void FCollectionToSkeletalMeshDataflowNode::DebugDraw(UE::Dataflow::FContext& Context,
	IDataflowDebugDrawInterface& DataflowRenderingInterface,
	const FDebugDrawParameters& DebugDrawParameters) const
{
	if (const FDataflowOutput* Output = FindOutput(&SkeletalMesh))
	{
		if (TObjectPtr<const USkeletalMesh> InSkeletalMesh = Output->ReadValue(Context, SkeletalMesh))
		{
			TRefCountPtr<IDataflowDebugDrawObject> SkeletonObject(
				MakeDebugDrawObject<FDataflowDebugDrawSkeletonObject>(
					DataflowRenderingInterface.ModifyDataflowElements(), InSkeletalMesh->GetRefSkeleton()
				));

			DataflowRenderingInterface.DrawObject(SkeletonObject);
		}
	}
}

bool FCollectionToSkeletalMeshDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}
#endif
