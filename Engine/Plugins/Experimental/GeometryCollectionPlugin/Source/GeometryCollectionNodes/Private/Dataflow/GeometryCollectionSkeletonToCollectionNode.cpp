// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionSkeletonToCollectionNode.h"

#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionSkeletonToCollectionNode)

void FSkeletonToCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection))
	{
		DataType OutCollection;
		TObjectPtr<const USkeleton> SkeletonValue = GetValue<TObjectPtr<const USkeleton>>(Context, &Skeleton);
		if (SkeletonValue)
		{
			FGeometryCollectionEngineConversion::AppendSkeleton(SkeletonValue.Get(), FTransform::Identity, &OutCollection);
		}
		SetValue(Context, MoveTemp(OutCollection), &Collection);
	}
}


