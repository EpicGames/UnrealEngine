// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionStaticMeshToCollectionNode.h"
#include "Dataflow/DataflowCore.h"

#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionEngineConversion.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionStaticMeshToCollectionNode)

namespace UE::Dataflow
{
	void RegisterGeometryStaticMeshToCollectionNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FStaticMeshToCollectionDataflowNode_v2);

		// Deprecated nodes (Need to stay registered) 
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FStaticMeshToCollectionDataflowNode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FStaticMeshToCollectionDataflowNode_v2::FStaticMeshToCollectionDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&StaticMesh);
	RegisterInputConnection(&MeshTransform);
	RegisterOutputConnection(&Collection);
	RegisterOutputConnection(&Materials);
	RegisterOutputConnection(&InstancedMeshes);
	RegisterOutputConnection(&RootProxyMeshes);
}

void FStaticMeshToCollectionDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	ensure(Out->IsA(&Collection) || Out->IsA(&Materials) || Out->IsA(&InstancedMeshes) || Out->IsA(&RootProxyMeshes));

	FManagedArrayCollection OutCollection;
	TArray<TObjectPtr<UMaterialInterface>> OutMaterials;
	TArray<FGeometryCollectionAutoInstanceMesh> OutInstancedMeshes;
	TArray<FDataflowRootProxyMesh> OutRootProxyMeshes;

	TObjectPtr<UStaticMesh> StaticMeshVal = GetValue(Context, &StaticMesh, StaticMesh);
	const FTransform& InMeshTransform = GetValue(Context, &MeshTransform, MeshTransform);
	if (StaticMeshVal)
	{
		FGeometryCollectionEngineConversion::ConvertStaticMeshToGeometryCollection(StaticMeshVal, InMeshTransform, OutCollection, OutMaterials, OutInstancedMeshes, bSetInternalFromMaterialIndex, bSplitComponents);

		FDataflowRootProxyMesh RootProxyMesh;
		RootProxyMesh.Mesh = StaticMeshVal;
		RootProxyMesh.Transform = InMeshTransform;
		OutRootProxyMeshes.Add(RootProxyMesh);
	}

	// Set Outputs
	SetValue(Context, MoveTemp(OutCollection), &Collection);
	SetValue(Context, MoveTemp(OutMaterials), &Materials);
	SetValue(Context, MoveTemp(OutInstancedMeshes), &InstancedMeshes);
	SetValue(Context, MoveTemp(OutRootProxyMeshes), &RootProxyMeshes);
}


////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
//
// Deprecated version 

FStaticMeshToCollectionDataflowNode::FStaticMeshToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&StaticMesh);
	RegisterInputConnection(&MeshTransform);
	RegisterOutputConnection(&Collection);
	RegisterOutputConnection(&Materials);
	RegisterOutputConnection(&MaterialInstances);
	RegisterOutputConnection(&InstancedMeshes);
}

void FStaticMeshToCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	ensure(Out->IsA(&Collection) || Out->IsA(&Materials) || Out->IsA(&MaterialInstances) || Out->IsA(&InstancedMeshes));

	FManagedArrayCollection OutCollection;
	TArray<TObjectPtr<UMaterialInterface>> OutMaterialInstances;
	TArray<FGeometryCollectionAutoInstanceMesh> OutInstancedMeshes;

	TObjectPtr<UStaticMesh> StaticMeshVal = GetValue(Context, &StaticMesh, StaticMesh);
	const FTransform& InMeshTransform = GetValue(Context, &MeshTransform, MeshTransform);
	if (StaticMeshVal)
	{
		FGeometryCollectionEngineConversion::ConvertStaticMeshToGeometryCollection(StaticMeshVal, InMeshTransform, OutCollection, OutMaterialInstances, OutInstancedMeshes, bSetInternalFromMaterialIndex, bSplitComponents);
	}

	TArray<TObjectPtr<UMaterial>> OutMaterials;
	FGeometryCollectionEngineConversion::GetMaterialsFromInstances(OutMaterialInstances, OutMaterials);

	// Set Outputs
	SetValue(Context, MoveTemp(OutCollection), &Collection);
	SetValue(Context, MoveTemp(OutMaterials), &Materials);
	SetValue(Context, MoveTemp(OutMaterialInstances), &MaterialInstances);
	SetValue(Context, MoveTemp(OutInstancedMeshes), &InstancedMeshes);
}

