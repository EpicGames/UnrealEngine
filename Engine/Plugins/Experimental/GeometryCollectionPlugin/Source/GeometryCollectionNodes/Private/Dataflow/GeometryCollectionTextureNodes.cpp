// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionTextureNodes.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "ChaosLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionTextureNodes)

namespace UE::Dataflow
{
	void RegisterGeometryCollectionTextureNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBakeTextureFromCollectionDataflowNode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////

FBakeTextureFromCollectionDataflowNode::FBakeTextureFromCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&FaceSelection);
	RegisterInputConnection(&UVChannel);
	RegisterInputConnection(&GutterSize);
	RegisterInputConnection(&Resolution);

	// hidden by default inputs
	RegisterInputConnection(&MaxDistance).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&OcclusionRays).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&OcclusionBlurRadius).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&CurvatureBlurRadius).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&VoxelResolution).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SmoothingIterations).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ThicknessFactor).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxCurvature).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bClearGutterAlpha).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bFullyPadGutter).SetCanHidePin(true).SetPinIsHidden(true);
	
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Image);
	RegisterOutputConnection(&UVChannel, &UVChannel);
}

void FBakeTextureFromCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA(&UVChannel))
	{
		SafeForwardInput(Context, &UVChannel, &UVChannel);
	}
	else if (Out->IsA(&Image))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);
		const int32 InUVChannel = GetValue(Context, &UVChannel);
		const int32 InResolution = (int32)GetValue(Context, &Resolution);
		const int32 InGutterSize = GetValue(Context, &bFullyPadGutter) ? InResolution : GetValue(Context, &GutterSize);
		

		TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>());
		check(GeomCollection.IsValid());

		const bool bIsValidUVChannel = (InUVChannel >= 0 && InUVChannel < GeomCollection->NumUVLayers());
		if (!bIsValidUVChannel)
		{
			// log until we have proper error reporting in context 
			UE_LOGF(LogChaosDataflow, Error, "Dataflow Node [%ls] : Invalid UV channel [%d] - The collection has %d UV channels", *GetName().ToString(), InUVChannel, GeomCollection->NumUVLayers());

			const FString Message = FString::Printf(TEXT("Invalid UV channel [%d] - The collection has %d UV channels"), InUVChannel, GeomCollection->NumUVLayers());
			Context.Warning(Message, this, Out);

			// Write out a red 16x16 image as a sign of error
			FDataflowImage DefaultImage;
			DefaultImage.CreateFromColor(EDataflowImageResolution::Resolution16, FLinearColor::Red);
			SetValue(Context, MoveTemp(DefaultImage), &Image);

			return;
		}

		const UE::Geometry::FIndex4i BakeAttributes{ (int32)RedChannel, (int32)GreenChannel, (int32)BlueChannel, (int32)AlphaChannel };

		UE::PlanarCut::FTextureAttributeSettings BakeAttributeSettings;
		BakeAttributeSettings.ToExternal_MaxDistance = GetValue(Context, &MaxDistance);;
		BakeAttributeSettings.AO_Rays = GetValue(Context, &OcclusionRays);
		BakeAttributeSettings.AO_BlurRadius = GetValue(Context, &OcclusionBlurRadius);
		BakeAttributeSettings.Curvature_BlurRadius = GetValue(Context, &CurvatureBlurRadius);
		BakeAttributeSettings.Curvature_SmoothingSteps = GetValue(Context, &SmoothingIterations);
		BakeAttributeSettings.Curvature_VoxelRes = GetValue(Context, &VoxelResolution);
		BakeAttributeSettings.Curvature_ThicknessFactor = GetValue(Context, &ThicknessFactor);
		BakeAttributeSettings.Curvature_MaxValue = GetValue(Context, &MaxCurvature);
		BakeAttributeSettings.ClearGutterChannel = GetValue(Context, &bClearGutterAlpha) ? 3 : INDEX_NONE;

		UE::Geometry::TImageBuilder<FVector4f> ImageBuilder;
		ImageBuilder.SetDimensions({ InResolution, InResolution });
		ImageBuilder.Clear(FVector4f(1, 0, 0, 0));

		if (IsConnected(&FaceSelection))
		{
			const FDataflowFaceSelection& InFaceSelection = GetValue(Context, &FaceSelection);
			if (InFaceSelection.Num() == GeomCollection->NumElements(FGeometryCollection::FacesGroup))
			{
				TArray<bool> FacesToBake;
				FacesToBake.SetNumUninitialized(InFaceSelection.Num());
				for (int32 Index = 0; Index < InFaceSelection.Num(); ++Index)
				{
					FacesToBake[Index] = InFaceSelection.IsSelected(Index);
				}

				UE::PlanarCut::TextureSpecifiedFaces(
					InUVChannel, *GeomCollection, InGutterSize,
					BakeAttributes, BakeAttributeSettings, ImageBuilder,
					FacesToBake, nullptr /* Progress */);
			}
			else
			{
				Context.Warning(TEXT("Selection does not match the collection, the collection may have changed since the selection was generated from it"), this, Out);
			}
		}
		else
		{
			// selection not connected, bake all faces
			UE::PlanarCut::TextureSpecifiedFaces(
				InUVChannel, *GeomCollection, InGutterSize,
				BakeAttributes, BakeAttributeSettings, ImageBuilder,
				UE::PlanarCut::ETargetFaces::AllFaces, {}, nullptr /* Progress */);
		}

		// copy the image data over and set the corresponding value
		FDataflowImage OutImage;
		OutImage.CreateRGBA32F(InResolution, InResolution);
		OutImage.CopyRGBAPixels(ImageBuilder.GetImageBuffer());
		SetValue(Context, MoveTemp(OutImage), &Image);
	}
}

