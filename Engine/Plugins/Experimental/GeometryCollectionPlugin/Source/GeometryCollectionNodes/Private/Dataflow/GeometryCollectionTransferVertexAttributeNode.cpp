// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionTransferVertexAttributeNode.h"

#include "Chaos/Triangle.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/HierarchicalSpatialHash.h"
#include "Chaos/TriangleCollisionPoint.h"
#include "Dataflow/DataflowInputOutput.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionTransferVertexAttributeNode)
#define LOCTEXT_NAMESPACE "FGeometryCollectionTransferVertexAttributeNode"

namespace UE::Private
{
	typedef Chaos::FSphere SphereType;
	typedef Chaos::TBoundingVolumeHierarchy<TArray<SphereType*>, TArray<int32>, Chaos::FReal, 3> BVH;
	class FTransferFacade
	{
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;
	public:
		FTransferFacade(FManagedArrayCollection& InCollection)
			: ConstCollection(InCollection)
			, Collection(&InCollection)
			, BoneMap(InCollection, FName("BoneMap"), FName("Vertices"))
			, Vertex(InCollection, FName("Vertex"), FName("Vertices"))
			, Indices(InCollection, FName("Indices"), FName("Faces"))
			, Transform(InCollection, FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup)
			, Parent(InCollection, FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup)
			, VertexStart(InCollection, "VertexStart", FGeometryCollection::GeometryGroup)
			, VertexCount(InCollection, "VertexCount", FGeometryCollection::GeometryGroup)
			, FaceStart(InCollection, "FaceStart", FGeometryCollection::GeometryGroup)
			, FaceCount(InCollection, "FaceCount", FGeometryCollection::GeometryGroup)
		{}

		FTransferFacade(const FManagedArrayCollection& InCollection)
			: ConstCollection(InCollection)
			, Collection(nullptr)
			, BoneMap(InCollection, FName("BoneMap"), FName("Vertices"))
			, Vertex(InCollection, FName("Vertex"), FName("Vertices"))
			, Indices(InCollection, FName("Indices"), FName("Faces"))
			, Transform(InCollection, FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup)
			, Parent(InCollection, FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup)
			, VertexStart(InCollection, "VertexStart", FGeometryCollection::GeometryGroup)
			, VertexCount(InCollection, "VertexCount", FGeometryCollection::GeometryGroup)
			, FaceStart(InCollection, "FaceStart", FGeometryCollection::GeometryGroup)
			, FaceCount(InCollection, "FaceCount", FGeometryCollection::GeometryGroup)
		{}


		bool IsValid() const {
			return BoneMap.IsValid() && Vertex.IsValid() && Indices.IsValid() && Transform.IsValid() && Parent.IsValid() && VertexStart.IsValid() &&
				VertexCount.IsValid() && FaceStart.IsValid() && FaceCount.IsValid();
		}

		template<typename T>
		const TManagedArray<T>* GetAttributeArray(FString AttributeName, FString Group) const
		{
			return ConstCollection.FindAttributeTyped<T>(FName(AttributeName), FName(Group));
		}

		template<typename T>
		TManagedArray<T>* GetAttributeArray(FString AttributeName, FString Group)
		{
			TManagedArray<T>* TargetAttributeArray = nullptr;
			if (!Collection->HasAttribute(FName(AttributeName), FName(Group)))
			{
				Collection->AddAttribute<T>(FName(AttributeName), FName(Group));
			}
			return Collection->FindAttributeTyped<T>(FName(AttributeName), FName(Group));
		}


		TManagedArrayAccessor<int32> BoneMap;
		TManagedArrayAccessor<FVector3f> Vertex;
		TManagedArrayAccessor<FIntVector3> Indices;
		TManagedArrayAccessor<FTransform3f> Transform;
		TManagedArrayAccessor<int32> Parent;
		TManagedArrayAccessor<int32> VertexStart;
		TManagedArrayAccessor<int32> VertexCount;
		TManagedArrayAccessor<int32> FaceStart;
		TManagedArrayAccessor<int32> FaceCount;
	};

	struct FTransferData
	{
		TArray<int32> SourceIndices;
		TArray<float> SourceWeights;
		float FalloffScale = 0.f;
		FTransferData() {}
		FTransferData(TArray<int32> SourceIndicesIn, TArray<float> SourceWeightsIn, float FalloffScaleIn) :
			SourceIndices(SourceIndicesIn), SourceWeights(SourceWeightsIn), FalloffScale(FalloffScaleIn) {}
	};

	struct FTransferProperties
	{
		EDataflowTransferVertexAttributeNodeBoundingVolume BoundingVolumeType = EDataflowTransferVertexAttributeNodeBoundingVolume::Dataflow_Max;
		EDataflowTransferVertexAttributeNodeSourceScale SourceScale = EDataflowTransferVertexAttributeNodeSourceScale::Dataflow_Max;
		EDataflowTransferVertexAttributeNodeFalloff Falloff = EDataflowTransferVertexAttributeNodeFalloff::Dataflow_Max;
		float FalloffThreshold = 0.f;
		float EdgeMultiplier = 0.f;
		float BoundMultiplier = 0.f;
		FString TransformNameSuffix;
	};

	static float MaxEdgeLength(TArray<FVector3f>& Vert, const TManagedArray<FIntVector3>& Tri, int32 VertexOffset, int32 TriStart, int32 TriCount)
	{
		auto TriInRange = [](const FIntVector3& T, int32 Max) {
			for (int32 k = 0; k < 3; k++)
			{
				if (ensure(0 <= T[k] && T[k] < Max))
				{
					return true;
				}
			}
			return false;
			};

		float Max = 0;
		int32 TriStop = TriStart + TriCount;
		for (int32 i = TriStart; i < TriStop; i++)
		{
			if (TriInRange(Tri[i] - FIntVector3(VertexOffset), Vert.Num()))
			{
				Max = FMath::Max(Max, (Vert[Tri[i][0] - VertexOffset] - Vert[Tri[i][1] - VertexOffset]).SquaredLength());
				Max = FMath::Max(Max, (Vert[Tri[i][0] - VertexOffset] - Vert[Tri[i][2] - VertexOffset]).SquaredLength());
				Max = FMath::Max(Max, (Vert[Tri[i][1] - VertexOffset] - Vert[Tri[i][2] - VertexOffset]).SquaredLength());
			}
		}
		return FMath::Sqrt(Max);
	}

	static void BuildComponentSpaceVertices(const TManagedArray<FTransform3f>& LocalSpaceTransform, const TManagedArray<int32>& Parent, const TManagedArray<int32>& BoneMapArray,
		const TManagedArray<FVector3f>& VertexArray, int32 Start, int32 Count, TArray<FVector3f>& ComponentSpaceVertices)
	{
		TArray<FTransform3f> ComponentTransform;
		GeometryCollectionAlgo::GlobalMatrices(LocalSpaceTransform, Parent, ComponentTransform);

		ComponentSpaceVertices.SetNumUninitialized(Count);
		for (int32 i = 0; i < Count; ++i)
		{
			int32 j = i + Start;
			if (0 < BoneMapArray[i] && BoneMapArray[i] < ComponentTransform.Num())
			{
				ComponentSpaceVertices[i] = ComponentTransform[BoneMapArray[j]].TransformPosition(VertexArray[j]);
			}
			else
			{
				ComponentSpaceVertices[i] = VertexArray[j];
			}
		}
	}

	static BVH* BuildParticleSphereBVH(const TArray<FVector3f>& Vertices, float Radius)
	{
		TArray<SphereType*> VertexSpherePtrs;
		TArray<SphereType> VertexSpheres;
		VertexSpheres.Init(SphereType(Chaos::FVec3f(0), Radius), Vertices.Num());
		VertexSpherePtrs.SetNum(Vertices.Num());

		for (int32 i = 0; i < Vertices.Num(); i++)
		{
			Chaos::FVec3f SphereCenter(Vertices[i]);
			SphereType VertexSphere(SphereCenter, Radius);
			VertexSpheres[i] = SphereType(SphereCenter, Radius);
			VertexSpherePtrs[i] = &VertexSpheres[i];
		}
		return new BVH(VertexSpherePtrs);
	}

	static void TriangleToVertexIntersections(
		const BVH& VertexBVH, const TArray<FVector3f>& ComponentSpaceVertices, const FIntVector3& Triangle, TArray<int32>& OutTargetVertexIntersection)
	{
		OutTargetVertexIntersection.Empty();

		TArray<int32> TargetVertexIntersection0 = VertexBVH.FindAllIntersections(ComponentSpaceVertices[Triangle[0]]);
		TArray<int32> TargetVertexIntersection1 = VertexBVH.FindAllIntersections(ComponentSpaceVertices[Triangle[1]]);
		TArray<int32> TargetVertexIntersection2 = VertexBVH.FindAllIntersections(ComponentSpaceVertices[Triangle[2]]);
		TargetVertexIntersection0.Sort();
		TargetVertexIntersection1.Sort();
		TargetVertexIntersection2.Sort();

		for (int32 k = 0; k < TargetVertexIntersection0.Num(); k++)
		{
			if (TargetVertexIntersection1.Contains(TargetVertexIntersection0[k])
				&& TargetVertexIntersection2.Contains(TargetVertexIntersection0[k]))
			{
				OutTargetVertexIntersection.Emplace(TargetVertexIntersection0[k]);
			}
		}
	}


	static float CalculateFalloffScale(EDataflowTransferVertexAttributeNodeFalloff FalloffSetting, float Threshold, float Distance)
	{
		float Denominator = 1.0;
		if (Distance > Threshold && !FMath::IsNearlyZero(Threshold))
		{
			Denominator = Distance / Threshold;
		}
		switch (FalloffSetting)
		{
		case EDataflowTransferVertexAttributeNodeFalloff::Linear:
			return 1. / Denominator;
		case EDataflowTransferVertexAttributeNodeFalloff::Squared:
			return 1. / FMath::Square(Denominator);
		}
		return 1.0;
	}

	static TArray<FIntVector2> FindSourceToTargetGeometryMap(const FManagedArrayCollection& SourceCollection, const FManagedArrayCollection& TargetCollection, const FString& TransformNameSuffix)
	{
		TArray<FIntVector2> Mapping;
		const TManagedArray<FString>* const SourceName = SourceCollection.FindAttribute<FString>(FName("BoneName"), FTransformCollection::TransformGroup);
		const TManagedArray<int32>* const SourceGeometryGroup = SourceCollection.FindAttribute<int32>(FName("TransformToGeometryIndex"), FTransformCollection::TransformGroup);
		const TManagedArray<FString>* const TargetName = TargetCollection.FindAttribute<FString>(FName("BoneName"), FTransformCollection::TransformGroup);
		const TManagedArray<int32>* const TargetGeometryGroup = TargetCollection.FindAttribute<int32>(FName("TransformToGeometryIndex"), FTransformCollection::TransformGroup);
		if (SourceName && SourceGeometryGroup && TargetName && TargetGeometryGroup)
		{
			for (int32 i = 0; i < SourceName->Num(); ++i)
			{
				const FString TestName = FString::Printf(TEXT("%s%s"), *(*SourceName)[i], *TransformNameSuffix);
				for (int32 j = 0; j < TargetName->Num(); ++j)
				{
					if ((*TargetName)[j].StartsWith(TestName))
					{
						Mapping.Add(FIntVector2((*SourceGeometryGroup)[i], (*TargetGeometryGroup)[j]));
						break;
					}
				}
			}
		}

		return MoveTemp(Mapping);
	}

	static TMap<int32, int32> FindSourceToTargetTransformMap(const FManagedArrayCollection& SourceCollection, const FManagedArrayCollection& TargetCollection)
	{
		TMap<int32, int32> SourceIndexToTargetIndex;
		TMap<FString, int32> TargetNameToIndex;
		const TManagedArray<FString>* const SourceName = SourceCollection.FindAttribute<FString>(FName("BoneName"), FTransformCollection::TransformGroup);
		const TManagedArray<FString>* const TargetName = TargetCollection.FindAttribute<FString>(FName("BoneName"), FTransformCollection::TransformGroup);
		if (SourceName && TargetName)
		{
			for (int32 j = 0; j < TargetName->Num(); j++)
			{
				TargetNameToIndex.Add((*TargetName)[j], j);
			}
			for (int32 i = 0; i < SourceName->Num(); i++)
			{
				if (int32* TargetIndex = TargetNameToIndex.Find((*SourceName)[i]))
				{
					SourceIndexToTargetIndex.Add(i, *TargetIndex);
				}
			}
		}

		return MoveTemp(SourceIndexToTargetIndex);
	}

	static TArray<FTransferData> PairedGeometryTransfer(const TArray<FIntVector2>& PairedGeometry,
		const FTransferFacade& Source, const FTransferFacade& Target, const FTransferProperties& TransferProperties)
	{
		TArray<FTransferData> TransferDataArray;
		TransferDataArray.SetNum(Target.Vertex.Num());
		Chaos::FReal SphereFullRadius = 0.f;

		if (TransferProperties.SourceScale == EDataflowTransferVertexAttributeNodeSourceScale::Asset_Edge || TransferProperties.SourceScale == EDataflowTransferVertexAttributeNodeSourceScale::Asset_Bound)
		{
			// Build component space vertices for TargetCollection
			TArray<FVector3f> ComponentSpaceFullTargetVertices; // size of num vertices of the geometry entry. 
			BuildComponentSpaceVertices(Target.Transform.Get(), Target.Parent.Get(), Target.BoneMap.Get(), Target.Vertex.Get(),
				0, Target.Vertex.Num(), ComponentSpaceFullTargetVertices);

			// Build component space vertices for TargetCollection
			TArray<FVector3f> ComponentSpaceFullVertices; // size of num vertices of the geometry entry
			BuildComponentSpaceVertices(Source.Transform.Get(), Source.Parent.Get(), Source.BoneMap.Get(), Source.Vertex.Get(),
				0, Source.Vertex.Num(), ComponentSpaceFullVertices);
			if (TransferProperties.SourceScale == EDataflowTransferVertexAttributeNodeSourceScale::Asset_Edge)
			{
				SphereFullRadius = Chaos::FReal(TransferProperties.EdgeMultiplier * FMath::Max(
					MaxEdgeLength(ComponentSpaceFullTargetVertices, Target.Indices.Get(), 0, 0, Target.Indices.Num()),
					MaxEdgeLength(ComponentSpaceFullVertices, Source.Indices.Get(), 0, 0, Source.Indices.Num())));
			}
			else if (TransferProperties.SourceScale == EDataflowTransferVertexAttributeNodeSourceScale::Asset_Bound)
			{

				Chaos::FVec3f CoordMaxs(-FLT_MAX);
				Chaos::FVec3f CoordMins(FLT_MAX);
				for (int32 i = 0; i < ComponentSpaceFullVertices.Num(); i++)
				{
					CoordMaxs = CoordMaxs.ComponentwiseMax(ComponentSpaceFullVertices[i]);
					CoordMins = CoordMins.ComponentwiseMin(ComponentSpaceFullVertices[i]);
				}
				const Chaos::FVec3f CoordDiff = (CoordMaxs - CoordMins) * TransferProperties.BoundMultiplier;
				SphereFullRadius = FMath::Min3(CoordDiff[0], CoordDiff[1], CoordDiff[2]);
			}
		}
		ParallelFor(PairedGeometry.Num(), [&](int32 Pdx)
			{
				const int32 AttributeGeometryIndex = PairedGeometry[Pdx][0];
				const int32 TargetGeometryIndex = PairedGeometry[Pdx][1];
				if (ensure(0 <= AttributeGeometryIndex && AttributeGeometryIndex < Source.VertexStart.Num()))
				{
					if (ensure(0 <= TargetGeometryIndex && TargetGeometryIndex < Target.VertexStart.Num()))
					{
						// Build component space vertices for TargetCollection
						TArray<FVector3f> ComponentSpaceTargetVertices; // size of num vertices of the geometry entry. 
						BuildComponentSpaceVertices(Target.Transform.Get(), Target.Parent.Get(), Target.BoneMap.Get(), Target.Vertex.Get(),
							Target.VertexStart[TargetGeometryIndex], Target.VertexCount[TargetGeometryIndex], ComponentSpaceTargetVertices);

						// Build component space vertices for SourceCollection
						TArray<FVector3f> ComponentSpaceVertices; // size of num vertices of the geometry entry
						BuildComponentSpaceVertices(Source.Transform.Get(), Source.Parent.Get(), Source.BoneMap.Get(), Source.Vertex.Get(),
							Source.VertexStart[AttributeGeometryIndex], Source.VertexCount[AttributeGeometryIndex], ComponentSpaceVertices);

						// build Sphere based BVH
						Chaos::FReal SphereRadius = SphereFullRadius;
						if (TransferProperties.SourceScale == EDataflowTransferVertexAttributeNodeSourceScale::Component_Edge)
						{
							SphereRadius = Chaos::FReal(TransferProperties.EdgeMultiplier * FMath::Max(
								MaxEdgeLength(ComponentSpaceTargetVertices, Target.Indices.Get(), Target.VertexStart[TargetGeometryIndex], Target.FaceStart[TargetGeometryIndex], Target.FaceCount[TargetGeometryIndex]),
								MaxEdgeLength(ComponentSpaceVertices, Source.Indices.Get(), Source.VertexStart[AttributeGeometryIndex], Source.FaceStart[AttributeGeometryIndex], Source.FaceCount[AttributeGeometryIndex])));
						}

						const int32 TargetVertexStartVal = Target.VertexStart[TargetGeometryIndex];
						const int32 TargetVertexCountVal = Target.VertexCount[TargetGeometryIndex];
						const int32 VertexStartVal = Source.VertexStart[AttributeGeometryIndex];
						const int32 FaceStartVal = Source.FaceStart[AttributeGeometryIndex];
						const int32 FaceCountVal = Source.FaceCount[AttributeGeometryIndex];

						if (TransferProperties.BoundingVolumeType == EDataflowTransferVertexAttributeNodeBoundingVolume::Triangle)
						{
							TArray<Chaos::TVec3<Chaos::FReal>> ComponentSpaceVerticesTVec3;
							ComponentSpaceVerticesTVec3.SetNum(ComponentSpaceVertices.Num());
							for (int32 SourceIndex = 0; SourceIndex < ComponentSpaceVerticesTVec3.Num(); SourceIndex++)
							{
								ComponentSpaceVerticesTVec3[SourceIndex] = Chaos::TVec3<Chaos::FReal>(ComponentSpaceVertices[SourceIndex]);
							}
							TConstArrayView<Chaos::TVec3<Chaos::FReal>> ConstComponentSpaceVertices(ComponentSpaceVerticesTVec3);
							Chaos::FTriangleMesh TriangleMesh;
							TArray<Chaos::TVec3<int32>> SourceElements;
							SourceElements.SetNum(FaceCountVal);
							for (int32 ElementIndex = 0; ElementIndex < FaceCountVal; ++ElementIndex)
							{
								FIntVector3 Element = Source.Indices[FaceStartVal + ElementIndex];
								SourceElements[ElementIndex] = Chaos::TVec3<int32>(Element[0] - VertexStartVal, Element[1] - VertexStartVal, Element[2] - VertexStartVal);
							}
							TriangleMesh.Init(SourceElements);
							Chaos::FTriangleMesh::TSpatialHashType<Chaos::FReal> SpatialHash;
							TriangleMesh.BuildSpatialHash(ConstComponentSpaceVertices, SpatialHash, SphereRadius);
							for (int32 TargetIndex = 0; TargetIndex < TargetVertexCountVal; ++TargetIndex)
							{
								TArray<Chaos::TTriangleCollisionPoint<Chaos::FReal>> Result;
								if (TriangleMesh.PointClosestTriangleQuery(SpatialHash, ConstComponentSpaceVertices,
									TargetIndex, Chaos::TVec3<Chaos::FReal>(ComponentSpaceTargetVertices[TargetIndex]), SphereRadius / 2.f, SphereRadius / 2.f,
									[](const int32 PointIndex, const int32 TriangleIndex)->bool {return true; }, Result))
								{
									for (const Chaos::TTriangleCollisionPoint<Chaos::FReal>& CollisionPoint : Result)
									{
										const float CurrentDistance = abs(CollisionPoint.Phi);
										const float TriRadius = TransferProperties.FalloffThreshold * MaxEdgeLength(ComponentSpaceVertices, Source.Indices.Get(), VertexStartVal, FaceStartVal + CollisionPoint.Indices[1], 1);
										const float FalloffScale = CalculateFalloffScale(TransferProperties.Falloff, TriRadius, CurrentDistance);
										if (!FMath::IsNearlyZero(FalloffScale))
										{
											const int32 TargetCandidateIndex = CollisionPoint.Indices[0] + TargetVertexStartVal;
											TArray<int32> TransferIndices;
											TArray<float> TransferWeights;
											for (int32 k = 0; k < 3; k++)
											{
												TransferIndices.Add(Source.Indices[FaceStartVal + CollisionPoint.Indices[1]][k]);
												TransferWeights.Add(CollisionPoint.Bary[k + 1]);
											}
											TransferDataArray[TargetCandidateIndex] = FTransferData(TransferIndices, TransferWeights, FalloffScale);
											break;
										}
									}
								}
							}
						}
						else if (TransferProperties.BoundingVolumeType == EDataflowTransferVertexAttributeNodeBoundingVolume::Vertex)
						{
							const float FalloffScale = 1.f;
							TUniquePtr<BVH> VertexBVH(BuildParticleSphereBVH(ComponentSpaceVertices, SphereRadius));
							for (int32 TargetIdx = 0; TargetIdx < ComponentSpaceTargetVertices.Num(); ++TargetIdx)
							{
								float MinDistance = FLT_MAX;
								int32 MinSourceIdx = INDEX_NONE;
								for (int32 SourceIdx : VertexBVH->FindAllIntersections(ComponentSpaceTargetVertices[TargetIdx]))
								{
									float Distance = (ComponentSpaceTargetVertices[TargetIdx] - ComponentSpaceVertices[SourceIdx]).Size();
									if (Distance < MinDistance)
									{
										MinDistance = Distance;
										MinSourceIdx = SourceIdx;
									}
								}
								if (MinSourceIdx != INDEX_NONE)
								{
									TArray<int32> TransferIndices = { MinSourceIdx + VertexStartVal };
									TArray<float> TransferWeights = { 1.f };
									TransferDataArray[TargetIdx + TargetVertexStartVal] = FTransferData(TransferIndices, TransferWeights, FalloffScale);
								}
							}
						}
					}
				}
			});
		return MoveTemp(TransferDataArray);
	}

	static TArray<FTransferData> GlobalAssetTransfer(const FTransferFacade& Source, const FTransferFacade& Target, const FTransferProperties& TransferProperties)
	{
		TArray<FTransferData> TransferDataArray;
		TransferDataArray.SetNum(Target.Vertex.Num());
		// Build component space vertices for TargetCollection
		TArray<FVector3f> ComponentSpaceTargetVertices;
		BuildComponentSpaceVertices(Target.Transform.Get(), Target.Parent.Get(), Target.BoneMap.Get(), Target.Vertex.Get(), 0, Target.Vertex.Num(), ComponentSpaceTargetVertices);

		// Build component space vertices for SourceCollection
		TArray<FVector3f> ComponentSpaceVertices;
		BuildComponentSpaceVertices(Source.Transform.Get(), Source.Parent.Get(), Source.BoneMap.Get(), Source.Vertex.Get(), 0, Source.Vertex.Num(), ComponentSpaceVertices);

		// build Sphere based BVH
		Chaos::FReal SphereRadius = 0.f;
		if (TransferProperties.SourceScale == EDataflowTransferVertexAttributeNodeSourceScale::Asset_Edge || TransferProperties.SourceScale == EDataflowTransferVertexAttributeNodeSourceScale::Component_Edge)
		{
			SphereRadius = Chaos::FReal(TransferProperties.EdgeMultiplier * FMath::Max(
				MaxEdgeLength(ComponentSpaceTargetVertices, Target.Indices.Get(), 0, 0, Target.Indices.Num()),
				MaxEdgeLength(ComponentSpaceVertices, Source.Indices.Get(), 0, 0, Source.Indices.Num())));
		}
		else if (TransferProperties.SourceScale == EDataflowTransferVertexAttributeNodeSourceScale::Asset_Bound)
		{
			Chaos::FVec3f CoordMaxs(-FLT_MAX);
			Chaos::FVec3f CoordMins(FLT_MAX);
			for (int32 i = 0; i < ComponentSpaceVertices.Num(); i++)
			{
				CoordMaxs = CoordMaxs.ComponentwiseMax(ComponentSpaceVertices[i]);
				CoordMins = CoordMins.ComponentwiseMin(ComponentSpaceVertices[i]);
			}
			const Chaos::FVec3f CoordDiff = (CoordMaxs - CoordMins) * TransferProperties.BoundMultiplier;
			SphereRadius = FMath::Min3(CoordDiff[0], CoordDiff[1], CoordDiff[2]);
		}
		if (TransferProperties.BoundingVolumeType == EDataflowTransferVertexAttributeNodeBoundingVolume::Triangle)
		{
			TArray<Chaos::TVec3<Chaos::FReal>> ComponentSpaceVerticesTVec3;
			ComponentSpaceVerticesTVec3.SetNum(ComponentSpaceVertices.Num());
			for (int32 SourceIndex = 0; SourceIndex < ComponentSpaceVerticesTVec3.Num(); SourceIndex++)
			{
				ComponentSpaceVerticesTVec3[SourceIndex] = Chaos::TVec3<Chaos::FReal>(ComponentSpaceVertices[SourceIndex]);
			}
			TConstArrayView<Chaos::TVec3<Chaos::FReal>> ConstComponentSpaceVertices(ComponentSpaceVerticesTVec3);
			Chaos::FTriangleMesh TriangleMesh;
			TArray<Chaos::TVec3<int32>> SourceElements;
			SourceElements.SetNum(Source.Indices.Num());
			for (int32 ElementIndex = 0; ElementIndex < Source.Indices.Num(); ++ElementIndex)
			{
				SourceElements[ElementIndex] = Chaos::TVec3<int32>(Source.Indices[ElementIndex][0], Source.Indices[ElementIndex][1], Source.Indices[ElementIndex][2]);
			}
			TriangleMesh.Init(SourceElements);
			Chaos::FTriangleMesh::TSpatialHashType<Chaos::FReal> SpatialHash;
			TriangleMesh.BuildSpatialHash(ConstComponentSpaceVertices, SpatialHash, SphereRadius);
			for (int32 TargetIndex = 0; TargetIndex < ComponentSpaceTargetVertices.Num(); ++TargetIndex)
			{
				TArray<Chaos::TTriangleCollisionPoint<Chaos::FReal>> Result;
				if (TriangleMesh.PointClosestTriangleQuery(SpatialHash, ConstComponentSpaceVertices,
					TargetIndex, Chaos::TVec3<Chaos::FReal>(ComponentSpaceTargetVertices[TargetIndex]), SphereRadius / 2.f, SphereRadius / 2.f,
					[](const int32 PointIndex, const int32 TriangleIndex)->bool {return true; }, Result))
				{
					for (const Chaos::TTriangleCollisionPoint<Chaos::FReal>& CollisionPoint : Result)
					{
						const float CurrentDistance = abs(CollisionPoint.Phi);
						const float TriRadius = TransferProperties.FalloffThreshold * MaxEdgeLength(ComponentSpaceVertices, Source.Indices.Get(), 0, CollisionPoint.Indices[1], 1);
						const float FalloffScale = CalculateFalloffScale(TransferProperties.Falloff, TriRadius, CurrentDistance);
						if (!FMath::IsNearlyZero(FalloffScale))
						{
							const int32 TargetCandidateIndex = CollisionPoint.Indices[0];
							TArray<int32> TransferIndices;
							TArray<float> TransferWeights;
							for (int32 k = 0; k < 3; k++)
							{
								TransferIndices.Add(Source.Indices[CollisionPoint.Indices[1]][k]);
								TransferWeights.Add(CollisionPoint.Bary[k + 1]);
							}
							TransferDataArray[TargetCandidateIndex] = FTransferData(TransferIndices, TransferWeights, FalloffScale);
							break;
						}
					}
				}
			}
		}
		else if (TransferProperties.BoundingVolumeType == EDataflowTransferVertexAttributeNodeBoundingVolume::Vertex)
		{
			const float FalloffScale = 1.f;
			TUniquePtr<BVH> VertexBVH(BuildParticleSphereBVH(ComponentSpaceVertices, SphereRadius));
			for (int32 TargetIdx = 0; TargetIdx < ComponentSpaceTargetVertices.Num(); ++TargetIdx)
			{
				float MinDistance = FLT_MAX;
				int32 MinSourceIdx = INDEX_NONE;
				for (int32 SourceIdx : VertexBVH->FindAllIntersections(ComponentSpaceTargetVertices[TargetIdx]))
				{
					float Distance = (ComponentSpaceTargetVertices[TargetIdx] - ComponentSpaceVertices[SourceIdx]).Size();
					if (Distance < MinDistance)
					{
						MinDistance = Distance;
						MinSourceIdx = SourceIdx;
					}
				}
				if (MinSourceIdx != INDEX_NONE)
				{
					TArray<int32> TransferIndices = { MinSourceIdx };
					TArray<float> TransferWeights = { 1.f };
					TransferDataArray[TargetIdx] = FTransferData(TransferIndices, TransferWeights, FalloffScale);
				}
			}
		}
		return MoveTemp(TransferDataArray);
	}

	//store barycentric weight info TransferDataArray and returns transfer method used
	static TArray<FTransferData> ComputeTransferData(const FManagedArrayCollection& SourceCollection, const FManagedArrayCollection& TargetCollection, const FTransferProperties& TransferProperties, const EDataflowTransferVertexAttributeNodeTransferMethod& TransferMethod)
	{
		TArray<FTransferData> TransferDataArray;
		const FTransferFacade Target(TargetCollection);
		const FTransferFacade Source(SourceCollection);
		if (Target.IsValid() && Source.IsValid())
		{
			if (TransferMethod == EDataflowTransferVertexAttributeNodeTransferMethod::Component)
			{
				TArray<FIntVector2> AlignedGeometry = FindSourceToTargetGeometryMap(SourceCollection, TargetCollection, TransferProperties.TransformNameSuffix);
				if (AlignedGeometry.Num() != SourceCollection.NumElements(FGeometryCollection::GeometryGroup))
				{
					TSet<int32> MatchedSourceGeometries;
					for (const FIntVector2& Pair : AlignedGeometry)
					{
						MatchedSourceGeometries.Add(Pair.X);
					}
					for (int32 SourceIdx = 0; SourceIdx < SourceCollection.NumElements(FGeometryCollection::GeometryGroup); ++SourceIdx)
					{
						if (!MatchedSourceGeometries.Contains(SourceIdx))
						{
							UE_LOGF(LogChaosDataflow, Warning, "Source geometry indexed %d is not matched to any target geometry. Check if this is intentional.", SourceIdx);
						}
					}
				}
				TransferDataArray = PairedGeometryTransfer(AlignedGeometry, Source, Target, TransferProperties);
			}
			else if (TransferMethod == EDataflowTransferVertexAttributeNodeTransferMethod::Global)
			{
				TransferDataArray = GlobalAssetTransfer(Source, Target, TransferProperties);
			}
		}
		return TransferDataArray;
	}
}

using namespace UE::Private;

void FGeometryCollectionTransferVertexAttributeNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	FCollectionAttributeKey Key = GetValue(Context, &AttributeKey, AttributeKey);

	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection TargetCollection = GetValue(Context, &Collection);
		const FManagedArrayCollection& SourceCollection = GetValue(Context, &FromCollection);
		FTransferFacade Target(TargetCollection);
		const FTransferFacade Source(SourceCollection);

		const FTransferProperties TransferProperties = {
			BoundingVolumeType,
			SourceScale,
			Falloff,
			FalloffThreshold,
			EdgeMultiplier,
			BoundMultiplier,
			TransformNameSuffix
		};

		if (TransferMethod != EDataflowTransferVertexAttributeNodeTransferMethod::None
			&& Target.IsValid() && Source.IsValid())
		{
			TArray<FTransferData> TransferDataArray = ComputeTransferData(SourceCollection, TargetCollection, TransferProperties, TransferMethod);
			if (const TManagedArray<float>* SourceAttributeFloatArray = Source.GetAttributeArray<float>(Key.Attribute, Key.Group))
			{
				TManagedArray<float>* TargetAttributeArray = Target.GetAttributeArray<float>(Key.Attribute, Key.Group);
				TargetAttributeArray->Fill(0.f);
				for (int32 VertexIndex = 0; VertexIndex < TargetAttributeArray->Num(); VertexIndex++)
				{
					const FTransferData& TransferData = TransferDataArray[VertexIndex];
					if (TransferData.SourceIndices.Num())
					{
						for (int32 SourceLocalIdx = 0; SourceLocalIdx < TransferData.SourceIndices.Num(); ++SourceLocalIdx)
						{
							(*TargetAttributeArray)[VertexIndex] += (*SourceAttributeFloatArray)[TransferData.SourceIndices[SourceLocalIdx]]
								* TransferData.SourceWeights[SourceLocalIdx] * TransferData.FalloffScale;
						}
					}
				}
			}
			else if (const TManagedArray<FLinearColor>* SourceAttributeColorArray = Source.GetAttributeArray<FLinearColor>(Key.Attribute, Key.Group))
			{
				TManagedArray<FLinearColor>* TargetAttributeArray = Target.GetAttributeArray<FLinearColor>(Key.Attribute, Key.Group);
				TargetAttributeArray->Fill(FLinearColor(0, 0, 0, 0));
				for (int32 VertexIndex = 0; VertexIndex < TargetAttributeArray->Num(); VertexIndex++)
				{
					const FTransferData& TransferData = TransferDataArray[VertexIndex];
					if (TransferData.SourceIndices.Num())
					{
						for (int32 SourceLocalIdx = 0; SourceLocalIdx < TransferData.SourceIndices.Num(); ++SourceLocalIdx)
						{
							(*TargetAttributeArray)[VertexIndex] += (*SourceAttributeColorArray)[TransferData.SourceIndices[SourceLocalIdx]]
								* TransferData.SourceWeights[SourceLocalIdx] * TransferData.FalloffScale;
						}
					}
				}
			}
			else
			{
				Context.Error(TEXT("Source attribute is not found"), this, Out);
				UE_LOGF(LogChaosDataflow, Warning, "TransferVertexAttribute: attribute[%ls] of group[%ls] has type that is not float or FLinearColor, transfer failed.", *Key.Attribute, *Key.Group);
			}
		}

		SetValue(Context, MoveTemp(TargetCollection), &Collection);
	}
	else if (Out->IsA(&AttributeKey))
	{
		SetValue(Context, MoveTemp(Key), &AttributeKey);
	}
}

void FGeometryCollectionTransferVertexSkinWeightsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection TargetCollection = GetValue(Context, &Collection);
		const FManagedArrayCollection& SourceCollection = GetValue(Context, &FromCollection);
		FTransferFacade Target(TargetCollection);
		const FTransferFacade Source(SourceCollection);

		const FTransferProperties TransferProperties = {
			BoundingVolumeType,
			SourceScale,
			Falloff,
			FalloffThreshold,
			EdgeMultiplier,
			BoundMultiplier,
			TransformNameSuffix
		};

		if (TransferMethod != EDataflowTransferVertexAttributeNodeTransferMethod::None
			&& Target.IsValid() && Source.IsValid())
		{
			TArray<FTransferData> TransferDataArray = ComputeTransferData(SourceCollection, TargetCollection, TransferProperties, TransferMethod);
			GeometryCollection::Facades::FVertexBoneWeightsFacade SourceVertexBoneWeightsFacade(SourceCollection);
			GeometryCollection::Facades::FVertexBoneWeightsFacade TargetVertexBoneWeightsFacade(TargetCollection);
			const TManagedArray<TArray<int32>>* SourceBoneIndices = SourceVertexBoneWeightsFacade.FindBoneIndices();
			const TManagedArray<TArray<float>>* SourceBoneWeights = SourceVertexBoneWeightsFacade.FindBoneWeights();
			if (SourceBoneIndices && SourceBoneWeights)
			{
				//
				// Compute the bone index mappings. This allows the transfer operator to retarget weights to the correct skeleton.
				//
				const TMap<int32, int32> SourceBoneToTargetBone = FindSourceToTargetTransformMap(SourceCollection, TargetCollection);
				TSet<int32> MissingSourceBones;
				for (int32 VertexIndex = 0; VertexIndex < Target.Vertex.Num(); VertexIndex++)
				{
					const FTransferData& TransferData = TransferDataArray[VertexIndex];
					if (TransferData.SourceIndices.Num())
					{
						TMap<int32, float> BoneWeightBucket;
						for (int32 SourceLocalIdx = 0; SourceLocalIdx < TransferData.SourceIndices.Num(); ++SourceLocalIdx)
						{
							for (int32 BoneLocalIdx = 0; BoneLocalIdx < (*SourceBoneIndices)[TransferData.SourceIndices[SourceLocalIdx]].Num(); ++BoneLocalIdx)
							{
								const int32 SourceBoneIndex = (*SourceBoneIndices)[TransferData.SourceIndices[SourceLocalIdx]][BoneLocalIdx];
								if (SourceBoneToTargetBone.Contains(SourceBoneIndex))
								{
									const int32 TargetBoneIndex = SourceBoneToTargetBone[SourceBoneIndex];
									const float BoneWeight = (*SourceBoneWeights)[TransferData.SourceIndices[SourceLocalIdx]][BoneLocalIdx];
									if (BoneWeightBucket.Contains(TargetBoneIndex))
									{
										BoneWeightBucket[TargetBoneIndex] += TransferData.SourceWeights[SourceLocalIdx] * BoneWeight;
									}
									else
									{
										BoneWeightBucket.Add(TargetBoneIndex, TransferData.SourceWeights[SourceLocalIdx] * BoneWeight);
									}
								}
								else if (!MissingSourceBones.Contains(SourceBoneIndex))
								{
									MissingSourceBones.Add(SourceBoneIndex);
									UE_LOGF(LogChaosDataflow, Warning, "Target geometry does not contain Source Transform indexed [%d].", SourceBoneIndex);
								}
							}
						}

						TArray<int32> VertexBoneIndex;
						TArray<float> VertexBoneWeight;
						VertexBoneIndex.Reserve(BoneWeightBucket.Num());
						VertexBoneWeight.Reserve(BoneWeightBucket.Num());
						for (const TPair<int32, float>& Pair : BoneWeightBucket)
						{
							VertexBoneIndex.Add(Pair.Key);
							VertexBoneWeight.Add(Pair.Value);
						}
						TargetVertexBoneWeightsFacade.ModifyBoneWeight(VertexIndex, VertexBoneIndex, VertexBoneWeight);
					}
				}
			}
			else
			{
				Context.Error(TEXT("Source skin weight is not found"), this, Out);
			}
		}
		SetValue(Context, MoveTemp(TargetCollection), &Collection);
	}
}
void FGeometryCollectionSetKinematicVertexSelectionNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);
		if (IsConnected(&VertexSelection))
		{
			FDataflowVertexSelection VertexSelectionIn = GetValue<FDataflowVertexSelection>(Context, &VertexSelection);
			GeometryCollection::Facades::FVertexBoneWeightsFacade VertexBoneWeightsFacade(InCollection);
			const bool bSetKinematic = (KinematicValue == ESetKinematicVertexSelectionKinematicValue::SetKinematic);
			VertexBoneWeightsFacade.SetVertexArrayKinematic(VertexSelectionIn.AsArray(), bSetKinematic);
		}
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE
