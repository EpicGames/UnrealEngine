// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionTriangleBoundaryIndicesNode.h"

#include "Chaos/BoundingVolumeHierarchy.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/Facades/CollectionPositionTargetFacade.h"
#include "Chaos/Utilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionTriangleBoundaryIndicesNode)

FTriangleBoundaryIndicesNode::FTriangleBoundaryIndicesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&BoundaryIndicesOut);
}

void FTriangleBoundaryIndicesNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&BoundaryIndicesOut))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);

		TArray<Chaos::TVec3<int32>> TriangleMeshArray;

		if (const TManagedArray<int32>* TriangleMeshIndices = InCollection.FindAttribute<int32>("ObjectIndices", "TriangleMesh"))
		{
			if (const TManagedArray<FIntVector>* Indices = InCollection.FindAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup))
			{
				if (const TManagedArray<int32>* FaceStarts = InCollection.FindAttribute<int32>("FaceStart", FGeometryCollection::GeometryGroup))
				{
					if (const TManagedArray<int32>* FaceCounts = InCollection.FindAttribute<int32>("FaceCount", FGeometryCollection::GeometryGroup))
					{
						for (int32 i = 0; i < TriangleMeshIndices->Num(); i++)
						{
							const int32 ObjectIndex = (*TriangleMeshIndices)[i];
							
							const int32 FaceStartIndex = (*FaceStarts)[ObjectIndex];
							const int32 FaceNum = (*FaceCounts)[ObjectIndex];
							for (int32 e = FaceStartIndex; e < FaceStartIndex + FaceNum; e++)
							{
								Chaos::TVec3<int32> IndicesTemp((*Indices)[e][0], (*Indices)[e][1], (*Indices)[e][2]);
								TriangleMeshArray.Emplace(IndicesTemp);
							}
						}	
					}
				}
			}
		}

		TArray<int32> IndicesOut = Chaos::Utilities::ComputeBoundaryNodes(TriangleMeshArray);

		SetValue(Context, MoveTemp(IndicesOut), &BoundaryIndicesOut);
	}
}
