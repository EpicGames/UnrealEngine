// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionUVNodes.h"
#include "GeometryCollection/Facades/CollectionUVFacade.h"

#include "FractureAutoUV.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionUVNodes)

namespace UE::Dataflow
{
	void RegisterGeometryCollectionUVNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAddUVChannelDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FAutoUnwrapUVDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMergeUVIslandsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBoxProjectUVDataflowNode);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////

FAddUVChannelDataflowNode::FAddUVChannelDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);

	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&UVChannel);
}

void FAddUVChannelDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection) || Out->IsA(&UVChannel))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);

		GeometryCollection::Facades::FCollectionUVFacade UVFacade(InCollection);

		int32 NumUVChannel = 0;
		if (UVFacade.IsValid())
		{
			NumUVChannel = UVFacade.GetNumUVLayers();
		}

		int32 NewUVChannel = NumUVChannel;
		if (UVFacade.SetNumUVLayers(NumUVChannel + 1))
		{
			// init the UV
			if (TManagedArray<FVector2f>* UVAttribute = UVFacade.FindUVLayer(NewUVChannel))
			{
				UVAttribute->Fill(DefaultValue);
			}
		}
		else
		{
			// for now if we fail ( we maxed out thenumber of channel for example ), let's use the channel 0 so that it is obvious
			// todo(dataflow) in the future we should output an error 
			NewUVChannel = 0;
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);
		SetValue(Context, NewUVChannel, &UVChannel);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////

FMergeUVIslandsDataflowNode::FMergeUVIslandsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&FaceSelection);
	RegisterInputConnection(&UVChannel);
	RegisterInputConnection(&AreaDistortionThreshold).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxNormalDeviationDeg).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&NormalSmoothingRounds).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&NormalSmoothingAlpha).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&UVChannel, &UVChannel);
}

void FMergeUVIslandsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&UVChannel))
	{
		SafeForwardInput(Context, &UVChannel, &UVChannel);
	}
	else if (Out->IsA(&Collection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const int32 FaceCount = InCollection.NumElements(FGeometryCollection::FacesGroup);
		if (FaceCount == 0)
		{
			// nothing to do : forward and exit 
			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		const int32 InUVChannel = GetValue(Context, &UVChannel);
		UE::PlanarCut::FMergeIslandSettings MergeIslandSettings;
		MergeIslandSettings.MaxNormalDeviationDeg = GetValue(Context, &MaxNormalDeviationDeg);
		MergeIslandSettings.AreaDistortionThreshold = GetValue(Context, &AreaDistortionThreshold);
		MergeIslandSettings.NormalSmoothingAlpha = GetValue(Context, &NormalSmoothingAlpha);
		MergeIslandSettings.NormalSmoothingRounds = GetValue(Context, &NormalSmoothingRounds);

		TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>());
		check(GeomCollection.IsValid());

		if (IsConnected(&FaceSelection))
		{
			const FDataflowFaceSelection& InFaceSelection = GetValue(Context, &FaceSelection);
			if (InFaceSelection.Num() > 0)
			{
				TArray<bool> FacesToUnwrap;
				FacesToUnwrap.SetNumUninitialized(InFaceSelection.Num());
				for (int32 Index = 0; Index < InFaceSelection.Num(); ++Index)
				{
					FacesToUnwrap[Index] = InFaceSelection.IsSelected(Index);
				}

				UE::PlanarCut::MergeUVIslands(InUVChannel, *GeomCollection, MergeIslandSettings,
					FacesToUnwrap, nullptr /* progress */);
			}
		}
		else
		{
			// All visible faces
			TArray<bool> FacesToUnwrap = GeomCollection->Visible.GetAsBoolArray();
			UE::PlanarCut::MergeUVIslands(InUVChannel, *GeomCollection, MergeIslandSettings,
				FacesToUnwrap, nullptr /* progress */);
		}

		SetValue(Context, (const FManagedArrayCollection&)(*GeomCollection), &Collection);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////

FAutoUnwrapUVDataflowNode::FAutoUnwrapUVDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&FaceSelection);
	RegisterInputConnection(&UVChannel);
	RegisterInputConnection(&GutterSize).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&UVChannel, &UVChannel);
}

void FAutoUnwrapUVDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&UVChannel))
	{
		SafeForwardInput(Context, &UVChannel, &UVChannel);
	}
	else if (Out->IsA(&Collection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const int32 FaceCount = InCollection.NumElements(FGeometryCollection::FacesGroup);
		if (FaceCount == 0)
		{
			// nothing to do : forward and exit 
			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		const int32 InUVChannel = GetValue(Context, &UVChannel);
		const int32 InGutterSize = FMath::Max(1, GetValue(Context, &GutterSize));

		TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>());
		check(GeomCollection.IsValid());

		// this is used as reference for the gutter size that is expressed in pixels for a specific resolution 
		constexpr int32 ReferenceResolution = 512;
		constexpr bool bRecreateUVsForDegenerateIslands = true;

		if (IsConnected(&FaceSelection))
		{
			const FDataflowFaceSelection& InFaceSelection = GetValue(Context, &FaceSelection);
			if (InFaceSelection.Num() > 0)
			{
				TArray<bool> FacesToUnwrap;
				FacesToUnwrap.SetNumUninitialized(InFaceSelection.Num());
				for (int32 Index = 0; Index < InFaceSelection.Num(); ++Index)
				{
					FacesToUnwrap[Index] = InFaceSelection.IsSelected(Index);
				}

				UE::PlanarCut::UVLayout(InUVChannel, *GeomCollection, ReferenceResolution, InGutterSize,
					FacesToUnwrap, bRecreateUVsForDegenerateIslands, nullptr /* progress */);
			}
		}
		else
		{
			// All faces
			UE::PlanarCut::UVLayout(InUVChannel, *GeomCollection, ReferenceResolution, InGutterSize,
				UE::PlanarCut::ETargetFaces::AllFaces, {}, bRecreateUVsForDegenerateIslands, nullptr /* progress */);
		}
		
		SetValue(Context, (const FManagedArrayCollection&)(*GeomCollection), &Collection);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////

FBoxProjectUVDataflowNode::FBoxProjectUVDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	// RegisterInputConnection(&FaceSelection); not supported yet
	RegisterInputConnection(&UVChannel);
	RegisterInputConnection(&ProjectionScale);
	RegisterInputConnection(&UVOffset);
	RegisterInputConnection(&bAutoFitToBounds);
	RegisterInputConnection(&bCenterBoxAtPivot);
	RegisterInputConnection(&bUniformProjectionScale);
	RegisterInputConnection(&GutterSize).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&UVChannel, &UVChannel);
}

void FBoxProjectUVDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&UVChannel))
	{
		SafeForwardInput(Context, &UVChannel, &UVChannel);
	}
	else if (Out->IsA(&Collection))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const int32 FaceCount = InCollection.NumElements(FGeometryCollection::FacesGroup);
		if (FaceCount == 0)
		{
			// nothing to do : forward and exit 
			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		const int32 InUVChannel = GetValue(Context, &UVChannel);
		const int32 InGutterSize = FMath::Max(1, GetValue(Context, &GutterSize));
		const FVector InProjectionScale = GetValue(Context, &ProjectionScale);
		const FVector2f InUVOffset = GetValue(Context, &UVOffset);
		const bool bInAutoFitToBounds = GetValue(Context, &bAutoFitToBounds);
		const bool bInCenterBoxAtPivot = GetValue(Context, &bCenterBoxAtPivot);
		const bool bInUniformProjectionScale = GetValue(Context, &bUniformProjectionScale);

		TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>());
		check(GeomCollection.IsValid());

		// this is used as reference for the gutter size that is expressed in pixels for a specific resolution 
		constexpr int32 ReferenceResolution = 512;
		constexpr bool bRecreateUVsForDegenerateIslands = true;

		//if (IsConnected(&FaceSelection))
		//{
		//	const FDataflowFaceSelection& InFaceSelection = GetValue(Context, &FaceSelection);
		//	if (InFaceSelection.Num() > 0)
		//	{
		//		TArray<bool> FacesToUnwrap;
		//		FacesToUnwrap.SetNumUninitialized(InFaceSelection.Num());
		//		for (int32 Index = 0; Index < InFaceSelection.Num(); ++Index)
		//		{
		//			FacesToUnwrap[Index] = InFaceSelection.IsSelected(Index);
		//		}

		//		UE::PlanarCut::UVLayout(InUVChannel, *GeomCollection, ReferenceResolution, InGutterSize,
		//			FacesToUnwrap, bRecreateUVsForDegenerateIslands, nullptr /* progress */);
		//	}
		//}
		//else
		{
			// All faces
			UE::PlanarCut::BoxProjectUVs(
				InUVChannel, *GeomCollection, 
				InProjectionScale, 
				UE::PlanarCut::ETargetFaces::AllFaces, {}, 
				InUVOffset, 
				bAutoFitToBounds,
				bCenterBoxAtPivot,
				bUniformProjectionScale
				);
		}

		SetValue(Context, (const FManagedArrayCollection&)(*GeomCollection), &Collection);
	}
}
