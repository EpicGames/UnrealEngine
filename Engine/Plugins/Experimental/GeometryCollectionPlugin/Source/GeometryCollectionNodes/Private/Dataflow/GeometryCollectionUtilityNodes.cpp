// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionUtilityNodes.h"
#include "Dataflow/DataflowCore.h"
#if WITH_EDITOR
#include "Dataflow/DataflowRenderingViewMode.h"
#endif

#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"
#include "GeometryCollectionToDynamicMesh.h"
#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowDebugDraw.h"
#include "Dataflow/DataflowDynamicMeshDebugDrawMesh.h"
#include "FractureEngineConvex.h"
#include "DynamicMesh/DynamicMesh3.h"

#include "Chaos/Sphere.h"
#include "Chaos/Box.h"
#include "Chaos/Capsule.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/ImplicitObjectUnion.h"

#include "PCA3.h"

#include "Operations/MeshSelfUnion.h"
#include "MeshQueries.h"

#include "MeshSimplification.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionUtilityNodes)


#define LOCTEXT_NAMESPACE "GeometryCollectionUtilityNodes"

namespace UE::Dataflow
{

	void GeometryCollectionUtilityNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMakeDataflowConvexDecompositionSettingsNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCreateNegativeSpaceSphereCovering);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCreateLeafConvexHullsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSetExternalCollisionsFromPrimitiveDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSimplifyConvexHullsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCreateNonOverlappingConvexHullsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGenerateClusterConvexHullsFromLeafHullsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGenerateClusterConvexHullsFromChildrenHullsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FClearConvexHullsDataflowNode);
		// Note: FCopyConvexHullsFromRootDataflowNode is temporarily disabled as we rework its functionality
		//DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FCopyConvexHullsFromRootDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMergeConvexHullsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FUpdateVolumeAttributesDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FGetConvexHullVolumeDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FFixTinyGeoDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSplitIslandsDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FRecomputeNormalsInGeometryCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FResampleGeometryCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FValidateGeometryCollectionDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FComputeVolumeStatsDataflowNode);
	}
}

FMakeDataflowConvexDecompositionSettingsNode::FMakeDataflowConvexDecompositionSettingsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MinSizeToDecompose).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxGeoToHullVolumeRatioToDecompose).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ErrorTolerance).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxHullsPerGeometry).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinThicknessTolerance).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&NumAdditionalSplits).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bProtectNegativeSpace).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bOnlyConnectedToHull).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&NegativeSpaceTolerance).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&NegativeSpaceMinRadius).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&DecompositionSettings);
}

void FMakeDataflowConvexDecompositionSettingsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&DecompositionSettings))
	{
		FDataflowConvexDecompositionSettings OutSettings;
		OutSettings.MinSizeToDecompose = GetValue(Context, &MinSizeToDecompose);
		OutSettings.MaxGeoToHullVolumeRatioToDecompose = GetValue(Context, &MaxGeoToHullVolumeRatioToDecompose);
		OutSettings.ErrorTolerance = GetValue(Context, &ErrorTolerance);
		OutSettings.MaxHullsPerGeometry = GetValue(Context, &MaxHullsPerGeometry);
		OutSettings.MinThicknessTolerance = GetValue(Context, &MinThicknessTolerance);
		OutSettings.NumAdditionalSplits = GetValue(Context, &NumAdditionalSplits);
		OutSettings.bProtectNegativeSpace = GetValue(Context, &bProtectNegativeSpace);
		OutSettings.bOnlyConnectedToHull = GetValue(Context, &bOnlyConnectedToHull);
		OutSettings.NegativeSpaceTolerance = GetValue(Context, &NegativeSpaceTolerance);
		OutSettings.NegativeSpaceMinRadius = GetValue(Context, &NegativeSpaceMinRadius);

		SetValue(Context, OutSettings, &DecompositionSettings);
	}
}

/* --------------------------------------------------------------------------------------------------------------------------- */

namespace UE::Dataflow::Convex
{
	static FLinearColor GetRandomColor(const int32 RandomSeed, int32 Idx)
	{
		FRandomStream RandomStream(RandomSeed * 23 + Idx * 4078);

		const uint8 R = static_cast<uint8>(RandomStream.FRandRange(128, 255));
		const uint8 G = static_cast<uint8>(RandomStream.FRandRange(128, 255));
		const uint8 B = static_cast<uint8>(RandomStream.FRandRange(128, 255));

		return FLinearColor(FColor(R, G, B, 255));
	}

	static void DebugDrawProc(IDataflowDebugDrawInterface& DataflowRenderingInterface, const FManagedArrayCollection& InCollection, const bool bRandomizeColor, const int32 ColorRandomSeed, const FDataflowTransformSelection& Selection)
	{
		using namespace UE::Geometry;

		TArray<FDynamicMesh3> HullsMeshes;
		const bool bRestrictToSelection = (Selection.Num() > 0);
	
		UE::FractureEngine::Convex::GetConvexHullsAsDynamicMeshes(InCollection, HullsMeshes, bRestrictToSelection, Selection.AsArray());

		int32 Idx = 0;
		for (const FDynamicMesh3& Mesh : HullsMeshes)
		{
			FDynamicMeshDebugDrawMesh DebugdrawMesh(&Mesh);
			DataflowRenderingInterface.DrawMesh(DebugdrawMesh);

			if (bRandomizeColor)
			{
				DataflowRenderingInterface.SetColor(UE::Dataflow::Convex::GetRandomColor(ColorRandomSeed, Idx++));
			}
		}
	}

	static void SphereCoveringDebugDrawProc(IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDataflowSphereCovering& OutSpheres, const FDataflowNodeSphereCoveringDebugDrawSettings& SphereCoveringDebugDrawRenderSettings)
	{
		constexpr int32 CMaxNumberOfSpheres = 500;
		using namespace UE::Geometry;

		const int32 NumSpheres = OutSpheres.Spheres.Num();
		if (NumSpheres > 0)
		{
			DataflowRenderingInterface.SetLineWidth(SphereCoveringDebugDrawRenderSettings.LineWidthMultiplier);
			if (SphereCoveringDebugDrawRenderSettings.RenderType == EDataflowDebugDrawRenderType::Shaded)
			{
				DataflowRenderingInterface.SetShaded(true);
				DataflowRenderingInterface.SetTranslucent(SphereCoveringDebugDrawRenderSettings.bTranslucent);
				DataflowRenderingInterface.SetWireframe(true);
			}
			else
			{
				DataflowRenderingInterface.SetShaded(false);
				DataflowRenderingInterface.SetWireframe(true);
			}
			DataflowRenderingInterface.SetWorldPriority();
			DataflowRenderingInterface.SetColor(SphereCoveringDebugDrawRenderSettings.Color);

			int32 N = 1;
			if (NumSpheres > CMaxNumberOfSpheres)
			{
				N = NumSpheres / CMaxNumberOfSpheres;
			}

			float MinRadius = FLT_MAX, MaxRadius = -FLT_MAX;
			for (int32 Idx = 0; Idx < NumSpheres; ++Idx)
			{
				if (Idx % N == 0)
				{
					const float Radius = OutSpheres.Spheres.GetRadius(Idx);

					if (Radius < MinRadius)
					{
						MinRadius = Radius;
					}
					else if (Radius > MaxRadius)
					{
							MaxRadius = Radius;
					}
				}
			}

			for (int32 Idx = 0; Idx < NumSpheres; ++Idx)
			{
				if (Idx % N == 0)
				{
					if (SphereCoveringDebugDrawRenderSettings.ColorMethod == EDataflowSphereCoveringColorMethod::Random)
					{
						DataflowRenderingInterface.SetColor(UE::Dataflow::Convex::GetRandomColor(SphereCoveringDebugDrawRenderSettings.ColorRandomSeed + 7, Idx));
					}
					else if (SphereCoveringDebugDrawRenderSettings.ColorMethod == EDataflowSphereCoveringColorMethod::ColorByRadius)
					{
						float Progress = (OutSpheres.Spheres.GetRadius(Idx) - MinRadius) / (MaxRadius - MinRadius);
						FLinearColor Color = FLinearColor::LerpUsingHSV(SphereCoveringDebugDrawRenderSettings.ColorA, SphereCoveringDebugDrawRenderSettings.ColorB, Progress);
						DataflowRenderingInterface.SetColor(Color);
					}

					DataflowRenderingInterface.DrawSphere(OutSpheres.Spheres.GetCenter(Idx), OutSpheres.Spheres.GetRadius(Idx));
				}
			}
		}
	}
}

FCreateLeafConvexHullsDataflowNode::FCreateLeafConvexHullsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&OptionalSelectionFilter);
	RegisterInputConnection(&SimplificationDistanceThreshold).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ConvexDecompositionSettings).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RequireNegativeSpaceCovering).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Collection);
	RegisterOutputConnection(&SphereCovering);

	DebugDrawRenderSettings.RenderType = EDataflowDebugDrawRenderType::Wireframe;
	DebugDrawRenderSettings.Color = FLinearColor::Green;
	DebugDrawRenderSettings.LineWidthMultiplier = 2.0;
}

void FCreateLeafConvexHullsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection) || Out->IsA(&SphereCovering))
	{
		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		FDataflowSphereCovering CombinedSphereCovering;
		if (InCollection.NumElements(FGeometryCollection::TransformGroup) == 0)
		{
			SetValue(Context, InCollection, &Collection);
			SetValue(Context, MoveTemp(CombinedSphereCovering), &SphereCovering);
			return;
		}

		if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
		{
			TArray<int32> SelectedBones;
			bool bRestrictToSelection = false;
			if (IsConnected(&OptionalSelectionFilter))
			{
				const FDataflowTransformSelection& InOptionalSelectionFilter = GetValue<FDataflowTransformSelection>(Context, &OptionalSelectionFilter);
				bRestrictToSelection = true;
				SelectedBones = InOptionalSelectionFilter.AsArray();
				GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(InCollection);
				SelectionFacade.Sanitize(SelectedBones, /* bFavorParent */false);
			}

			float InSimplificationDistanceThreshold = GetValue(Context, &SimplificationDistanceThreshold);

			FGeometryCollectionConvexUtility::FLeafConvexHullSettings LeafSettings(InSimplificationDistanceThreshold, GenerateMethod);
			LeafSettings.IntersectFilters.OnlyIntersectIfComputedIsSmallerFactor = IntersectIfComputedIsSmallerByFactor;
			LeafSettings.IntersectFilters.MinExternalVolumeToIntersect = MinExternalVolumeToIntersect;
			FDataflowConvexDecompositionSettings InDecompSettings = GetValue(Context, &ConvexDecompositionSettings);
			LeafSettings.DecompositionSettings.MaxGeoToHullVolumeRatioToDecompose = InDecompSettings.MaxGeoToHullVolumeRatioToDecompose;
			LeafSettings.DecompositionSettings.MinGeoVolumeToDecompose = InDecompSettings.MinSizeToDecompose * InDecompSettings.MinSizeToDecompose * InDecompSettings.MinSizeToDecompose;
			LeafSettings.DecompositionSettings.ErrorTolerance = InDecompSettings.ErrorTolerance;
			LeafSettings.DecompositionSettings.MaxHullsPerGeometry = InDecompSettings.MaxHullsPerGeometry;
			LeafSettings.DecompositionSettings.MinThicknessTolerance = InDecompSettings.MinThicknessTolerance;
			LeafSettings.DecompositionSettings.NumAdditionalSplits = InDecompSettings.NumAdditionalSplits;
			LeafSettings.DecompositionSettings.bProtectNegativeSpace = InDecompSettings.bProtectNegativeSpace;
			LeafSettings.DecompositionSettings.bOnlyConnectedToHull = InDecompSettings.bOnlyConnectedToHull;
			LeafSettings.DecompositionSettings.NegativeSpaceMinRadius = InDecompSettings.NegativeSpaceMinRadius;
			LeafSettings.DecompositionSettings.NegativeSpaceTolerance = InDecompSettings.NegativeSpaceTolerance;

			const FDataflowSphereCovering& RequireCovering = GetValue(Context, &RequireNegativeSpaceCovering);
			LeafSettings.DecompositionSettings.CustomEmptySpace = RequireCovering.Spheres.Num() > 0 ? &RequireCovering.Spheres : nullptr;

			LeafSettings.bComputeIntersectionsBeforeHull = bComputeIntersectionsBeforeHull;
			TArray<FGeometryCollectionConvexUtility::FSphereCoveringInfo> SphereCoverings;
			FGeometryCollectionConvexUtility::GenerateLeafConvexHulls(*GeomCollection, bRestrictToSelection, SelectedBones, LeafSettings, &SphereCoverings);
			SetValue(Context, (const FManagedArrayCollection&)(*GeomCollection), &Collection);
			
			for (FGeometryCollectionConvexUtility::FSphereCoveringInfo& Info : SphereCoverings)
			{
				CombinedSphereCovering.Spheres.AppendTransformed(Info.SphereCovering, Info.Transform);
			}
			
			SetValue(Context, MoveTemp(CombinedSphereCovering), &SphereCovering);
		}
	}
}

#if WITH_EDITOR
bool FCreateLeafConvexHullsDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FCreateLeafConvexHullsDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	using namespace UE::Geometry;

	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		DebugDrawRenderSettings.SetDebugDrawSettings(DataflowRenderingInterface);

		if (const FDataflowOutput* Output = FindOutput(&Collection))
		{
			const FManagedArrayCollection& InCollection = Output->GetValue(Context, Collection);
			const FDataflowTransformSelection& InOptionalSelectionFilter = GetValue(Context, &OptionalSelectionFilter);
			UE::Dataflow::Convex::DebugDrawProc(DataflowRenderingInterface, InCollection, bRandomizeColor, ColorRandomSeed, InOptionalSelectionFilter);
		}

		if (SphereCoveringDebugDrawRenderSettings.bDisplaySphereCovering)
		{
			if (const FDataflowOutput* SphereCoveringOutput = FindOutput(&SphereCovering))
			{
				const FDataflowSphereCovering& OutSpheres = SphereCoveringOutput->GetValue(Context, SphereCovering);

				UE::Dataflow::Convex::SphereCoveringDebugDrawProc(DataflowRenderingInterface, OutSpheres, SphereCoveringDebugDrawRenderSettings);
			}
		}
	}
}
#endif

/* --------------------------------------------------------------------------------------------------------------------------- */

FSetExternalCollisionsFromPrimitiveDataflowNode::FSetExternalCollisionsFromPrimitiveDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&OptionalSelectionFilter);
	RegisterInputConnection(&Scale);
	RegisterOutputConnection(&Collection, &Collection);

	DebugDrawRenderSettings.RenderType = EDataflowDebugDrawRenderType::Wireframe;
	DebugDrawRenderSettings.Color = FLinearColor::Green;
	DebugDrawRenderSettings.LineWidthMultiplier = 2.0;
}

namespace UE::Dataflow::Private
{
	// Capsule fit derived from PCA over a vertex cloud.
	// LongAxis is a unit vector in bone-local space; HalfHeight is along LongAxis (cylinder
	// portion only, excluding caps); Radius is perpendicular.
	struct FCapsulePCAFit
	{
		FVector LongAxis = FVector::ZAxisVector;
		FVector Center = FVector::ZeroVector;
		float HalfHeight = 0.f;
		float Radius = 0.f;
	};

	// Returns false if PCA fails (degenerate vertex set); caller falls back to AABB fit.
	static bool ComputeCapsulePCAFit(TArrayView<const FVector3f> BoneVerts, FCapsulePCAFit& Out)
	{
		using namespace UE::Geometry;

		if (BoneVerts.Num() < 2)
		{
			return false;
		}

		FPCA3f PCA;
		if (!PCA.Compute(BoneVerts))
		{
			return false;
		}

		// Take the dominant eigenvector as a candidate axis; build an orthonormal frame off it.
		FVector3f ZCand = PCA.Eigenvectors[0];
		if (!ZCand.Normalize())
		{
			return false;
		}

		FVector3f XCand, YCand;
		ZCand.FindBestAxisVectors(XCand, YCand);

		// Project all verts onto each candidate axis to measure extent along the oriented frame.
		auto Interval = [BoneVerts](const FVector3f& A) -> float
		{
			float Min = std::numeric_limits<float>::max();
			float Max = std::numeric_limits<float>::lowest();
			for (const FVector3f& V : BoneVerts)
			{
				const float P = FVector3f::DotProduct(V, A);
				if (P < Min) Min = P;
				if (P > Max) Max = P;
			}
			return Max - Min;
		};

		const float ExtX = Interval(XCand);
		const float ExtY = Interval(YCand);
		const float ExtZ = Interval(ZCand);

		// Pick the longest axis as the capsule's long direction; Radius = half the diagonal of
		// the other two extents. Mirrors BoneGeometryGenerators.cpp:823-838 but keeps the
		// orientation as a single unit vector instead of a hand-built basis (which avoids the
		// chirality flip a Swap() would introduce).
		FVector3f LongAxis;
		float Length, ExtA, ExtB;
		if (ExtX >= ExtY && ExtX >= ExtZ)
		{
			LongAxis = XCand; Length = ExtX; ExtA = ExtY; ExtB = ExtZ;
		}
		else if (ExtY >= ExtZ)
		{
			LongAxis = YCand; Length = ExtY; ExtA = ExtX; ExtB = ExtZ;
		}
		else
		{
			LongAxis = ZCand; Length = ExtZ; ExtA = ExtX; ExtB = ExtY;
		}

		const float Radius = 0.5f * FMath::Sqrt(ExtA * ExtA + ExtB * ExtB);

		Out.LongAxis = (FVector)LongAxis;
		Out.Center = (FVector)PCA.Mean;
		Out.Radius = Radius;
		Out.HalfHeight = FMath::Max(0.5f * Length - Radius, 0.0f);
		return true;
	}

	// Oriented bounding box fit derived from PCA over a vertex cloud.
	// AxisX/Y/Z form a right-handed orthonormal frame in bone-local space;
	// HalfExtent is along each axis; Center is the OBB center in bone-local space.
	struct FBoxPCAFit
	{
		FVector AxisX;
		FVector AxisY;
		FVector AxisZ;
		FVector Center;
		FVector HalfExtent;
	};

	// Returns false if PCA fails (degenerate vertex set); caller falls back to AABB fit.
	static bool ComputeBoxPCAFit(TArrayView<const FVector3f> BoneVerts, FBoxPCAFit& Out)
	{
		using namespace UE::Geometry;

		if (BoneVerts.Num() < 2)
		{
			return false;
		}

		FPCA3f PCA;
		if (!PCA.Compute(BoneVerts))
		{
			return false;
		}

		// Take the two largest eigenvectors and derive Z via cross product to guarantee a
		// right-handed orthonormal frame (PCA returns orthogonal but not necessarily right-handed).
		FVector AxisX = (FVector)PCA.Eigenvectors[0];
		FVector AxisY = (FVector)PCA.Eigenvectors[1];
		if (!AxisX.Normalize() || !AxisY.Normalize())
		{
			return false;
		}
		FVector AxisZ = FVector::CrossProduct(AxisX, AxisY);
		if (!AxisZ.Normalize())
		{
			return false;
		}
		// Re-orthogonalize Y in case the two largest eigenvectors weren't quite orthonormal.
		AxisY = FVector::CrossProduct(AxisZ, AxisX);

		// Project all verts onto each axis to measure extent.
		auto Interval = [BoneVerts](const FVector& A, float& Min, float& Max)
		{
			Min = std::numeric_limits<float>::max();
			Max = std::numeric_limits<float>::lowest();
			for (const FVector3f& V : BoneVerts)
			{
				const float P = (float)FVector::DotProduct((FVector)V, A);
				if (P < Min) Min = P;
				if (P > Max) Max = P;
			}
		};

		float MinX, MaxX, MinY, MaxY, MinZ, MaxZ;
		Interval(AxisX, MinX, MaxX);
		Interval(AxisY, MinY, MaxY);
		Interval(AxisZ, MinZ, MaxZ);

		Out.AxisX = AxisX;
		Out.AxisY = AxisY;
		Out.AxisZ = AxisZ;
		Out.HalfExtent = FVector(0.5 * (MaxX - MinX), 0.5 * (MaxY - MinY), 0.5 * (MaxZ - MinZ));
		// OBB center reconstructed in bone-local space from per-axis midpoints.
		Out.Center =
			0.5 * (MinX + MaxX) * AxisX +
			0.5 * (MinY + MaxY) * AxisY +
			0.5 * (MinZ + MaxZ) * AxisZ;
		return true;
	}

	// Builds a primitive implicit that fits the given bone-local bounds (AABB) — and optionally,
	// for box/capsule with bAlignToPrincipalAxis, the principal axes of the vertex cloud.
	// Returns null if the bounds are empty/degenerate.
	static Chaos::FImplicitObjectPtr MakePrimitiveImplicit(
		EPrimitiveCollisionShapeDataflowEnum Shape,
		float Scale,
		bool bAlignToPrincipalAxis,
		const FBox& BoundsLocal,
		TArrayView<const FVector3f> BoneVerts)
	{
		using namespace Chaos;

		if (!BoundsLocal.IsValid)
		{
			return nullptr;
		}

		const FVector BoundsCenter = BoundsLocal.GetCenter();
		const FVector BoundsExtent = BoundsLocal.GetExtent() * Scale; // half-extent, padded by Scale

		FImplicitObjectPtr Result;

		switch (Shape)
		{
		case EPrimitiveCollisionShapeDataflowEnum::Sphere:
		{
			const FReal Radius = FMath::Max((FReal)BoundsExtent.Size(), (FReal)UE_KINDA_SMALL_NUMBER);
			Result = MakeImplicitObjectPtr<TSphere<FReal, 3>>(FVec3(BoundsCenter), Radius);
			break;
		}

		case EPrimitiveCollisionShapeDataflowEnum::Box:
		{
			FBoxPCAFit Fit;
			const bool bHavePCA = bAlignToPrincipalAxis && ComputeBoxPCAFit(BoneVerts, Fit);

			if (bHavePCA)
			{
				const FVector HalfExt = Fit.HalfExtent * Scale;
				FImplicitObjectPtr Inner = MakeImplicitObjectPtr<TBox<FReal, 3>>(
					FVec3(-HalfExt.X, -HalfExt.Y, -HalfExt.Z),
					FVec3(+HalfExt.X, +HalfExt.Y, +HalfExt.Z));
				const FQuat Rotation = FRotationMatrix::MakeFromXY(Fit.AxisX, Fit.AxisY).ToQuat();
				const FTransform Wrapper(Rotation, Fit.Center);
				Result = MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(
					MoveTemp(Inner), FRigidTransform3(Wrapper));
			}
			else
			{
				FImplicitObjectPtr Inner = MakeImplicitObjectPtr<TBox<FReal, 3>>(
					FVec3(-BoundsExtent.X, -BoundsExtent.Y, -BoundsExtent.Z),
					FVec3(+BoundsExtent.X, +BoundsExtent.Y, +BoundsExtent.Z));
				if (BoundsCenter.IsNearlyZero())
				{
					Result = MoveTemp(Inner);
				}
				else
				{
					Result = MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(
						MoveTemp(Inner), FRigidTransform3(FTransform(BoundsCenter)));
				}
			}
			break;
		}

		case EPrimitiveCollisionShapeDataflowEnum::Capsule:
		{
			FCapsulePCAFit Fit;
			const bool bHavePCA = bAlignToPrincipalAxis && ComputeCapsulePCAFit(BoneVerts, Fit);

			if (bHavePCA)
			{
				const FReal Radius = FMath::Max((FReal)(Fit.Radius * Scale), (FReal)UE_KINDA_SMALL_NUMBER);
				const FReal HalfHeight = FMath::Max((FReal)(Fit.HalfHeight * Scale), (FReal)0.0);

				FImplicitObjectPtr Inner = MakeImplicitObjectPtr<FCapsule>(
					FVec3(0, 0, -HalfHeight), FVec3(0, 0, +HalfHeight), Radius);

				// Rotate the capsule's native +Z to the PCA-derived long axis.
				const FQuat Rotation = FQuat::FindBetweenNormals(FVector::ZAxisVector, Fit.LongAxis);
				const FTransform Wrapper(Rotation, Fit.Center);
				Result = MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(
					MoveTemp(Inner), FRigidTransform3(Wrapper));
			}
			else
			{
				// Fallback: +Z-aligned AABB capsule.
				const FReal Radius = FMath::Max((FReal)FMath::Max(BoundsExtent.X, BoundsExtent.Y), (FReal)UE_KINDA_SMALL_NUMBER);
				const FReal HalfHeight = FMath::Max((FReal)BoundsExtent.Z - Radius, (FReal)0.0);
				FImplicitObjectPtr Inner = MakeImplicitObjectPtr<FCapsule>(
					FVec3(0, 0, -HalfHeight), FVec3(0, 0, +HalfHeight), Radius);

				if (BoundsCenter.IsNearlyZero())
				{
					Result = MoveTemp(Inner);
				}
				else
				{
					Result = MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(
						MoveTemp(Inner), FRigidTransform3(FTransform(BoundsCenter)));
				}
			}
			break;
		}
		}

		return Result;
	}
}

void FSetExternalCollisionsFromPrimitiveDataflowNode::GetBoneSelection(UE::Dataflow::FContext& Context, const FManagedArrayCollection& InCollection, TArray<int32>& OutSelection) const
{
	OutSelection.Reset();

	const GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(InCollection);

	const FDataflowTransformSelection& InSelection = GetValue(Context, &OptionalSelectionFilter);
	const bool bValidSelection = InSelection.IsValidForCollection(InCollection);
	if (bValidSelection)
	{
		OutSelection = InSelection.AsArray();
		SelectionFacade.Sanitize(OutSelection, /*bFavorParent*/ false);
	}
	if (OutSelection.IsEmpty())
	{
		OutSelection = SelectionFacade.SelectAll();
	}
}

void FSetExternalCollisionsFromPrimitiveDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (!Out->IsA(&Collection))
	{
		return;
	}

	FManagedArrayCollection InOutCollection = GetValue(Context, &Collection);
	if (InOutCollection.NumElements(FGeometryCollection::TransformGroup) == 0)
	{
		// not an error, silently forward the collection 
		SafeForwardInput(Context, &Collection, &Collection);
		return;
	}

	// Resolve selection: connected and non-empty after sanitize, OR fall back to all rigid leaves.
	TArray<int32> SelectedBones;
	GetBoneSelection(Context, InOutCollection, SelectedBones);
	if (SelectedBones.IsEmpty())
	{
		// not an error, silently forward the collection 
		SafeForwardInput(Context, &Collection, &Collection);
		return;
	}

	// Per-bone geometry bounds drive the shape size. Bones with no geometry (clusters) are skipped.
	const GeometryCollection::Facades::FBoundsFacade BoundsFacade(InOutCollection);
	if (!BoundsFacade.IsValid())
	{
		Context.Error(LOCTEXT("ExternalCollisionNode_MissingBoundingBoxAttribute", "Collection does not have a bounding box attribute, skipping generation of collision"));
		SafeForwardInput(Context, &Collection, &Collection);
		return;
	}

	// Per-bone vertex range — needed for PCA when bAlignToPrincipalAxis is on for capsules.
	const TManagedArray<int32>* VertexStarts = InOutCollection.FindAttribute<int32>(FGeometryCollection::VertexStartAttribute, FGeometryCollection::GeometryGroup);
	const TManagedArray<int32>* VertexCounts = InOutCollection.FindAttribute<int32>(FGeometryCollection::VertexCountAttribute, FGeometryCollection::GeometryGroup);
	const TManagedArray<FVector3f>* Vertices = InOutCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);

	if (!VertexStarts || !VertexCounts || !Vertices)
	{
		Context.Error(LOCTEXT("ExternalCollisionNode_MissingVertexAttributes", "Collection is missing vertex attributes, skipping generation of collision"));
		SafeForwardInput(Context, &Collection, &Collection);
		return;
	}

	TManagedArray<Chaos::FImplicitObjectPtr>& External = InOutCollection.AddAttribute<Chaos::FImplicitObjectPtr>(FGeometryCollection::ExternalCollisionsAttribute, FGeometryCollection::TransformGroup);

	const TManagedArray<int32>& TransformToGeometryIndices = BoundsFacade.GetTransformToGeometryIndex();
	const TManagedArray<FBox>& BoundingBoxes = BoundsFacade.GetBoundingBoxes();

	const float ScaleVal = GetValue(Context, &Scale);
	for (const int32 BoneIndex : SelectedBones)
	{
		const int32 GeoIdx = TransformToGeometryIndices.IsValidIndex(BoneIndex)? TransformToGeometryIndices[BoneIndex] : INDEX_NONE;
		if (GeoIdx == INDEX_NONE)
		{
			continue;
		}

		TArrayView<const FVector3f> BoneVerts;
		if (VertexCounts->IsValidIndex(GeoIdx) && VertexStarts->IsValidIndex(GeoIdx))
		{
			const int32 VertexStart = (*VertexStarts)[GeoIdx];
			const int32 VertexCount = (*VertexCounts)[GeoIdx];
			if (VertexCount > 0 && Vertices->IsValidIndex(VertexStart) && Vertices->IsValidIndex(VertexStart + VertexCount - 1))
			{
				BoneVerts = TArrayView<const FVector3f>(&(*Vertices)[VertexStart], VertexCount);
			}
		}

		if (BoundingBoxes.IsValidIndex(GeoIdx))
		{
			Chaos::FImplicitObjectPtr ShapeImplicit = UE::Dataflow::Private::MakePrimitiveImplicit(
				Shape, ScaleVal, bAlignToPrincipalAxis, BoundingBoxes[GeoIdx], BoneVerts);
			if (!ShapeImplicit)
			{
				continue;
			}

			// Wrap in a single-element FImplicitObjectUnion to match the layout produced by the
			// static-mesh importer (SetExternalCollisions in GeometryCollectionEngineConversion.cpp),
			// so downstream consumers see the same union-of-shapes shape they already handle.
			TArray<Chaos::FImplicitObjectPtr> ShapeArray;
			ShapeArray.Add(MoveTemp(ShapeImplicit));
			External[BoneIndex] = MakeImplicitObjectPtr<Chaos::FImplicitObjectUnion>(MoveTemp(ShapeArray));
		}
	}

	SetValue(Context, MoveTemp(InOutCollection), &Collection);
}

#if WITH_EDITOR
bool FSetExternalCollisionsFromPrimitiveDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FSetExternalCollisionsFromPrimitiveDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	if (!(DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		return;
	}

	const FDataflowOutput* Output = FindOutput(&Collection);
	if (!Output)
	{
		return;
	}

	const FManagedArrayCollection& OutCollection = Output->GetValue(Context, Collection);

	TArray<int32> SelectedBones;
	GetBoneSelection(Context, OutCollection, SelectedBones);
	if (SelectedBones.IsEmpty())
	{
		return;
	}

	const TManagedArray<FTransform3f>* BoneTransforms = OutCollection.FindAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FGeometryCollection::TransformGroup);
	const TManagedArray<int32>* Parents = OutCollection.FindAttribute<int32>(FTransformCollection::ParentAttribute, FGeometryCollection::TransformGroup);
	const TManagedArray<int32>* SimulationType = OutCollection.FindAttribute<int32>(FGeometryCollection::SimulationTypeAttribute, FGeometryCollection::TransformGroup);
	const TManagedArray<int32>* TransformToGeometryIndex = OutCollection.FindAttribute<int32>(FGeometryCollection::TransformToGeometryIndexAttribute, FGeometryCollection::TransformGroup);
	const TManagedArray<FBox>* BoundingBox = OutCollection.FindAttribute<FBox>(FGeometryCollection::BoundingBoxAttribute, FGeometryCollection::GeometryGroup);
	const TManagedArray<int32>* VertexStarts = OutCollection.FindAttribute<int32>(FGeometryCollection::VertexStartAttribute, FGeometryCollection::GeometryGroup);
	const TManagedArray<int32>* VertexCounts = OutCollection.FindAttribute<int32>(FGeometryCollection::VertexCountAttribute, FGeometryCollection::GeometryGroup);
	const TManagedArray<FVector3f>* Vertices = OutCollection.FindAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
	if (!BoneTransforms || !Parents || !TransformToGeometryIndex || !BoundingBox || !VertexStarts || !VertexCounts || !Vertices)
	{
		return;
	}

	DebugDrawRenderSettings.SetDebugDrawSettings(DataflowRenderingInterface);

	const float ScaleVal = GetValue(Context, &Scale);

	for (int32 BoneIndex : SelectedBones)
	{
		const int32 GeoIdx = TransformToGeometryIndex->IsValidIndex(BoneIndex)? (*TransformToGeometryIndex)[BoneIndex]: INDEX_NONE;
		if (GeoIdx == INDEX_NONE)
		{
			continue;
		}

		const FBox& BoundsLocal = BoundingBox->IsValidIndex(GeoIdx)? (*BoundingBox)[GeoIdx]: FBox();
		if (!BoundsLocal.IsValid)
		{
			continue;
		}

		const FVector BoundsCenter = BoundsLocal.GetCenter();
		const FVector BoundsExtent = BoundsLocal.GetExtent() * ScaleVal;
		const FTransform BoneWS = GeometryCollectionAlgo::GlobalMatrix(*BoneTransforms, *Parents, BoneIndex);

		// Per-bone vertex view, used by the box and capsule OBB/PCA fits below.
		TArrayView<const FVector3f> BoneVerts;
		if (VertexCounts->IsValidIndex(GeoIdx) && VertexStarts->IsValidIndex(GeoIdx))
		{
			const int32 VertexStart = (*VertexStarts)[GeoIdx];
			const int32 VertexCount = (*VertexCounts)[GeoIdx];
			if (VertexCount > 0 && Vertices->IsValidIndex(VertexStart) && Vertices->IsValidIndex(VertexStart + VertexCount - 1))
			{
				BoneVerts = TArrayView<const FVector3f>(&(*Vertices)[VertexStart], VertexCount);
			}
		}

		switch (Shape)
		{
		case EPrimitiveCollisionShapeDataflowEnum::Sphere:
		{
			const double Radius = FMath::Max(BoundsExtent.Size(), (double)UE_KINDA_SMALL_NUMBER);
			const FTransform ShapeWS = FTransform(BoundsCenter) * BoneWS;
			DataflowRenderingInterface.DrawSphere(ShapeWS.GetLocation(), Radius);
			break;
		}

		case EPrimitiveCollisionShapeDataflowEnum::Box:
		{
			UE::Dataflow::Private::FBoxPCAFit BoxFit;
			const bool bHavePCA = bAlignToPrincipalAxis && UE::Dataflow::Private::ComputeBoxPCAFit(BoneVerts, BoxFit);

			if (bHavePCA)
			{
				const FVector HalfExt = BoxFit.HalfExtent * ScaleVal;
				const FQuat LocalRot = FRotationMatrix::MakeFromXY(BoxFit.AxisX, BoxFit.AxisY).ToQuat();
				const FTransform ShapeWS = FTransform(LocalRot, BoxFit.Center) * BoneWS;
				DataflowRenderingInterface.DrawBox(HalfExt, ShapeWS.GetRotation(), ShapeWS.GetLocation(), /*UniformScale*/ 1.0);
			}
			else
			{
				const FTransform ShapeWS = FTransform(BoundsCenter) * BoneWS;
				DataflowRenderingInterface.DrawBox(BoundsExtent, ShapeWS.GetRotation(), ShapeWS.GetLocation(), /*UniformScale*/ 1.0);
			}
			break;
		}

		case EPrimitiveCollisionShapeDataflowEnum::Capsule:
		{
			UE::Dataflow::Private::FCapsulePCAFit Fit;
			const bool bHavePCA = bAlignToPrincipalAxis && UE::Dataflow::Private::ComputeCapsulePCAFit(BoneVerts, Fit);

			// Chaos::FCapsule uses HalfHeight = half the cylinder section (caps excluded).
			// FDebugRenderSceneProxy::FCapsule uses HalfHeight = half the total length (caps
			// included) and Base = bottom-most point. Convert between the two conventions.
			double Radius, ChaosHalfHeight;
			FTransform ShapeWS;
			if (bHavePCA)
			{
				Radius = FMath::Max((double)(Fit.Radius * ScaleVal), (double)UE_KINDA_SMALL_NUMBER);
				ChaosHalfHeight = FMath::Max((double)(Fit.HalfHeight * ScaleVal), 0.0);
				const FQuat LocalRot = FQuat::FindBetweenNormals(FVector::ZAxisVector, Fit.LongAxis);
				ShapeWS = FTransform(LocalRot, Fit.Center) * BoneWS;
			}
			else
			{
				Radius = FMath::Max((double)FMath::Max(BoundsExtent.X, BoundsExtent.Y), (double)UE_KINDA_SMALL_NUMBER);
				ChaosHalfHeight = FMath::Max(BoundsExtent.Z - Radius, 0.0);
				ShapeWS = FTransform(BoundsCenter) * BoneWS;
			}

			const double DebugHalfHeight = ChaosHalfHeight + Radius;
			const FQuat R = ShapeWS.GetRotation();
			const FVector ZAxisWS = R.GetAxisZ();
			const FVector Base = ShapeWS.GetLocation() - DebugHalfHeight * ZAxisWS;
			DataflowRenderingInterface.DrawCapsule(Base, Radius, DebugHalfHeight,
				R.GetAxisX(), R.GetAxisY(), ZAxisWS);
			break;
		}
		}
	}
}
#endif

/* --------------------------------------------------------------------------------------------------------------------------- */

FSimplifyConvexHullsDataflowNode::FSimplifyConvexHullsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&OptionalSelectionFilter);
	RegisterInputConnection(&SimplificationAngleThreshold).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SimplificationDistanceThreshold).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinTargetTriangleCount).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Collection);

	DebugDrawRenderSettings.RenderType = EDataflowDebugDrawRenderType::Wireframe;
	DebugDrawRenderSettings.Color = FLinearColor::Green;
	DebugDrawRenderSettings.LineWidthMultiplier = 2.0;
}

void FSimplifyConvexHullsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection) && IsConnected(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);
		if (InCollection.NumElements(FGeometryCollection::TransformGroup) == 0)
		{
			SetValue(Context, MoveTemp(InCollection), &Collection);
			return;
		}

		TArray<int32> SelectedBones;
		bool bRestrictToSelection = false;
		if (IsConnected(&OptionalSelectionFilter))
		{
			const FDataflowTransformSelection& InOptionalSelectionFilter = GetValue(Context, &OptionalSelectionFilter);
			bRestrictToSelection = true;
			SelectedBones = InOptionalSelectionFilter.AsArray();
		}

		UE::FractureEngine::Convex::FSimplifyHullSettings Settings;
		Settings.SimplifyMethod = SimplifyMethod;
		Settings.ErrorTolerance = GetValue(Context, &SimplificationDistanceThreshold);
		Settings.AngleThreshold = GetValue(Context, &SimplificationAngleThreshold);
		Settings.bUseGeometricTolerance = true;
		Settings.bUseTargetTriangleCount = true;
		Settings.bUseExistingVertexPositions = bUseExistingVertices;
		Settings.TargetTriangleCount = GetValue(Context, &MinTargetTriangleCount);
		UE::FractureEngine::Convex::SimplifyConvexHulls(InCollection, Settings, bRestrictToSelection, SelectedBones);
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

#if WITH_EDITOR
bool FSimplifyConvexHullsDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FSimplifyConvexHullsDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	using namespace UE::Geometry;

	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		DebugDrawRenderSettings.SetDebugDrawSettings(DataflowRenderingInterface);

		if (const FDataflowOutput* Output = FindOutput(&Collection))
		{
			const FManagedArrayCollection& OutCollection = Output->GetValue(Context, Collection);
			const FDataflowTransformSelection& InOptionalSelectionFilter = GetValue(Context, &OptionalSelectionFilter);

			UE::Dataflow::Convex::DebugDrawProc(DataflowRenderingInterface, OutCollection, bRandomizeColor, ColorRandomSeed, InOptionalSelectionFilter);
		}
	}
}
#endif

/* --------------------------------------------------------------------------------------------------------------------------- */

FCreateNonOverlappingConvexHullsDataflowNode::FCreateNonOverlappingConvexHullsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&CanRemoveFraction).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&SimplificationDistanceThreshold).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&CanExceedFraction).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&OverlapRemovalShrinkPercent).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Collection);

	DebugDrawRenderSettings.RenderType = EDataflowDebugDrawRenderType::Wireframe;
	DebugDrawRenderSettings.Color = FLinearColor::Green;
	DebugDrawRenderSettings.LineWidthMultiplier = 2.0;
}

void FCreateNonOverlappingConvexHullsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection) && IsConnected(&Collection))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
		{
			float InCanRemoveFraction = GetValue<float>(Context, &CanRemoveFraction);
			float InCanExceedFraction = GetValue<float>(Context, &CanExceedFraction);
			float InSimplificationDistanceThreshold = GetValue<float>(Context, &SimplificationDistanceThreshold);
			float InOverlapRemovalShrinkPercent = GetValue<float>(Context, &OverlapRemovalShrinkPercent);

			FGeometryCollectionConvexUtility::FGeometryCollectionConvexData ConvexData = FGeometryCollectionConvexUtility::CreateNonOverlappingConvexHullData(GeomCollection.Get(), 
				InCanRemoveFraction, 
				InSimplificationDistanceThreshold, 
				InCanExceedFraction,
				(EConvexOverlapRemoval)OverlapRemovalMethod,
				InOverlapRemovalShrinkPercent);

			SetValue<const FManagedArrayCollection&>(Context, *GeomCollection, &Collection);
		}
	}
}

#if WITH_EDITOR
bool FCreateNonOverlappingConvexHullsDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FCreateNonOverlappingConvexHullsDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	using namespace UE::Geometry;

	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		DebugDrawRenderSettings.SetDebugDrawSettings(DataflowRenderingInterface);

		if (const FDataflowOutput* Output = FindOutput(&Collection))
		{
			const FManagedArrayCollection& InCollection = Output->GetValue(Context, Collection);
			FDataflowTransformSelection EmptySelection;

			UE::Dataflow::Convex::DebugDrawProc(DataflowRenderingInterface, InCollection, bRandomizeColor, ColorRandomSeed, EmptySelection);
		}
	}
}
#endif

/* --------------------------------------------------------------------------------------------------------------------------- */

// local helper to convert the dataflow enum
static UE::Geometry::FNegativeSpaceSampleSettings::ESampleMethod ConvertNegativeSpaceSampleMethodDataflowEnum(ENegativeSpaceSampleMethodDataflowEnum SampleMethod)
{
	switch (SampleMethod)
	{
	case ENegativeSpaceSampleMethodDataflowEnum::Uniform:
		return UE::Geometry::FNegativeSpaceSampleSettings::ESampleMethod::Uniform;
	case ENegativeSpaceSampleMethodDataflowEnum::VoxelSearch:
		return UE::Geometry::FNegativeSpaceSampleSettings::ESampleMethod::VoxelSearch;
	case ENegativeSpaceSampleMethodDataflowEnum::NavigableVoxelSearch:
		return UE::Geometry::FNegativeSpaceSampleSettings::ESampleMethod::NavigableVoxelSearch;
	}
	return UE::Geometry::FNegativeSpaceSampleSettings::ESampleMethod::Uniform;
}

FGenerateClusterConvexHullsFromLeafHullsDataflowNode::FGenerateClusterConvexHullsFromLeafHullsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&ConvexCount).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ErrorTolerance).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&OptionalSelectionFilter);
	RegisterInputConnection(&bAllowMergingLeafHulls).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RequireNegativeSpaceCovering).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bProtectNegativeSpace).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&TargetNumSamples).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinSampleSpacing).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&NegativeSpaceTolerance).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinRadius).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Collection);
	RegisterOutputConnection(&SphereCovering);

	DebugDrawRenderSettings.RenderType = EDataflowDebugDrawRenderType::Wireframe;
	DebugDrawRenderSettings.Color = FLinearColor::Green;
	DebugDrawRenderSettings.LineWidthMultiplier = 2.0;
}

void FGenerateClusterConvexHullsFromLeafHullsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection) || Out->IsA(&SphereCovering))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		FDataflowSphereCovering Spheres;

		if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
		{
			TArray<int32> SelectionArray;
			bool bHasSelectionFilter = IsConnected(&OptionalSelectionFilter);
			if (bHasSelectionFilter)
			{
				const FDataflowTransformSelection& InOptionalSelectionFilter = GetValue<FDataflowTransformSelection>(Context, &OptionalSelectionFilter);
				SelectionArray = InOptionalSelectionFilter.AsArray();
				GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(InCollection);
				SelectionFacade.Sanitize(SelectionArray, /* bFavorParent */false);
			}

			bool bHasNegativeSpace = false;
			UE::Geometry::FSphereCovering NegativeSpace;
			if (GetValue(Context, &bProtectNegativeSpace))
			{
				UE::Geometry::FNegativeSpaceSampleSettings NegativeSpaceSettings;
				NegativeSpaceSettings.TargetNumSamples = GetValue(Context, &TargetNumSamples);
				NegativeSpaceSettings.MinRadius = GetValue(Context, &MinRadius);
				NegativeSpaceSettings.ReduceRadiusMargin = GetValue(Context, &NegativeSpaceTolerance);
				NegativeSpaceSettings.MinSpacing = GetValue(Context, &MinSampleSpacing);
				NegativeSpaceSettings.SampleMethod = ConvertNegativeSpaceSampleMethodDataflowEnum(SampleMethod);
				NegativeSpaceSettings.bRequireSearchSampleCoverage = bRequireSearchSampleCoverage;
				NegativeSpaceSettings.bOnlyConnectedToHull = bOnlyConnectedToHull;
				NegativeSpaceSettings.Sanitize();
				bHasNegativeSpace = UE::FractureEngine::Convex::ComputeConvexHullsNegativeSpace(*GeomCollection, NegativeSpace, NegativeSpaceSettings, bHasSelectionFilter, SelectionArray);
			}

			const FDataflowSphereCovering& RequireCovering = GetValue(Context, &RequireNegativeSpaceCovering);
			if (RequireCovering.Spheres.Num() > 0)
			{
				bHasNegativeSpace = true;
				NegativeSpace.Append(RequireCovering.Spheres);
			}

			const int32 InConvexCount = GetValue(Context, &ConvexCount);
			const double InErrorToleranceInCm = GetValue(Context, &ErrorTolerance);
			FGeometryCollectionConvexUtility::FClusterConvexHullSettings HullMergeSettings(InConvexCount, InErrorToleranceInCm, bPreferExternalCollisionShapes);
			HullMergeSettings.AllowMergesMethod = AllowMerges;
			HullMergeSettings.bAllowMergingLeafHulls = GetValue(Context, &bAllowMergingLeafHulls);
			HullMergeSettings.EmptySpace = bHasNegativeSpace ? &NegativeSpace : nullptr;
			HullMergeSettings.ProximityFilter = MergeProximityFilter;
			HullMergeSettings.ProximityDistanceThreshold = MergeProximityDistanceThreshold;

			if (bHasSelectionFilter)
			{
				FGeometryCollectionConvexUtility::GenerateClusterConvexHullsFromLeafHulls(
					*GeomCollection,
					HullMergeSettings,
					SelectionArray
				);
			}
			else
			{
				FGeometryCollectionConvexUtility::GenerateClusterConvexHullsFromLeafHulls(
					*GeomCollection,
					HullMergeSettings
				);
			}

			SetValue(Context, static_cast<const FManagedArrayCollection>(*GeomCollection), &Collection);
			// Move the negative space to the output container at the end to be sure it is no longer needed
			Spheres.Spheres = MoveTemp(NegativeSpace);
		}
		else
		{
			UE_LOGF(LogChaos, Error, "Error: Input collection could not be converted to a valid Geometry Collection");
			SetValue(Context, InCollection, &Collection);
		}

		SetValue(Context, MoveTemp(Spheres), &SphereCovering);
	}
}

#if WITH_EDITOR
bool FGenerateClusterConvexHullsFromLeafHullsDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FGenerateClusterConvexHullsFromLeafHullsDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	using namespace UE::Geometry;

	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		DebugDrawRenderSettings.SetDebugDrawSettings(DataflowRenderingInterface);

		if (const FDataflowOutput* Output = FindOutput(&Collection))
		{
			const FManagedArrayCollection& InCollection = Output->GetValue(Context, Collection);
			const FDataflowTransformSelection& InOptionalSelectionFilter = GetValue<FDataflowTransformSelection>(Context, &OptionalSelectionFilter);

			UE::Dataflow::Convex::DebugDrawProc(DataflowRenderingInterface, InCollection, bRandomizeColor, ColorRandomSeed, InOptionalSelectionFilter);
		}

		if (SphereCoveringDebugDrawRenderSettings.bDisplaySphereCovering)
		{
			if (const FDataflowOutput* SphereCoveringOutput = FindOutput(&SphereCovering))
			{
				const FDataflowSphereCovering& OutSpheres = SphereCoveringOutput->GetValue(Context, SphereCovering);

				UE::Dataflow::Convex::SphereCoveringDebugDrawProc(DataflowRenderingInterface, OutSpheres, SphereCoveringDebugDrawRenderSettings);
			}
		}
	}
}
#endif

/* --------------------------------------------------------------------------------------------------------------------------- */

FCreateNegativeSpaceSphereCovering::FCreateNegativeSpaceSphereCovering(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&OptionalSelectionFilter);
	RegisterInputConnection(&TargetNumSamples).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinSampleSpacing).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&NegativeSpaceTolerance).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinRadius).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&SphereCovering);
}

void FCreateNegativeSpaceSphereCovering::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&SphereCovering))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		FDataflowSphereCovering ResultSphereCovering;

		UE::Geometry::FNegativeSpaceSampleSettings NegativeSpaceSettings;
		NegativeSpaceSettings.TargetNumSamples = GetValue(Context, &TargetNumSamples);
		NegativeSpaceSettings.MinRadius = GetValue(Context, &MinRadius);
		NegativeSpaceSettings.ReduceRadiusMargin = GetValue(Context, &NegativeSpaceTolerance);
		NegativeSpaceSettings.MinSpacing = GetValue(Context, &MinSampleSpacing);
		NegativeSpaceSettings.SampleMethod = ConvertNegativeSpaceSampleMethodDataflowEnum(SampleMethod);
		NegativeSpaceSettings.bRequireSearchSampleCoverage = bRequireSearchSampleCoverage;
		NegativeSpaceSettings.bOnlyConnectedToHull = bOnlyConnectedToHull;
		NegativeSpaceSettings.Sanitize();

		FDataflowTransformSelection InTransformSelection = GetValue(Context, &OptionalSelectionFilter);

		// If optional selection input not connected select everything by default
		if (!IsConnected(&OptionalSelectionFilter))
		{
			InTransformSelection.InitializeFromCollection(InCollection, true);
		}
		if (InTransformSelection.AnySelected())
		{
			TArray<int32> TransformSelectionArray = InTransformSelection.AsArray();

			GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(InCollection);
			Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(InCollection);
			TArray<int32> LeafSelectionArray = TransformSelectionArray;
			SelectionFacade.ConvertSelectionToRigidNodes(LeafSelectionArray);

			UE::Geometry::FGeometryCollectionToDynamicMeshes CollectionToMeshes;
			UE::Geometry::FGeometryCollectionToDynamicMeshes::FToMeshOptions ToMeshOptions;
			ToMeshOptions.bWeldVertices = false;
			ToMeshOptions.bSaveIsolatedVertices = false;
			if (CollectionToMeshes.InitFromTransformSelection(InCollection, LeafSelectionArray, ToMeshOptions)
				&& !CollectionToMeshes.Meshes.IsEmpty())
			{
				UE::Geometry::FDynamicMesh3 CombinedMesh = MoveTemp(*CollectionToMeshes.Meshes[0].Mesh);
				for (int32 MeshIdx = 1; MeshIdx < CollectionToMeshes.Meshes.Num(); ++MeshIdx)
				{
					CombinedMesh.AppendWithOffsets(*CollectionToMeshes.Meshes[MeshIdx].Mesh);
				}

				UE::Geometry::FDynamicMeshAABBTree3 Tree(&CombinedMesh, true);
				UE::Geometry::TFastWindingTree<UE::Geometry::FDynamicMesh3> Winding(&Tree, true);
				constexpr bool bHasFlippedTriangles = false;
				ResultSphereCovering.Spheres.AddNegativeSpace(Winding, NegativeSpaceSettings, bHasFlippedTriangles);
			}
		}

		SetValue(Context, MoveTemp(ResultSphereCovering), &SphereCovering);
	}
}

#if WITH_EDITOR
bool FCreateNegativeSpaceSphereCovering::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FCreateNegativeSpaceSphereCovering::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	using namespace UE::Geometry;

	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		if (SphereCoveringDebugDrawRenderSettings.bDisplaySphereCovering)
		{
			if (const FDataflowOutput* SphereCoveringOutput = FindOutput(&SphereCovering))
			{
				const FDataflowSphereCovering& OutSpheres = SphereCoveringOutput->GetValue(Context, SphereCovering);

				UE::Dataflow::Convex::SphereCoveringDebugDrawProc(DataflowRenderingInterface, OutSpheres, SphereCoveringDebugDrawRenderSettings);
			}
		}
	}
}
#endif

/* --------------------------------------------------------------------------------------------------------------------------- */

FGenerateClusterConvexHullsFromChildrenHullsDataflowNode::FGenerateClusterConvexHullsFromChildrenHullsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&ConvexCount).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ErrorTolerance).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&OptionalSelectionFilter);
	RegisterInputConnection(&bAllowMergingLeafHulls).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RequireNegativeSpaceCovering).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bProtectNegativeSpace).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&TargetNumSamples).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinSampleSpacing).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&NegativeSpaceTolerance).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinRadius).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Collection);
	RegisterOutputConnection(&SphereCovering);

	DebugDrawRenderSettings.RenderType = EDataflowDebugDrawRenderType::Wireframe;
	DebugDrawRenderSettings.Color = FLinearColor::Green;
	DebugDrawRenderSettings.LineWidthMultiplier = 2.0;
}

void FGenerateClusterConvexHullsFromChildrenHullsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection) || Out->IsA(&SphereCovering))
	{
		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		FDataflowSphereCovering Spheres;

		if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
		{
			TArray<int32> SelectionArray;
			bool bHasSelectionFilter = IsConnected(&OptionalSelectionFilter);
			if (bHasSelectionFilter)
			{
				const FDataflowTransformSelection& InOptionalSelectionFilter = GetValue<FDataflowTransformSelection>(Context, &OptionalSelectionFilter);
				SelectionArray = InOptionalSelectionFilter.AsArray();
				GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(InCollection);
				SelectionFacade.Sanitize(SelectionArray, /* bFavorParent */false);
			}

			bool bHasNegativeSpace = false;
			UE::Geometry::FSphereCovering NegativeSpace;
			if (GetValue(Context, &bProtectNegativeSpace))
			{
				UE::Geometry::FNegativeSpaceSampleSettings NegativeSpaceSettings;
				NegativeSpaceSettings.TargetNumSamples = GetValue(Context, &TargetNumSamples);
				NegativeSpaceSettings.MinRadius = GetValue(Context, &MinRadius);
				NegativeSpaceSettings.ReduceRadiusMargin = GetValue(Context, &NegativeSpaceTolerance);
				NegativeSpaceSettings.MinSpacing = GetValue(Context, &MinSampleSpacing);
				NegativeSpaceSettings.SampleMethod = ConvertNegativeSpaceSampleMethodDataflowEnum(SampleMethod);
				NegativeSpaceSettings.bRequireSearchSampleCoverage = bRequireSearchSampleCoverage;
				NegativeSpaceSettings.bOnlyConnectedToHull = bOnlyConnectedToHull;
				NegativeSpaceSettings.Sanitize();
				bHasNegativeSpace = UE::FractureEngine::Convex::ComputeConvexHullsNegativeSpace(*GeomCollection, NegativeSpace, NegativeSpaceSettings, bHasSelectionFilter, SelectionArray);
			}

			const FDataflowSphereCovering& RequireCovering = GetValue(Context, &RequireNegativeSpaceCovering);
			if (RequireCovering.Spheres.Num() > 0)
			{
				bHasNegativeSpace = true;
				NegativeSpace.Append(RequireCovering.Spheres);
			}

			const int32 InConvexCount = GetValue(Context, &ConvexCount);
			const double InErrorToleranceInCm = GetValue(Context, &ErrorTolerance);
			FGeometryCollectionConvexUtility::FClusterConvexHullSettings HullMergeSettings(InConvexCount, InErrorToleranceInCm, bPreferExternalCollisionShapes);
			HullMergeSettings.AllowMergesMethod = EAllowConvexMergeMethod::Any; // Note: Only 'Any' is supported for this node currently
			HullMergeSettings.EmptySpace = bHasNegativeSpace ? &NegativeSpace : nullptr;
			HullMergeSettings.bAllowMergingLeafHulls = GetValue(Context, &bAllowMergingLeafHulls);
			HullMergeSettings.ProximityFilter = MergeProximityFilter;
			HullMergeSettings.ProximityDistanceThreshold = MergeProximityDistanceThreshold;

			if (bHasSelectionFilter)
			{
				FGeometryCollectionConvexUtility::GenerateClusterConvexHullsFromChildrenHulls(
					*GeomCollection,
					HullMergeSettings,
					SelectionArray
				);
			}
			else
			{
				FGeometryCollectionConvexUtility::GenerateClusterConvexHullsFromChildrenHulls(
					*GeomCollection,
					HullMergeSettings
				);
			}

			SetValue<const FManagedArrayCollection&>(Context, *GeomCollection, &Collection);
			// Move the negative space to the output container at the end to be sure it is no longer needed
			Spheres.Spheres = MoveTemp(NegativeSpace);
		}
		else
		{
			UE_LOGF(LogChaos, Error, "Error: Input collection could not be converted to a valid Geometry Collection");
			SetValue(Context, InCollection, &Collection);
		}
		
		SetValue(Context, MoveTemp(Spheres), &SphereCovering);
	}
}

#if WITH_EDITOR
bool FGenerateClusterConvexHullsFromChildrenHullsDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FGenerateClusterConvexHullsFromChildrenHullsDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	using namespace UE::Geometry;

	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		DebugDrawRenderSettings.SetDebugDrawSettings(DataflowRenderingInterface);

		if (const FDataflowOutput* Output = FindOutput(&Collection))
		{
			const FManagedArrayCollection& InCollection = Output->GetValue(Context, Collection);
			const FDataflowTransformSelection& InOptionalSelectionFilter = GetValue(Context, &OptionalSelectionFilter);

			UE::Dataflow::Convex::DebugDrawProc(DataflowRenderingInterface, InCollection, bRandomizeColor, ColorRandomSeed, InOptionalSelectionFilter);
		}

		if (SphereCoveringDebugDrawRenderSettings.bDisplaySphereCovering)
		{
			if (const FDataflowOutput* SphereCoveringOutput = FindOutput(&SphereCovering))
			{
				const FDataflowSphereCovering& OutSpheres = SphereCoveringOutput->GetValue(Context, SphereCovering);

				UE::Dataflow::Convex::SphereCoveringDebugDrawProc(DataflowRenderingInterface, OutSpheres, SphereCoveringDebugDrawRenderSettings);
			}
		}
	}
}
#endif

/* --------------------------------------------------------------------------------------------------------------------------- */

FMergeConvexHullsDataflowNode::FMergeConvexHullsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&MaxConvexCount).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ErrorTolerance).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&OptionalSelectionFilter);
	RegisterInputConnection(&bProtectNegativeSpace).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RequireNegativeSpaceCovering).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&TargetNumSamples).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinSampleSpacing).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&NegativeSpaceTolerance).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MinRadius).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&Collection);
	RegisterOutputConnection(&SphereCovering);

	DebugDrawRenderSettings.RenderType = EDataflowDebugDrawRenderType::Wireframe;
	DebugDrawRenderSettings.Color = FLinearColor::Green;
	DebugDrawRenderSettings.LineWidthMultiplier = 2.0;
}

#if WITH_EDITOR
bool FCopyConvexHullsFromRootDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FCopyConvexHullsFromRootDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	using namespace UE::Geometry;

	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		DebugDrawRenderSettings.SetDebugDrawSettings(DataflowRenderingInterface);

		if (const FDataflowOutput* Output = FindOutput(&Collection))
		{
			const FManagedArrayCollection& InCollection = Output->GetValue(Context, Collection);

			GeometryCollection::Facades::FCollectionTransformSelectionFacade TransformSelectionFacade(InCollection);
			const TArray<int32>& SelectionArr = TransformSelectionFacade.SelectRootBones();

			FDataflowTransformSelection RootSelection;
			RootSelection.InitializeFromCollection(InCollection, false);
			RootSelection.SetFromArray(SelectionArr);

			UE::Dataflow::Convex::DebugDrawProc(DataflowRenderingInterface, InCollection, bRandomizeColor, ColorRandomSeed, RootSelection);
		}
	}
}
#endif


FCopyConvexHullsFromRootDataflowNode::FCopyConvexHullsFromRootDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: Super(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&FromCollection);
	RegisterInputConnection(&bSkipIfEmpty).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Collection, &Collection);
	
	DebugDrawRenderSettings.RenderType = EDataflowDebugDrawRenderType::Wireframe;
	DebugDrawRenderSettings.Color = FLinearColor::Green;
	DebugDrawRenderSettings.LineWidthMultiplier = 2.0;
}

void FCopyConvexHullsFromRootDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);

		if (IsConnected(&Collection) && IsConnected(&FromCollection))
		{
			const FManagedArrayCollection& InFromCollection = GetValue(Context, &FromCollection);
			const bool bInSkipIfEmpty = GetValue(Context, &bSkipIfEmpty);

			if (FGeometryCollectionConvexUtility::HasConvexHullData(&InFromCollection))
			{
				GeometryCollection::Facades::FCollectionTransformSelectionFacade ToTransformSelectionFacade(InCollection);
				const TArray<int32> ToRoots = ToTransformSelectionFacade.SelectRootBones();
				GeometryCollection::Facades::FCollectionTransformSelectionFacade FromTransformSelectionFacade(InFromCollection);
				const TArray<int32> FromRoots = FromTransformSelectionFacade.SelectRootBones();
				if (ToRoots.Num() != FromRoots.Num())
				{
					UE_LOGF(LogChaosDataflow, Warning, "Failed to copy root collision across collections with different number of root nodes (%d vs %d)", ToRoots.Num(), FromRoots.Num());
				}
				else
				{
					FGeometryCollectionConvexUtility::CopyConvexHulls(InCollection, ToRoots, InFromCollection, FromRoots, bInSkipIfEmpty);
				}
			}
			else
			{
				if (!bInSkipIfEmpty)
				{
					GeometryCollection::Facades::FCollectionTransformSelectionFacade ToTransformSelectionFacade(InCollection);
					const TArray<int32> ToRoots = ToTransformSelectionFacade.SelectRootBones();
					FGeometryCollectionConvexUtility::RemoveConvexHulls(&InCollection, ToRoots);
				}
			}
		}
		
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

void FClearConvexHullsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);

		if (!IsConnected(&Collection) || !FGeometryCollectionConvexUtility::HasConvexHullData(&InCollection))
		{
			SetValue(Context, MoveTemp(InCollection), &Collection);
			return;
		}

		GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(InCollection);

		TArray<int32> ToClear;
		if (IsConnected(&TransformSelection))
		{ 
			const FDataflowTransformSelection& InTransformSelection = GetValue<FDataflowTransformSelection>(Context, &TransformSelection);
			ToClear = InTransformSelection.AsArray();

			SelectionFacade.Sanitize(ToClear, /* bFavorParent */false);
		}
		else
		{
			ToClear = SelectionFacade.SelectAll();
		}

		FGeometryCollectionConvexUtility::RemoveConvexHulls(&InCollection, ToClear);
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

void FMergeConvexHullsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection) || Out->IsA(&SphereCovering))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		FDataflowSphereCovering Spheres;

		TArray<int32> SelectionArray;
		bool bHasSelectionFilter = IsConnected(&OptionalSelectionFilter);
		if (bHasSelectionFilter)
		{
			const FDataflowTransformSelection& InOptionalSelectionFilter = GetValue<FDataflowTransformSelection>(Context, &OptionalSelectionFilter);
			SelectionArray = InOptionalSelectionFilter.AsArray();
			GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(InCollection);
			SelectionFacade.Sanitize(SelectionArray, /* bFavorParent */false);
		}

		bool bHasPrecomputedNegativeSpace = false;
		UE::Geometry::FSphereCovering NegativeSpace;
		bool bInProtectNegativeSpace = GetValue(Context, &bProtectNegativeSpace);
		UE::Geometry::FNegativeSpaceSampleSettings NegativeSpaceSettings;
		if (bInProtectNegativeSpace)
		{
			NegativeSpaceSettings.TargetNumSamples = GetValue(Context, &TargetNumSamples);
			NegativeSpaceSettings.MinRadius = GetValue(Context, &MinRadius);
			NegativeSpaceSettings.ReduceRadiusMargin = GetValue(Context, &NegativeSpaceTolerance);
			NegativeSpaceSettings.MinSpacing = GetValue(Context, &MinSampleSpacing);
			NegativeSpaceSettings.SampleMethod = ConvertNegativeSpaceSampleMethodDataflowEnum(SampleMethod);
			NegativeSpaceSettings.bRequireSearchSampleCoverage = bRequireSearchSampleCoverage;
			NegativeSpaceSettings.bOnlyConnectedToHull = bOnlyConnectedToHull;
			NegativeSpaceSettings.Sanitize();
		}
		if (bInProtectNegativeSpace && !bComputeNegativeSpacePerBone)
		{
			bHasPrecomputedNegativeSpace = UE::FractureEngine::Convex::ComputeConvexHullsNegativeSpace(InCollection, NegativeSpace, NegativeSpaceSettings, bHasSelectionFilter, SelectionArray, false);
		}
		const FDataflowSphereCovering& RequireCovering = GetValue(Context, &RequireNegativeSpaceCovering);
		if (RequireCovering.Spheres.Num() > 0)
		{
			bHasPrecomputedNegativeSpace = true;
			NegativeSpace.Append(RequireCovering.Spheres);
		}

		const int32 InMaxConvexCount = GetValue(Context, &MaxConvexCount);
		const double InErrorToleranceInCm = GetValue(Context, &ErrorTolerance);
		FGeometryCollectionConvexUtility::FMergeConvexHullSettings HullMergeSettings;
		HullMergeSettings.EmptySpace = bHasPrecomputedNegativeSpace ? &NegativeSpace : nullptr;
		HullMergeSettings.ErrorToleranceInCm = InErrorToleranceInCm;
		HullMergeSettings.MaxConvexCount = InMaxConvexCount;
		HullMergeSettings.ComputeEmptySpacePerBoneSettings = (bInProtectNegativeSpace && bComputeNegativeSpacePerBone) ? &NegativeSpaceSettings : nullptr;
		HullMergeSettings.ProximityFilter = MergeProximityFilter;
		HullMergeSettings.ProximityDistanceThreshold = MergeProximityDistanceThreshold;

		UE::Geometry::FSphereCovering UsedNegativeSpace;
		FGeometryCollectionConvexUtility::MergeHullsOnTransforms(InCollection, HullMergeSettings, bHasSelectionFilter, SelectionArray, &UsedNegativeSpace);

		SetValue(Context, MoveTemp(InCollection), &Collection);

		Spheres.Spheres = MoveTemp(UsedNegativeSpace);
		SetValue(Context, MoveTemp(Spheres), &SphereCovering);
	}
}

#if WITH_EDITOR
bool FMergeConvexHullsDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FMergeConvexHullsDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	using namespace UE::Geometry;

	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		DebugDrawRenderSettings.SetDebugDrawSettings(DataflowRenderingInterface);

		if (const FDataflowOutput* Output = FindOutput(&Collection))
		{
			const FManagedArrayCollection& InCollection = Output->GetValue(Context, Collection);
			const FDataflowTransformSelection& InOptionalSelectionFilter = GetValue<FDataflowTransformSelection>(Context, &OptionalSelectionFilter);

			UE::Dataflow::Convex::DebugDrawProc(DataflowRenderingInterface, InCollection, bRandomizeColor, ColorRandomSeed, InOptionalSelectionFilter);
		}

		if (SphereCoveringDebugDrawRenderSettings.bDisplaySphereCovering)
		{
			if (const FDataflowOutput* SphereCoveringOutput = FindOutput(&SphereCovering))
			{
				const FDataflowSphereCovering& OutSpheres = SphereCoveringOutput->GetValue(Context, SphereCovering);

				UE::Dataflow::Convex::SphereCoveringDebugDrawProc(DataflowRenderingInterface, OutSpheres, SphereCoveringDebugDrawRenderSettings);
			}
		}
	}
}
#endif

/* --------------------------------------------------------------------------------------------------------------------------- */

FUpdateVolumeAttributesDataflowNode::FUpdateVolumeAttributesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
}

void FUpdateVolumeAttributesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);
		if (InCollection.NumElements(FGeometryCollection::TransformGroup) > 0)
		{
			FGeometryCollectionConvexUtility::SetVolumeAttributes(&InCollection);
		}
		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

/* --------------------------------------------------------------------------------------------------------------------------- */

namespace ComputeVolumeStatsDataflowNode::Private
{
	static const FName VolumeAttributeName("Volume");
	static const FName TransformToConvexIndicesAttributeName("TransformToConvexIndices");

	static float ComputeGeometryVolume(const FManagedArrayCollection& InCollection, int32 TransformIndex)
	{
		float OutVolume = 0.f;
		if (const TManagedArray<float>* VolumeAttributePtr = InCollection.FindAttribute<float>(VolumeAttributeName, FTransformCollection::TransformGroup))
		{
			if (VolumeAttributePtr->IsValidIndex(TransformIndex))
			{
				OutVolume = (*VolumeAttributePtr)[TransformIndex];
			}
		}
		return OutVolume;
	}

	static float ComputeConvexVolume(FGeometryCollection& InCollection, int32 TransformIndex)
	{
		float ConvexVolume = 0;

		// we want a fully non optimized encompassing convex, we don't care about overlaps as well
		constexpr double FractionAllowRemove = 0.f;
		constexpr double SimplificationDistanceThreshold = 0.0f; // no simplification
		constexpr double CanExceedFraction = 1000000; // arbitrary large number since we don't want thegeneration to fail
		constexpr EConvexOverlapRemoval OverlapRemovalMethod = EConvexOverlapRemoval::None; // no overlap removal
		constexpr double OverlapRemovalShrinkPercent = 0.0f;

		FGeometryCollectionConvexUtility::FGeometryCollectionConvexData ConvexData =
			FGeometryCollectionConvexUtility::CreateNonOverlappingConvexHullData(
				&InCollection,
				FractionAllowRemove,
				SimplificationDistanceThreshold,
				CanExceedFraction,
				OverlapRemovalMethod,
				OverlapRemovalShrinkPercent
			);

		if (const TManagedArray<TSet<int32>>* TransformToConvexIndicesAttributePtr = InCollection.FindAttribute<TSet<int32>>(TransformToConvexIndicesAttributeName, FTransformCollection::TransformGroup))
		{
			if (TransformToConvexIndicesAttributePtr->IsValidIndex(TransformIndex))
			{
				// we should normally have only one convex but for safety let's sum up all the convex that may have been generated for this transform index
				ConvexVolume = 0;

				const TSet<int32>& ConvexIndices = (*TransformToConvexIndicesAttributePtr)[TransformIndex];
				for (const int32 ConvexIndex : ConvexIndices)
				{
					if (ConvexData.ConvexHull.IsValidIndex(ConvexIndex))
					{
						if (const Chaos::FConvexPtr ConvexPtr = ConvexData.ConvexHull[ConvexIndex])
						{
							ConvexVolume += ConvexPtr->GetVolume();
						}
					}
				}
			}
		}
		return ConvexVolume;
	}
}

FComputeVolumeStatsDataflowNode::FComputeVolumeStatsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&TransformIndex);

	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&GeometryVolume);
	RegisterOutputConnection(&ConvexVolume);
	RegisterOutputConnection(&VolumeRatio);
	RegisterOutputConnection(&VolumePercentage);
	RegisterOutputConnection(&EmptySpaceCubeSize);
}

void FComputeVolumeStatsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection) || 
		Out->IsA(&GeometryVolume) || 
		Out->IsA(&ConvexVolume) || 
		Out->IsA(&VolumeRatio) || 
		Out->IsA(&VolumePercentage) ||
		Out->IsA(&EmptySpaceCubeSize)
		)
	{
		float OutGeometryVolume = 0.f;
		float OutConvexVolume = 0.f;

		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		if (TUniquePtr<FGeometryCollection> GeomCollection = TUniquePtr<FGeometryCollection>(InCollection.NewCopy<FGeometryCollection>()))
		{
			const int32 NumTransforms = GeomCollection->NumElements(FGeometryCollection::TransformGroup);

			Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(*GeomCollection);

			const int32 RootIndex = HierarchyFacade.GetRootIndex();
			int32 InTransformIndex = GetValue(Context, &TransformIndex, TransformIndex);
			if (InTransformIndex < 0 || InTransformIndex >= NumTransforms)
			{
				InTransformIndex = RootIndex;
				const FText Message = FText::Format(LOCTEXT("ComputeVolumeStats_InvalidTransformIndexUsingRoot", "Invalid Transform Index, trying to use root index : {0}"), FText::AsNumber(InTransformIndex));
				Context.Warning(Message, this, Out);
			}
			if (InTransformIndex < 0 || InTransformIndex >= NumTransforms)
			{
				const FText Message = FText::Format(LOCTEXT("ComputeVolumeStats_InvalidTransformAndRootIndex", "Invalid Transform and root Index : {0}"), FText::AsNumber(InTransformIndex));
				Context.Error(Message, this, Out);
			}
			else
			{
				// make sure we have a volume attribute 
				FGeometryCollectionConvexUtility::SetVolumeAttributes(GeomCollection.Get());
				if (!GeomCollection->HasAttribute(ComputeVolumeStatsDataflowNode::Private::VolumeAttributeName, FGeometryCollection::TransformGroup))
				{
					const FText Message = LOCTEXT("ComputeVolumeStats_NoVolumeAttribute", "Could not generate volume attribute");
					Context.Error(Message, this, Out);
				}
				else
				{
					OutGeometryVolume = ComputeVolumeStatsDataflowNode::Private::ComputeGeometryVolume(*GeomCollection, InTransformIndex);
					OutConvexVolume = ComputeVolumeStatsDataflowNode::Private::ComputeConvexVolume(*GeomCollection, InTransformIndex);
				}
			}
		}
		else
		{
			const FText Message = LOCTEXT("ComputeVolumeStats_GeometryCollectionCopyFail", "Failed to make a Geometry collection out of the input collection");
			Context.Error(Message, this, Out);
		}

		const float OutVolumeRatio = OutGeometryVolume / FMath::Max(OutConvexVolume, UE_SMALL_NUMBER);
		const float OutVolumePercentage = OutVolumeRatio * 100.0f;
		const float OutEmptySpaceCubeSize = FMath::Pow(FMath::Max(0, (OutConvexVolume - OutGeometryVolume)), 1.f / 3.f);

		SafeForwardInput(Context, &Collection, &Collection);
		SetValue(Context, OutGeometryVolume, &GeometryVolume);
		SetValue(Context, OutConvexVolume, &ConvexVolume);
		SetValue(Context, OutVolumeRatio, &VolumeRatio);
		SetValue(Context, OutVolumePercentage, &VolumePercentage);
		SetValue(Context, OutEmptySpaceCubeSize, &EmptySpaceCubeSize);
	}
}



/* --------------------------------------------------------------------------------------------------------------------------- */

FGetConvexHullVolumeDataflowNode::FGetConvexHullVolumeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&TransformSelection);
	RegisterOutputConnection(&Volume);

	DebugDrawRenderSettings.RenderType = EDataflowDebugDrawRenderType::Wireframe;
	DebugDrawRenderSettings.Color = FLinearColor::Green;
	DebugDrawRenderSettings.LineWidthMultiplier = 2.0;
}

void FGetConvexHullVolumeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Volume))
	{
		float VolumeSum = 0;

		if (!IsConnected(&Collection) || !IsConnected(&TransformSelection))
		{
			SetValue(Context, VolumeSum, &Volume);
			return;
		}

		const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
		const FDataflowTransformSelection& InSelection = GetValue(Context, &TransformSelection);

		if (!FGeometryCollectionConvexUtility::HasConvexHullData(&InCollection))
		{
			SetValue(Context, VolumeSum, &Volume);
			return;
		}

		const int32 NumTransforms = InCollection.NumElements(FGeometryCollection::TransformGroup);
		TArray<int32> SelectionToSum = InSelection.AsArray();
		GeometryCollection::Facades::FCollectionTransformSelectionFacade SelectionFacade(InCollection);
		SelectionFacade.Sanitize(SelectionToSum);
		if (NumTransforms == 0 || SelectionToSum.Num() == 0)
		{
			SetValue(Context, VolumeSum, &Volume);
			return;
		}

		const TManagedArray<TSet<int32>>& TransformToConvexIndices = InCollection.GetAttribute<TSet<int32>>("TransformToConvexIndices", FTransformCollection::TransformGroup);
		const TManagedArray<Chaos::FConvexPtr>& ConvexHulls = InCollection.GetAttribute<Chaos::FConvexPtr>(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup);

		Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(InCollection);

		auto IterateHulls = [this, &TransformToConvexIndices, &HierarchyFacade](TArray<int32>& SelectionToSum, TFunctionRef<void(int32)> ProcessFn)
		{
			while (!SelectionToSum.IsEmpty())
			{
				int32 TransformIdx = SelectionToSum.Pop(EAllowShrinking::No);
				if (!bSumChildrenForClustersWithoutHulls || !TransformToConvexIndices[TransformIdx].IsEmpty())
				{
					ProcessFn(TransformIdx);
				}
				else if (const TSet<int32>* Children = HierarchyFacade.FindChildren(TransformIdx))
				{
					SelectionToSum.Append(Children->Array());
				}
			}
		};

		if (!bVolumeOfUnion)
		{
			IterateHulls(SelectionToSum, [&VolumeSum, &ConvexHulls, &TransformToConvexIndices](int32 TransformIdx)
				{
					for (int32 ConvexIdx : TransformToConvexIndices[TransformIdx])
					{
						VolumeSum += ConvexHulls[ConvexIdx]->GetVolume();
					}
				});
		}
		else
		{
			TArray<int32> SelectedBones;
			SelectedBones.Reserve(SelectionToSum.Num());
			IterateHulls(SelectionToSum, [&SelectedBones](int32 TransformIdx)
				{
					SelectedBones.Add(TransformIdx);
				});
			UE::Geometry::FDynamicMesh3 Mesh;
			UE::FractureEngine::Convex::GetConvexHullsAsDynamicMesh(InCollection, Mesh, true, SelectedBones);
			UE::Geometry::FMeshSelfUnion Union(&Mesh);
			// Disable quality-related features, since we just want the volume
			Union.TryToImproveTriQualityThreshold = -1;
			Union.bWeldSharedEdges = false;
			Union.Compute();
			VolumeSum = UE::Geometry::TMeshQueries<UE::Geometry::FDynamicMesh3>::GetVolumeNonWatertight(Mesh);
		}
		
		SetValue(Context, VolumeSum, &Volume);
	}
}

#if WITH_EDITOR
bool FGetConvexHullVolumeDataflowNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}

void FGetConvexHullVolumeDataflowNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	using namespace UE::Geometry;

	if ((DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned))
	{
		DebugDrawRenderSettings.SetDebugDrawSettings(DataflowRenderingInterface);

		if (IsConnected(&Collection))
		{
			const FManagedArrayCollection& InCollection = GetValue(Context, &Collection);
			const FDataflowTransformSelection& InSelection = GetValue(Context, &TransformSelection);

			UE::Dataflow::Convex::DebugDrawProc(DataflowRenderingInterface, InCollection, bRandomizeColor, ColorRandomSeed, InSelection);
		}
	}
}
#endif

/* --------------------------------------------------------------------------------------------------------------------------- */

void FFixTinyGeoDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FDataflowTransformSelection InTransformSelection = GetValue(Context, &TransformSelection);
		//
		// If not connected select everything by default
		//
		if (!IsConnected(&TransformSelection))
		{
			InTransformSelection.InitializeFromCollection(GetValue(Context, &Collection), true);
		}

		if (InTransformSelection.AnySelected())
		{
			FManagedArrayCollection InCollection = GetValue(Context, &Collection);

			FFractureEngineUtility::FixTinyGeo(InCollection,
				InTransformSelection,
				MergeType,
				bOnFractureLevel,
				SelectionMethod,
				MinVolumeCubeRoot,
				RelativeVolume,
				UseBoneSelection,
				bOnlyClusters,
				NeighborSelection,
				bOnlyToConnected,
				MergeType == EFixTinyGeoMergeType::MergeClusters ? bOnlySameParent : bGeometryOnlySameParent,
				bUseCollectionProximityForConnections);

			SetValue(Context, MoveTemp(InCollection), &Collection);

			return;
		}

		SafeForwardInput(Context, &Collection, &Collection);
	}
}

FSplitIslandsDataflowNode::FSplitIslandsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&TransformSelection);
	RegisterOutputConnection(&Collection, &Collection);
}

void FSplitIslandsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FDataflowTransformSelection InTransformSelection = GetValue(Context, &TransformSelection);
		// If not connected select everything by default
		if (!IsConnected(&TransformSelection))
		{
			InTransformSelection.InitializeFromCollection(GetValue(Context, &Collection), true);
		}

		if (InTransformSelection.AnySelected())
		{
			FManagedArrayCollection InCollection = GetValue(Context, &Collection);

			FFractureEngineUtility::SplitIslands(InCollection,
				InTransformSelection,
				CloseVertexDistance,
				VertexToSurfaceBridgeDistance);

			SetValue(Context, MoveTemp(InCollection), &Collection);

			return;
		}

		SafeForwardInput(Context, &Collection, &Collection);
	}
}

void FRecomputeNormalsInGeometryCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FDataflowTransformSelection InTransformSelection = GetValue(Context, &TransformSelection);
		//
		// If not connected select everything by default
		//
		if (!IsConnected(&TransformSelection))
		{
			InTransformSelection.InitializeFromCollection(GetValue(Context, &Collection), true);
		}

		if (InTransformSelection.AnySelected())
		{
			FManagedArrayCollection InCollection = GetValue(Context, &Collection);

			FFractureEngineUtility::RecomputeNormalsInGeometryCollection(InCollection,
				InTransformSelection,
				bOnlyTangents,
				bRecomputeSharpEdges,
				SharpEdgeAngleThreshold,
				bOnlyInternalSurfaces);

			SetValue(Context, MoveTemp(InCollection), &Collection);

			return;
		}

		SafeForwardInput(Context, &Collection, &Collection);
	}
}

void FResampleGeometryCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FDataflowTransformSelection InTransformSelection = GetValue(Context, &TransformSelection);
		//
		// If not connected select everything by default
		//
		if (!IsConnected(&TransformSelection))
		{
			InTransformSelection.InitializeFromCollection(GetValue(Context, &Collection), true);
		}

		if (InTransformSelection.AnySelected())
		{
			FManagedArrayCollection InCollection = GetValue(Context, &Collection);

			FFractureEngineUtility::ResampleGeometryCollection(InCollection,
				InTransformSelection,
				AddSamplesForCollision ? GetValue(Context, &CollisionSampleSpacing) : 0.f);

			SetValue(Context, MoveTemp(InCollection), &Collection);

			return;
		}

		SafeForwardInput(Context, &Collection, &Collection);
	}
}

void FValidateGeometryCollectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);

		FFractureEngineUtility::ValidateGeometryCollection(InCollection,
			bRemoveUnreferencedGeometry,
			bRemoveClustersOfOne,
			bRemoveDanglingClusters);

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE