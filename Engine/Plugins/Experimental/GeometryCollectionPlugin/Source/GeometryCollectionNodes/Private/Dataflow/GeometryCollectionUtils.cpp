// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionUtils.h"
#include "Dataflow/DataflowSettings.h"

#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "Internationalization/Regex.h"

namespace UE::Dataflow
{
	namespace Utils
	{
#if WITH_EDITOR
		void DebugDrawProximity(IDataflowDebugDrawInterface& DataflowRenderingInterface,
			const FManagedArrayCollection& Collection,
			const FLinearColor Color,
			const float LineWidthMultiplier,
			const float CenterSize,
			const FLinearColor CenterColor,
			const bool bRandomizeColor,
			const int32 ColorRandomSeed)
		{
			DataflowRenderingInterface.SetLineWidth(LineWidthMultiplier);
			DataflowRenderingInterface.SetWireframe(true);
			DataflowRenderingInterface.SetWorldPriority();
			DataflowRenderingInterface.SetPointSize(CenterSize);

			if (Collection.HasAttribute("Proximity", FGeometryCollection::GeometryGroup) &&
				Collection.HasAttribute("BoundingBox", FGeometryCollection::GeometryGroup))
			{
				const TManagedArray<TSet<int32>>& Proximity = Collection.GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);
				const TManagedArray<FBox>& BoundingBox = Collection.GetAttribute<FBox>("BoundingBox", FGeometryCollection::GeometryGroup);
				const TManagedArray<FTransform3f>* Transform = Collection.FindAttributeTyped<FTransform3f>(FGeometryCollection::TransformAttribute, FGeometryCollection::TransformGroup);
				const TManagedArray<int32>* Parent = Collection.FindAttributeTyped<int32>(FGeometryCollection::ParentAttribute, FGeometryCollection::TransformGroup);
				const TManagedArray<int32>* TransformIndex = Collection.FindAttributeTyped<int32>(FGeometryCollection::TransformIndexAttribute, FGeometryCollection::GeometryGroup);
				TArray<FTransform> GlobalTransformArray;
				if (Transform && Parent && TransformIndex)
				{
					GeometryCollectionAlgo::GlobalMatrices(*Transform, *Parent, GlobalTransformArray);
				}
				auto TransformFromGeometry = [&GlobalTransformArray, &TransformIndex](int32 GeometryIdx, FVector Point) -> FVector
				{
					if (GlobalTransformArray.IsEmpty() || !TransformIndex || !TransformIndex->IsValidIndex(GeometryIdx))
					{
						return Point;
					}
					else
					{
						return GlobalTransformArray[(*TransformIndex)[GeometryIdx]].TransformPosition(Point);
					}
				};

				const int32 NumGeometry = Collection.NumElements(FGeometryCollection::GeometryGroup);
				TArray<FVector> Centers;
				for (int32 Idx = 0; Idx < NumGeometry; ++Idx)
				{
					if (Proximity[Idx].Num() > 0)
					{
						const FVector Center = TransformFromGeometry(Idx, BoundingBox[Idx].GetCenter());

						Centers.Add(Center);

						if (bRandomizeColor)
						{
							DataflowRenderingInterface.SetColor(UE::Dataflow::Color::GetRandomColor(ColorRandomSeed + 17, Idx));
						}
						else
						{
							DataflowRenderingInterface.SetColor(Color);
						}

						for (int32 IdxOther : Proximity[Idx])
						{
							const FVector CenterOther = TransformFromGeometry(IdxOther, BoundingBox[IdxOther].GetCenter());

							DataflowRenderingInterface.DrawLine(Center, CenterOther);
						}

						DataflowRenderingInterface.SetPointSize(CenterSize);
						DataflowRenderingInterface.SetColor(CenterColor);
						for (FVector& Point : Centers)
						{
							DataflowRenderingInterface.DrawPoint(Point);
						}

						Centers.Reset();
					}
				}
			}
		}

		FLinearColor GetColorByLevel(const FManagedArrayCollection& InCollection, int32 InLevel)
		{
			const UDataflowSettings* DataflowSettings = GetDefault<UDataflowSettings>();
			const int32 NumTransformLevelColors = DataflowSettings->TransformLevelColors.LevelColors.Num();

			if (const TManagedArray<int32>* Levels = InCollection.FindAttribute<int32>(FTransformCollection::LevelAttribute, FGeometryCollection::TransformGroup))
			{
				const int32 NumTransforms = Levels->Num();

				if (InLevel >= 0)
				{
					return DataflowSettings->TransformLevelColors.LevelColors[InLevel % NumTransformLevelColors];
				}

				return DataflowSettings->TransformLevelColors.BlankColor;
			}
			else
			{
				return FLinearColor::White;
			}
		}
#endif

		/* e.g. "0, 2, 5-10, 12-15". If left empty, all will be used */
		bool ParseIndicesStr(const FString& InFramesString, TArray<int32>& OutIndices, uint32& OutErrorCode)
		{
			static const FRegexPattern AllowedCharsPattern(TEXT("^[-,0-9\\s]+$"));

			if (!FRegexMatcher(AllowedCharsPattern, InFramesString).FindNext())
			{
				// Input contains invalid characters
				OutErrorCode = Error_InvalidChars;
				return false;
			}

			static const FRegexPattern SingleNumberPattern(TEXT("^\\s*(\\d+)\\s*$"));
			static const FRegexPattern RangePattern(TEXT("^\\s*(\\d+)\\s*-\\s*(\\d+)\\s*$"));

			TArray<FString> Segments;
			InFramesString.ParseIntoArray(Segments, TEXT(","), true);
			for (const FString& Segment : Segments)
			{
				bool bSegmentValid = false;

				FRegexMatcher SingleNumberMatcher(SingleNumberPattern, Segment);
				if (SingleNumberMatcher.FindNext())
				{
					const int32 SingleNumber = FCString::Atoi(*SingleNumberMatcher.GetCaptureGroup(1));
					OutIndices.Add(SingleNumber);
					bSegmentValid = true;
				}
				else
				{
					FRegexMatcher RangeMatcher(RangePattern, Segment);
					if (RangeMatcher.FindNext())
					{
						const int32 RangeStart = FCString::Atoi(*RangeMatcher.GetCaptureGroup(1));
						const int32 RangeEnd = FCString::Atoi(*RangeMatcher.GetCaptureGroup(2));

						for (int32 i = RangeStart; i <= RangeEnd; ++i)
						{
							OutIndices.Add(i);
						}
						bSegmentValid = true;
					}
				}

				if (!bSegmentValid)
				{
					// Invalid format in segment
					OutErrorCode = Error_InvalidFormatInSegment;
					return false;
				}
			}

			return true;
		}
	}
}
