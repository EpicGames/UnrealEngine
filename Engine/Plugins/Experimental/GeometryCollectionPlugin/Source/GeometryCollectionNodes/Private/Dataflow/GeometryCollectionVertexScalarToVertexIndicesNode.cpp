// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionVertexScalarToVertexIndicesNode.h"

#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionVertexScalarToVertexIndicesNode)
#define LOCTEXT_NAMESPACE "FGeometryCollectionVertexScalarToVertexIndicesNode"

FGeometryCollectionVertexScalarToVertexIndicesNode::FGeometryCollectionVertexScalarToVertexIndicesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&AttributeKey);
	RegisterOutputConnection(&VertexIndices);
}

void FGeometryCollectionVertexScalarToVertexIndicesNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA< TArray<int32> >(&VertexIndices))
	{
		TArray<int32> IndicesOut;

		const FManagedArrayCollection& InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		FCollectionAttributeKey Key = GetValue(Context, &AttributeKey);

		if( const TManagedArray<float>* FloatArray = InCollection.FindAttribute<float>(FName(Key.Attribute), FName(Key.Group) ) )
		{
			for (int i = 0; i < FloatArray->Num(); i++)
			{
				if ((*FloatArray)[i] > SelectionThreshold)
				{
					IndicesOut.Add(i);
				}
			}
		}
		SetValue< TArray<int32> >(Context, MoveTemp(IndicesOut), &VertexIndices);
	}
}


#undef LOCTEXT_NAMESPACE
