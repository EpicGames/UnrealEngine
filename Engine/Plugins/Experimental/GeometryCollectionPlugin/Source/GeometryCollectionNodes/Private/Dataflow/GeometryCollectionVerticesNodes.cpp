// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/GeometryCollectionVerticesNodes.h"
#include "GeometryCollection/GeometryCollection.h"

#include "GeometryCollection/Facades/CollectionBoundsFacade.h"

#include "Dataflow/DataflowCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionVerticesNodes)

namespace UE::Dataflow
{
	void GeometryCollectionVerticesNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FTransformCollectionAttributeDataflowNode);
	}
}


void FTransformCollectionAttributeDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<DataType>(&Collection))
	{
		const FTransform& Transform = GetValue(Context, &TransformIn);
		FManagedArrayCollection CollectionValue = GetValue(Context, &Collection);
		FName AttribFName(*AttributeName);
		FName GroupFName(*GroupName);
		if (TManagedArray<FVector3f>* VecAttrib = CollectionValue.FindAttributeTyped<FVector3f>(AttribFName, GroupFName))
		{
			FMatrix44f Matrix(LocalTransform.ToMatrixWithScale() * Transform.ToMatrixWithScale());
			for (int32 i = 0; i < VecAttrib->Num(); i++)
			{
				(*VecAttrib)[i] = Matrix.TransformPosition((*VecAttrib)[i]);
			}

			// If we've moved vertices, also try to update bounding boxes
			if (AttribFName == FName("Vertex") && GroupFName == FGeometryCollection::VerticesGroup)
			{
				GeometryCollection::Facades::FBoundsFacade BoundsFacade(CollectionValue);
				if (BoundsFacade.IsValid())
				{
					BoundsFacade.UpdateBoundingBox();
				}
			}
		}
		else
		{
			UE_LOGF(LogChaosDataflow, Warning, "Could not find Vector3f Attribute \"%ls\" in Group \"%ls\"", *AttributeName, *GroupName);
		}
		SetValue(Context, MoveTemp(CollectionValue), &Collection);

	}
}

