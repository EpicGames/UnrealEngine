// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/MeshMedialSkeletonConversionNodes.h"

#include "DynamicMesh/DynamicBoneAttribute.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "Operations/MedialSkeletonSkinBinding.h"
#include "Operations/SkinWeightBinding.h"
#include "UDynamicMesh.h"

#include "SkeletalMeshAttributes.h" // for FSkeletalMeshAttributes::DefaultSkinWeightProfileName


#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshMedialSkeletonConversionNodes)

#define LOCTEXT_NAMESPACE "MeshMedialSkeletonConversionNodes"

namespace UE::Dataflow
{

	void RegisterMeshMedialSkeletonConversionNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertMedialSkeletonToAnimationSkeletonDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBindSkeletonToMeshDataflowNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSkinMeshViaMedialSkeleton_v2);

		// Deprecated
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FConvertMedialSkeletonToAnimationSkeletonDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FBindSkeletonToMeshDataflowNode);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSkinMeshViaMedialSkeleton);
	}

}

namespace MeshMedialSkeletonConversionLocals
{


	void SetSkeletonConversionOptions(UE::Geometry::MedialAxis::FMedialSkeletonToTreeSkeletonOptions& OutOptions,
		EDataflowMedialSkeletonConversionEdgeWeightMethod EdgeWeightMethod,
		EDataflowMedialSkeletonConversionMergeDisconnectedMethod MergeDisconnectedMethod,
		EDataflowMedialSkeletonConversionSelectRootMethod SelectRootMethod,
		const FVector& RootSelectionPoint,
		const FVector& RootSelectionDirection)
	{
		using namespace UE::Geometry::MedialAxis;
		switch (EdgeWeightMethod)
		{
		case EDataflowMedialSkeletonConversionEdgeWeightMethod::ArrayOrder:
			OutOptions.EdgeWeightMethod = FMedialSkeletonToTreeSkeletonOptions::EEdgeWeightMethod::ArrayOrder;
			break;
		case EDataflowMedialSkeletonConversionEdgeWeightMethod::AvgRadius:
			OutOptions.EdgeWeightMethod = FMedialSkeletonToTreeSkeletonOptions::EEdgeWeightMethod::AvgRadius;
			break;
		case EDataflowMedialSkeletonConversionEdgeWeightMethod::EdgeLength:
			OutOptions.EdgeWeightMethod = FMedialSkeletonToTreeSkeletonOptions::EEdgeWeightMethod::EdgeLength;
			break;
		}
		switch (MergeDisconnectedMethod)
		{
		case EDataflowMedialSkeletonConversionMergeDisconnectedMethod::AddTopLevelRoot:
			OutOptions.MergeDisconnectedMethod = FMedialSkeletonToTreeSkeletonOptions::EMergeDisconnectedMethod::AddTopLevelRoot;
			break;
		case EDataflowMedialSkeletonConversionMergeDisconnectedMethod::ConnectClosestBones:
			OutOptions.MergeDisconnectedMethod = FMedialSkeletonToTreeSkeletonOptions::EMergeDisconnectedMethod::ConnectClosestBones;
			break;
		}
		switch (SelectRootMethod)
		{
		case EDataflowMedialSkeletonConversionSelectRootMethod::ClosestToPoint:
			OutOptions.SelectRootMethod = FMedialSkeletonToTreeSkeletonOptions::ESelectRootMethod::ClosestToPoint;
			break;
		case EDataflowMedialSkeletonConversionSelectRootMethod::FarthestInDirection:
			OutOptions.SelectRootMethod = FMedialSkeletonToTreeSkeletonOptions::ESelectRootMethod::FarthestInDirection;
			break;
		case EDataflowMedialSkeletonConversionSelectRootMethod::ClosestToBoundsCenter:
			OutOptions.SelectRootMethod = FMedialSkeletonToTreeSkeletonOptions::ESelectRootMethod::ClosestToBoundsCenter;
			break;
		case EDataflowMedialSkeletonConversionSelectRootMethod::LargestSphere:
			OutOptions.SelectRootMethod = FMedialSkeletonToTreeSkeletonOptions::ESelectRootMethod::LargestSphere;
			break;
		case EDataflowMedialSkeletonConversionSelectRootMethod::ArrayOrder:
			OutOptions.SelectRootMethod = FMedialSkeletonToTreeSkeletonOptions::ESelectRootMethod::ArrayOrder;
			break;
		}
		OutOptions.RootSelectionPoint = FVector3d(RootSelectionPoint);
		OutOptions.RootSelectionDirection = FVector3d(RootSelectionDirection);
	}

	void SetSkinBindSettings(UE::Geometry::SkinBinding::FBindSettings& OutSettings, EDataflowBindSkeletonMethod BindMethod, float Stiffness, int32 MaxInfluences, int32 VoxelRes)
	{
		using namespace UE::Geometry;
		switch (BindMethod)
		{
		case EDataflowBindSkeletonMethod::DirectDistance:
			OutSettings.BindType = ESkinBindingType::DirectDistance;
			break;
		case EDataflowBindSkeletonMethod::GeodesicVoxel:
			OutSettings.BindType = ESkinBindingType::GeodesicVoxel;
			break;
		}
		OutSettings.MaxInfluences = MaxInfluences;
		OutSettings.Stiffness = Stiffness;
		OutSettings.VoxelResolution = VoxelRes;
	}

	TArray<UE::Geometry::SkinBinding::FBonePoseInfo> GetBindingBoneInfoFromReferenceSkeleton(const FReferenceSkeleton& InRefSkeleton)
	{
		// Only use non-virtual bones, since virtual bones cannot be bound to the skin.
		const TArray<FMeshBoneInfo>& BoneInfo = InRefSkeleton.GetRawRefBoneInfo();
		const TArray<FTransform>& BonePose = InRefSkeleton.GetRawRefBonePose();

		TArray<UE::Geometry::SkinBinding::FBonePoseInfo> Bones;
		Bones.Reserve(BoneInfo.Num());

		for (int32 Index = 0; Index < BoneInfo.Num(); Index++)
		{
			UE::Geometry::SkinBinding::FBonePoseInfo Info{ .LocalTransform = BonePose[Index], .Name = BoneInfo[Index].Name, .ParentIndex = BoneInfo[Index].ParentIndex };
			Bones.Add(Info);
		}
		return Bones;
	}

}

// ---------------------------------------------------------------------------------------------------------------------------------------
 
FConvertMedialSkeletonToAnimationSkeletonDataflowNode::FConvertMedialSkeletonToAnimationSkeletonDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MedialSkeleton);
	RegisterInputConnection(&AvoidIntersectingMesh);
	RegisterInputConnection(&RootSelectionPoint).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RootSelectionDirection).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RootClusterIndex).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bAddCustomAnimationRoot).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&CustomAnimationRootPosition).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Skeleton);
}

void FConvertMedialSkeletonToAnimationSkeletonDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Skeleton))
	{
		const FDataflowMedialSkeleton& InMedialSkeleton = GetValue(Context, &MedialSkeleton);
		FDataflowSkeleton OutSkeleton;

		TObjectPtr<const UDynamicMesh> InMesh = GetValue(Context, &AvoidIntersectingMesh);
		
		TMeshAABBTree3<FDynamicMesh3> MeshBVHLocal;
		const TMeshAABBTree3<FDynamicMesh3>* AvoidIntersectingMeshBVH = nullptr;
		if (InMesh && InMesh->GetMeshRef().TriangleCount() > 0)
		{
			MeshBVHLocal.SetMesh(InMesh->GetMeshPtr());
			AvoidIntersectingMeshBVH = &MeshBVHLocal;
		}

		TArray<int32> Parents;
		TArray<FTransform> Poses;

		UE::Geometry::MedialAxis::FMedialSkeletonToTreeSkeletonOptions Options;
		MeshMedialSkeletonConversionLocals::SetSkeletonConversionOptions(Options, EdgeWeightMethod, MergeDisconnectedMethod, SelectRootMethod,
			GetValue(Context, &RootSelectionPoint), GetValue(Context, &RootSelectionDirection));
		if (GetValue(Context, &bAddCustomAnimationRoot))
		{
			Options.CustomRootPosition = FVector3d(GetValue(Context, &CustomAnimationRootPosition));
		}
		const int32 UseRootIndex = GetValue(Context, &RootClusterIndex);
		Options.ToHierarchy(InMedialSkeleton.Skeleton, AvoidIntersectingMeshBVH, Parents, Poses, Options, UseRootIndex);

		const int32 NumBones = Parents.Num();
		if (NumBones > 0)
		{
			FReferenceSkeletonModifier Modifier = OutSkeleton.ModifySkeleton();
			for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
			{
				FName BoneName("Bone", BoneIdx);
				Modifier.Add(FMeshBoneInfo(BoneName, BoneName.ToString(), Parents[BoneIdx]), Poses[BoneIdx], true);
			}
		}

		SetValue(Context, MoveTemp(OutSkeleton), &Skeleton);
	}
}

// ---------------------------------------------------------------------------------------------------------------------------------------
 
FBindSkeletonToMeshDataflowNode::FBindSkeletonToMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&Skeleton);
	RegisterInputConnection(&Stiffness).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxInfluences).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&VoxelResolution).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Mesh, &Mesh);
}

void FBindSkeletonToMeshDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;
	using namespace MeshMedialSkeletonConversionLocals;

	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDynamicMesh> InMesh = GetValue(Context, &Mesh);
		const FDataflowSkeleton& InSkeleton = GetValue(Context, &Skeleton);
		TObjectPtr<UDynamicMesh> OutMesh = NewObject<UDynamicMesh>();

		if (InMesh)
		{
			// Create a new mesh copy and bind skeleton
			OutMesh->SetMesh(InMesh->GetMeshRef());
			FDynamicMesh3& DynMesh = OutMesh->GetMeshRef();

			SkinBinding::FBindSettings Settings;
			SetSkinBindSettings(Settings, BindMethod,
				GetValue(Context, &Stiffness),
				GetValue(Context, &MaxInfluences),
				GetValue(Context, &VoxelResolution));

			TArray<SkinBinding::FBonePoseInfo> Bones = GetBindingBoneInfoFromReferenceSkeleton(InSkeleton.GetRefSkeleton());
			if (Bones.IsEmpty())
			{
				Context.Warning(LOCTEXT("BindSkeletonEmptySkelError", "BindSkeletonToMesh: Attempted to bind empty skeleton to mesh; creating a single-bone skeleton instead."), this);
				Bones.Add(SkinBinding::FBonePoseInfo{ .LocalTransform = FTransform::Identity, .Name = FName("Root"), .ParentIndex = INDEX_NONE });
			}
			SkinBinding::CreateSkinWeights(DynMesh, Bones, FSkeletalMeshAttributes::DefaultSkinWeightProfileName, Settings);
		}
		else
		{
			Context.Error(LOCTEXT("BindSkeletonNullMeshError", "BindSkeletonToMesh: Cannot bind skeleton to null input mesh"), this);
		}
		SetValue(Context, OutMesh, &Mesh);
	}
}

// ---------------------------------------------------------------------------------------------------------------------------------------

FSkinMeshViaMedialSkeleton::FSkinMeshViaMedialSkeleton(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&MedialSkeleton);
	RegisterInputConnection(&RootSelectionPoint).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RootSelectionDirection).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RootClusterIndex).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bAddCustomAnimationRoot).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&CustomAnimationRootPosition).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Stiffness).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxInfluences).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&VoxelResolution).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ClusterNeighborSearchRange).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Mesh, &Mesh);
}

void FSkinMeshViaMedialSkeleton::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;
	using namespace MeshMedialSkeletonConversionLocals;

	if (Out->IsA(&Mesh))
	{
		TObjectPtr<UDynamicMesh> InMesh = GetValue(Context, &Mesh);
		const FDataflowMedialSkeleton& InMedialSkeleton = GetValue(Context, &MedialSkeleton);
		TObjectPtr<UDynamicMesh> OutMesh = NewObject<UDynamicMesh>();

		if (InMesh)
		{
			// Create a new mesh copy and add the weights
			OutMesh->SetMesh(InMesh->GetMeshRef());
			FDynamicMesh3& DynMesh = OutMesh->GetMeshRef();

			if (InMedialSkeleton.Skeleton.Spheres.IsEmpty())
			{
				Context.Error(LOCTEXT("SkinMeshViaMedialSkeletonEmptyError", "SkinMeshViaMedialSkeleton: Input medial skeleton is empty."), this);
			}
			else
			{
				SkinBinding::FBindMedialSkeletonSettings Settings;
				Settings.ClusterNeighborSearchRange = GetValue(Context, &ClusterNeighborSearchRange);
				SetSkeletonConversionOptions(Settings.AnimationSkeletonOptions, EdgeWeightMethod, MergeDisconnectedMethod, SelectRootMethod,
					GetValue(Context, &RootSelectionPoint), GetValue(Context, &RootSelectionDirection));
				if (GetValue(Context, &bAddCustomAnimationRoot))
				{
					Settings.AnimationSkeletonOptions.CustomRootPosition = FVector3d(GetValue(Context, &CustomAnimationRootPosition));
				}
				const int32 UseRootIndex = GetValue(Context, &RootClusterIndex);

				SetSkinBindSettings(Settings.BindSettings, BindMethod,
					GetValue(Context, &Stiffness),
					GetValue(Context, &MaxInfluences),
					GetValue(Context, &VoxelResolution));

				bool bMeshWasCompatible = false;
				bool bSuccess = SkinBinding::CreateSkinWeightsFromMedialSkeleton(
					InMedialSkeleton.Skeleton,
					DynMesh,
					bMeshWasCompatible,
					FSkeletalMeshAttributes::DefaultSkinWeightProfileName,
					Settings,
					UseRootIndex);

				if (!bSuccess)
				{
					Context.Error(
						LOCTEXT("SkinMeshViaMedialSkeletonFailed",
							"SkinMeshViaMedialSkeleton: Failed to bind a skeleton."),
						this);
				}
				else if (!bMeshWasCompatible)
				{
					Context.Warning(
						LOCTEXT("SkinMeshViaMedialSkeletonIncompat",
							"SkinMeshViaMedialSkeleton: Medial skeleton VIDtoClusterIndex is not compatible with the target mesh. Falling back to unconstrained skin binding."),
						this);
				}
			}
		}
		else
		{
			Context.Error(LOCTEXT("SkinMeshViaMedialSkeletonNullMeshError", "SkinMeshViaMedialSkeleton: Cannot bind skeleton to null input mesh"), this);
		}
		SetValue(Context, OutMesh, &Mesh);
	}
}

// ---------------------------------------------------------------------------------------------------------------------------------------

FConvertMedialSkeletonToAnimationSkeletonDataflowNode_v2::FConvertMedialSkeletonToAnimationSkeletonDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&MedialSkeleton);
	RegisterInputConnection(&AvoidIntersectingMesh);
	RegisterInputConnection(&RootSelectionPoint).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RootSelectionDirection).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RootClusterIndex).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bAddCustomAnimationRoot).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&CustomAnimationRootPosition).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Skeleton);
}

void FConvertMedialSkeletonToAnimationSkeletonDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Skeleton))
	{
		const FDataflowMedialSkeleton& InMedialSkeleton = GetValue(Context, &MedialSkeleton);
		FDataflowSkeleton OutSkeleton;

		const TObjectPtr<const UDataflowMesh> InMesh = GetValue(Context, &AvoidIntersectingMesh);

		TMeshAABBTree3<FDynamicMesh3> MeshBVHLocal;
		const TMeshAABBTree3<FDynamicMesh3>* AvoidIntersectingMeshBVH = nullptr;
		if (InMesh && InMesh->GetDynamicMeshRef().TriangleCount() > 0)
		{
			MeshBVHLocal.SetMesh(InMesh->GetDynamicMesh());
			AvoidIntersectingMeshBVH = &MeshBVHLocal;
		}

		TArray<int32> Parents;
		TArray<FTransform> Poses;

		UE::Geometry::MedialAxis::FMedialSkeletonToTreeSkeletonOptions Options;
		MeshMedialSkeletonConversionLocals::SetSkeletonConversionOptions(Options, EdgeWeightMethod, MergeDisconnectedMethod, SelectRootMethod,
				GetValue(Context, &RootSelectionPoint), GetValue(Context, &RootSelectionDirection));
		
		if (GetValue(Context, &bAddCustomAnimationRoot))
		{
			Options.CustomRootPosition = FVector3d(GetValue(Context, &CustomAnimationRootPosition));
		}
		const int32 UseRootIndex = GetValue(Context, &RootClusterIndex);
		Options.ToHierarchy(InMedialSkeleton.Skeleton, AvoidIntersectingMeshBVH, Parents, Poses, Options, UseRootIndex);

		const int32 NumBones = Parents.Num();
		if (NumBones > 0)
		{
			FReferenceSkeletonModifier Modifier = OutSkeleton.ModifySkeleton();
			for (int32 BoneIdx = 0; BoneIdx < NumBones; ++BoneIdx)
			{
				FName BoneName("Bone", BoneIdx);
				Modifier.Add(FMeshBoneInfo(BoneName, BoneName.ToString(), Parents[BoneIdx]), Poses[BoneIdx], true);
			}
		}

		SetValue(Context, MoveTemp(OutSkeleton), &Skeleton);		
	}
}

// ---------------------------------------------------------------------------------------------------------------------------------------

FBindSkeletonToMeshDataflowNode_v2::FBindSkeletonToMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&Skeleton);
	RegisterInputConnection(&Stiffness).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxInfluences).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&VoxelResolution).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Mesh, &Mesh);
}

void FBindSkeletonToMeshDataflowNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;
	using namespace MeshMedialSkeletonConversionLocals;

	if (Out->IsA(&Mesh))
	{
		const TObjectPtr<const UDataflowMesh> InMesh = GetValue(Context, &Mesh);
		const FDataflowSkeleton& InSkeleton = GetValue(Context, &Skeleton);

		if (InMesh)
		{
			FDynamicMesh3 DynMesh(InMesh->GetDynamicMeshRef());

			SkinBinding::FBindSettings Settings;
			SetSkinBindSettings(Settings, BindMethod,
				GetValue(Context, &Stiffness),
				GetValue(Context, &MaxInfluences),
				GetValue(Context, &VoxelResolution));

			TArray<SkinBinding::FBonePoseInfo> Bones = GetBindingBoneInfoFromReferenceSkeleton(InSkeleton.GetRefSkeleton());
			if (Bones.IsEmpty())
			{
				Context.Warning(LOCTEXT("BindSkeletonEmptySkelError", "BindSkeletonToMesh: Attempted to bind empty skeleton to mesh; creating a single-bone skeleton instead."), this);
				Bones.Add(SkinBinding::FBonePoseInfo{ .LocalTransform = FTransform::Identity, .Name = FName("Root"), .ParentIndex = INDEX_NONE });
			}
			SkinBinding::CreateSkinWeights(DynMesh, Bones, FSkeletalMeshAttributes::DefaultSkinWeightProfileName, Settings);

			if (TObjectPtr<UDataflowMesh> NewDataflowMesh = NewObject<UDataflowMesh>())
			{
				NewDataflowMesh->SetDynamicMesh(MoveTemp(DynMesh));
				NewDataflowMesh->SetMaterials(InMesh->GetMaterials());

				SetValue(Context, NewDataflowMesh, &Mesh);

				return;
			}
		}
		else
		{
			Context.Error(LOCTEXT("BindSkeletonNullMeshError", "BindSkeletonToMesh: Cannot bind skeleton to null input mesh"), this);
		}

		SetValue(Context, TObjectPtr<UDataflowMesh>(), &Mesh);
	}
}

// ---------------------------------------------------------------------------------------------------------------------------------------

FSkinMeshViaMedialSkeleton_v2::FSkinMeshViaMedialSkeleton_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&MedialSkeleton);
	RegisterInputConnection(&RootSelectionPoint).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RootSelectionDirection).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&RootClusterIndex).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&bAddCustomAnimationRoot).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&CustomAnimationRootPosition).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&Stiffness).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&MaxInfluences).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&VoxelResolution).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&ClusterNeighborSearchRange).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Mesh, &Mesh);
}

void FSkinMeshViaMedialSkeleton_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;
	using namespace MeshMedialSkeletonConversionLocals;

	if (Out->IsA(&Mesh))
	{
		const TObjectPtr<const UDataflowMesh> InMesh = GetValue(Context, &Mesh);
		const FDataflowMedialSkeleton& InMedialSkeleton = GetValue(Context, &MedialSkeleton);

		if (InMesh)
		{
			FDynamicMesh3 DynMesh(InMesh->GetDynamicMeshRef());

			if (InMedialSkeleton.Skeleton.Spheres.IsEmpty())
			{
				Context.Error(LOCTEXT("SkinMeshViaMedialSkeletonEmptyError", "SkinMeshViaMedialSkeleton: Input medial skeleton is empty."), this);
			}
			else
			{
				SkinBinding::FBindMedialSkeletonSettings Settings;
				Settings.ClusterNeighborSearchRange = GetValue(Context, &ClusterNeighborSearchRange);
				SetSkeletonConversionOptions(Settings.AnimationSkeletonOptions, EdgeWeightMethod, MergeDisconnectedMethod, SelectRootMethod,
					GetValue(Context, &RootSelectionPoint), GetValue(Context, &RootSelectionDirection));
				if (GetValue(Context, &bAddCustomAnimationRoot))
				{
					Settings.AnimationSkeletonOptions.CustomRootPosition = FVector3d(GetValue(Context, &CustomAnimationRootPosition));
				}
				const int32 UseRootIndex = GetValue(Context, &RootClusterIndex);

				SetSkinBindSettings(Settings.BindSettings, BindMethod,
					GetValue(Context, &Stiffness),
					GetValue(Context, &MaxInfluences),
					GetValue(Context, &VoxelResolution));

				bool bMeshWasCompatible = false;
				bool bSuccess = SkinBinding::CreateSkinWeightsFromMedialSkeleton(
					InMedialSkeleton.Skeleton,
					DynMesh,
					bMeshWasCompatible,
					FSkeletalMeshAttributes::DefaultSkinWeightProfileName,
					Settings,
					UseRootIndex);

				if (!bSuccess)
				{
					Context.Error(
						LOCTEXT("SkinMeshViaMedialSkeletonFailed",
							"SkinMeshViaMedialSkeleton: Failed to bind a skeleton."),
						this);
				}
				else if (!bMeshWasCompatible)
				{
					Context.Warning(
						LOCTEXT("SkinMeshViaMedialSkeletonIncompat",
							"SkinMeshViaMedialSkeleton: Medial skeleton VIDtoClusterIndex is not compatible with the target mesh. Falling back to unconstrained skin binding."),
						this);
				}

				if (TObjectPtr<UDataflowMesh> NewDataflowMesh = NewObject<UDataflowMesh>())
				{
					NewDataflowMesh->SetDynamicMesh(MoveTemp(DynMesh));
					NewDataflowMesh->SetMaterials(InMesh->GetMaterials());

					SetValue(Context, NewDataflowMesh, &Mesh);

					return;
				}
			}
		}
		else
		{
			Context.Error(LOCTEXT("SkinMeshViaMedialSkeletonNullMeshError", "SkinMeshViaMedialSkeleton: Cannot bind skeleton to null input mesh"), this);
		}

		SetValue(Context, TObjectPtr<UDataflowMesh>(), &Mesh);
	}
}

#undef LOCTEXT_NAMESPACE
