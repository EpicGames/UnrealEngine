// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowDebugDraw.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowSkeleton.h"
#include "Dataflow/GeometryCollectionSamplingNodes.h"
#include "GeometryCollection/ManagedArrayCollection.h"


#include "MeshMedialSkeletonConversionNodes.generated.h"

UENUM()
enum class EDataflowMedialSkeletonConversionEdgeWeightMethod : uint8
{
	// Favor adding shorter edges first. (Note: Typically not desirable for an animation skeleton.)
	EdgeLength,
	// Favor edges between clusters that are earlier in the medial skeleton array. Because medial spheres are added incrementally in the highest error locations, their order can approximate importance.
	ArrayOrder,
	// Favor adding edges between larger medial spheres first
	AvgRadius
};

UENUM()
enum class EDataflowMedialSkeletonConversionMergeDisconnectedMethod : uint8
{
	// For disconnected components, connect the closest pair of nodes in each disconnected component and the root component
	ConnectClosestBones,
	// Add a top-level root node, and connect the selected root and all disconnected components to this node
	AddTopLevelRoot
};

UENUM()
enum class EDataflowMedialSkeletonConversionSelectRootMethod : uint8
{
	// Select the cluster closest to the RootSelectionPoint
	ClosestToPoint,
	// Select the cluster farthest in the RootSelectionDirection
	FarthestInDirection,
	// Select the cluster closest to the bounding box center of the medial skeleton
	ClosestToBoundsCenter,
	// Select the cluster with the largest medial sphere radius
	LargestSphere,
	// Select the first cluster (index 0)
	ArrayOrder
};

UENUM()
enum class EDataflowBindSkeletonMethod : uint8
{
	// Computes the binding strength by computing the Euclidean distance to the closest set of bones,
	// where the strength of binding is proportional to the inverse distance. May cause bones to affects
	// parts of geometry that, although close in space, may be topologically distant.
	DirectDistance = 0,

	// Computes the binding by computing the geodesic distance from each set of bones. This is slower than the
	// direct distance.
	GeodesicVoxel = 1,
};

/*
* FConvertMedialSkeletonToAnimationSkeletonDataflowNode
* DEPRECATED 5.8 - use FConvertMedialSkeletonToAnimationSkeletonDataflowNode_v2 instead
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FConvertMedialSkeletonToAnimationSkeletonDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertMedialSkeletonToAnimationSkeletonDataflowNode, "MedialToAnimSkeleton", "SkeletonUtil", "Skeletal Conversion")

private:

	/** Medial skeleton of input mesh */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowMedialSkeleton MedialSkeleton;

	/** If provided, edges not intersecting this mesh will be preferred for inclusion in the skeleton */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<const UDynamicMesh> AvoidIntersectingMesh;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowSkeleton Skeleton;

	UPROPERTY(EditAnywhere, Category = "Options")
	EDataflowMedialSkeletonConversionEdgeWeightMethod EdgeWeightMethod = EDataflowMedialSkeletonConversionEdgeWeightMethod::ArrayOrder;

	UPROPERTY(EditAnywhere, Category = "Options")
	EDataflowMedialSkeletonConversionMergeDisconnectedMethod MergeDisconnectedMethod = EDataflowMedialSkeletonConversionMergeDisconnectedMethod::ConnectClosestBones;

	/** Method to automatically select the root cluster when RootClusterIndex is not specified */
	UPROPERTY(EditAnywhere, Category = "Options")
	EDataflowMedialSkeletonConversionSelectRootMethod SelectRootMethod = EDataflowMedialSkeletonConversionSelectRootMethod::ArrayOrder;

	/** When SelectRootMethod is ClosestToPoint, find the closest cluster to this point */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, EditCondition = "SelectRootMethod == EDataflowMedialSkeletonConversionSelectRootMethod::ClosestToPoint"))
	FVector RootSelectionPoint = FVector::ZeroVector;

	/** When SelectRootMethod is FarthestInDirection, the direction to search along */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, EditCondition = "SelectRootMethod == EDataflowMedialSkeletonConversionSelectRootMethod::FarthestInDirection"))
	FVector RootSelectionDirection = FVector(0, 0, -1);

	/** If non-negative, the index of the medial axis cluster (sphere) to use as the root of the Skeleton */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput))
	int32 RootClusterIndex = INDEX_NONE;

	/** If true, add an extra root bone at CustomAnimationRootPosition, unrelated to any medial skeleton cluster */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput))
	bool bAddCustomAnimationRoot = false;

	/** The world-space position for the custom root bone. Only used when bAddCustomAnimationRoot is true. */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, EditCondition = "bAddCustomAnimationRoot"))
	FVector CustomAnimationRootPosition = FVector::ZeroVector;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;


public:
	FConvertMedialSkeletonToAnimationSkeletonDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

USTRUCT(meta = (DataflowGeometryCollection))
struct FConvertMedialSkeletonToAnimationSkeletonDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvertMedialSkeletonToAnimationSkeletonDataflowNode_v2, "MedialToAnimSkeleton", "SkeletonUtil", "Skeletal Conversion")

private:

	/** Medial skeleton of input mesh */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowMedialSkeleton MedialSkeleton;

	/** If provided, edges not intersecting this mesh will be preferred for inclusion in the skeleton */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<const UDataflowMesh> AvoidIntersectingMesh;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowSkeleton Skeleton;

	UPROPERTY(EditAnywhere, Category = "Options")
	EDataflowMedialSkeletonConversionEdgeWeightMethod EdgeWeightMethod = EDataflowMedialSkeletonConversionEdgeWeightMethod::ArrayOrder;

	UPROPERTY(EditAnywhere, Category = "Options")
	EDataflowMedialSkeletonConversionMergeDisconnectedMethod MergeDisconnectedMethod = EDataflowMedialSkeletonConversionMergeDisconnectedMethod::ConnectClosestBones;

	/** Method to automatically select the root cluster when RootClusterIndex is not specified */
	UPROPERTY(EditAnywhere, Category = "Options")
	EDataflowMedialSkeletonConversionSelectRootMethod SelectRootMethod = EDataflowMedialSkeletonConversionSelectRootMethod::ArrayOrder;

	/** When SelectRootMethod is ClosestToPoint, find the closest cluster to this point */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, EditCondition = "SelectRootMethod == EDataflowMedialSkeletonConversionSelectRootMethod::ClosestToPoint"))
	FVector RootSelectionPoint = FVector::ZeroVector;

	/** When SelectRootMethod is FarthestInDirection, the direction to search along */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, EditCondition = "SelectRootMethod == EDataflowMedialSkeletonConversionSelectRootMethod::FarthestInDirection"))
	FVector RootSelectionDirection = FVector(0, 0, -1);

	/** If non-negative, the index of the medial axis cluster (sphere) to use as the root of the Skeleton */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput))
	int32 RootClusterIndex = INDEX_NONE;

	/** If true, add an extra root bone at CustomAnimationRootPosition, unrelated to any medial skeleton cluster */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput))
	bool bAddCustomAnimationRoot = false;

	/** The world-space position for the custom root bone. Only used when bAddCustomAnimationRoot is true. */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, EditCondition = "bAddCustomAnimationRoot"))
	FVector CustomAnimationRootPosition = FVector::ZeroVector;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;


public:
	FConvertMedialSkeletonToAnimationSkeletonDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/*
* FBindSkeletonToMeshDataflowNode
* DEPRECATED 5.8 - use FBindSkeletonToMeshDataflowNode_v2 instead
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FBindSkeletonToMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBindSkeletonToMeshDataflowNode, "BindSkeletonToMesh", "SkeletonUtil", "Skeletal Conversion")

private:

	/** Mesh to add an animation skeleton to */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowIntrinsic, DataflowPassthrough = "Mesh"))
	TObjectPtr<UDynamicMesh> Mesh;

	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowSkeleton Skeleton;

	/** Binding method to use */
	UPROPERTY(EditAnywhere, Category = "Options")
	EDataflowBindSkeletonMethod BindMethod = EDataflowBindSkeletonMethod::DirectDistance;

	/** Stiffness of binding. Lower values allow more distant bones to contribute more */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, ClampMin=0, ClampMax=1))
	float Stiffness = 0.2f;

	/** Maximum bones that will influence each vertex */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, ClampMin = 1, UIMax = 10))
	int32 MaxInfluences = 5;

	/** The resolution of the voxel grid if doing geodesic voxel binding */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, ClampMin =1, UIMin = 8, ClampMax = 1024))
	int32 VoxelResolution = 256;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;


public:
	FBindSkeletonToMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

USTRUCT(meta = (DataflowGeometryCollection))
struct FBindSkeletonToMeshDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBindSkeletonToMeshDataflowNode_v2, "BindSkeletonToMesh", "SkeletonUtil", "Skeletal Conversion")

private:

	/** Mesh to add an animation skeleton to */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowIntrinsic, DataflowPassthrough = "Mesh"))
	TObjectPtr<UDataflowMesh> Mesh;

	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowSkeleton Skeleton;

	/** Binding method to use */
	UPROPERTY(EditAnywhere, Category = "Options")
	EDataflowBindSkeletonMethod BindMethod = EDataflowBindSkeletonMethod::DirectDistance;

	/** Stiffness of binding. Lower values allow more distant bones to contribute more */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, ClampMin = 0, ClampMax = 1))
	float Stiffness = 0.2f;

	/** Maximum bones that will influence each vertex */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, ClampMin = 1, UIMax = 10))
	int32 MaxInfluences = 5;

	/** The resolution of the voxel grid if doing geodesic voxel binding */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, ClampMin = 1, UIMin = 8, ClampMax = 1024))
	int32 VoxelResolution = 256;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;


public:
	FBindSkeletonToMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/*
* FSkinMeshViaMedialSkeleton
* DEPRECATED 5.8 - use FSkinMeshViaMedialSkeleton_v2 instead
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FSkinMeshViaMedialSkeleton : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSkinMeshViaMedialSkeleton, "SkinMeshViaMedialSkeleton", "SkeletonUtil", "Skeletal Mesh Conversion")

private:
	/** Mesh to add an animation skeleton to */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowIntrinsic, DataflowPassthrough = "Mesh"))
	TObjectPtr<UDynamicMesh> Mesh;

	/** Medial skeleton of input mesh */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowMedialSkeleton MedialSkeleton;

	UPROPERTY(EditAnywhere, Category = "Options")
	EDataflowMedialSkeletonConversionEdgeWeightMethod EdgeWeightMethod = EDataflowMedialSkeletonConversionEdgeWeightMethod::ArrayOrder;

	UPROPERTY(EditAnywhere, Category = "Options")
	EDataflowMedialSkeletonConversionMergeDisconnectedMethod MergeDisconnectedMethod = EDataflowMedialSkeletonConversionMergeDisconnectedMethod::ConnectClosestBones;

	/** Method to automatically select the root cluster when RootClusterIndex is not specified */
	UPROPERTY(EditAnywhere, Category = "Options")
	EDataflowMedialSkeletonConversionSelectRootMethod SelectRootMethod = EDataflowMedialSkeletonConversionSelectRootMethod::ArrayOrder;

	/** When SelectRootMethod is ClosestToPoint, find the closest cluster to this point */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, EditCondition = "SelectRootMethod == EDataflowMedialSkeletonConversionSelectRootMethod::ClosestToPoint"))
	FVector RootSelectionPoint = FVector::ZeroVector;

	/** When SelectRootMethod is FarthestInDirection, the direction to search along */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, EditCondition = "SelectRootMethod == EDataflowMedialSkeletonConversionSelectRootMethod::FarthestInDirection"))
	FVector RootSelectionDirection = FVector(0, 0, -1);

	/** If non-negative, the index of the medial axis cluster (sphere) to use as the root of the Skeleton */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput))
	int32 RootClusterIndex = INDEX_NONE;

	/** If true, add an extra root bone at CustomAnimationRootPosition. In constrained binding, this bone will not receive skin weights from clustered vertices. */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput))
	bool bAddCustomAnimationRoot = false;

	/** The world-space position for the custom root bone. Only used when bAddCustomAnimationRoot is true. */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, EditCondition = "bAddCustomAnimationRoot"))
	FVector CustomAnimationRootPosition = FVector::ZeroVector;

	/** Binding method to use for skin weights */
	UPROPERTY(EditAnywhere, Category = "Binding")
	EDataflowBindSkeletonMethod BindMethod = EDataflowBindSkeletonMethod::DirectDistance;

	/** Stiffness of binding. Lower values allow more distant bones to contribute more */
	UPROPERTY(EditAnywhere, Category = "Binding", meta = (DataflowInput, ClampMin=0, ClampMax=1))
	float Stiffness = 0.2f;

	/** Maximum bones that will influence each vertex */
	UPROPERTY(EditAnywhere, Category = "Binding", meta = (DataflowInput, ClampMin = 1, UIMax = 10))
	int32 MaxInfluences = 5;

	/** The resolution of the voxel grid if doing geodesic voxel binding */
	UPROPERTY(EditAnywhere, Category = "Binding", meta = (DataflowInput, ClampMin = 1, UIMin = 8, ClampMax = 1024))
	int32 VoxelResolution = 256;

	/** 
	 * How far to search through medial skeleton neighbors when building bone groups for constrained skin binding.
	 * 1 = immediate neighbors only, 2 = neighbors of neighbors, etc.
	 * Higher values allow smoother blending across more bones but reduce locality. 
	 */
	UPROPERTY(EditAnywhere, Category = "Binding", meta = (DataflowInput, ClampMin = 1, UIMax = 5, ClampMax = 100))
	int32 ClusterNeighborSearchRange = 1;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;


public:
	FSkinMeshViaMedialSkeleton(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

USTRUCT(meta = (DataflowGeometryCollection))
struct FSkinMeshViaMedialSkeleton_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSkinMeshViaMedialSkeleton_v2, "SkinMeshViaMedialSkeleton", "SkeletonUtil", "Skeletal Mesh Conversion")

private:
	/** Mesh to add an animation skeleton to */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowIntrinsic, DataflowPassthrough = "Mesh"))
	TObjectPtr<UDataflowMesh> Mesh;

	/** Medial skeleton of input mesh */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowMedialSkeleton MedialSkeleton;

	UPROPERTY(EditAnywhere, Category = "Options")
	EDataflowMedialSkeletonConversionEdgeWeightMethod EdgeWeightMethod = EDataflowMedialSkeletonConversionEdgeWeightMethod::ArrayOrder;

	UPROPERTY(EditAnywhere, Category = "Options")
	EDataflowMedialSkeletonConversionMergeDisconnectedMethod MergeDisconnectedMethod = EDataflowMedialSkeletonConversionMergeDisconnectedMethod::ConnectClosestBones;

	/** Method to automatically select the root cluster when RootClusterIndex is not specified */
	UPROPERTY(EditAnywhere, Category = "Options")
	EDataflowMedialSkeletonConversionSelectRootMethod SelectRootMethod = EDataflowMedialSkeletonConversionSelectRootMethod::ArrayOrder;

	/** When SelectRootMethod is ClosestToPoint, find the closest cluster to this point */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, EditCondition = "SelectRootMethod == EDataflowMedialSkeletonConversionSelectRootMethod::ClosestToPoint"))
	FVector RootSelectionPoint = FVector::ZeroVector;

	/** When SelectRootMethod is FarthestInDirection, the direction to search along */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, EditCondition = "SelectRootMethod == EDataflowMedialSkeletonConversionSelectRootMethod::FarthestInDirection"))
	FVector RootSelectionDirection = FVector(0, 0, -1);

	/** If non-negative, the index of the medial axis cluster (sphere) to use as the root of the Skeleton */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput))
	int32 RootClusterIndex = INDEX_NONE;

	/** If true, add an extra root bone at CustomAnimationRootPosition. In constrained binding, this bone will not receive skin weights from clustered vertices. */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput))
	bool bAddCustomAnimationRoot = false;

	/** The world-space position for the custom root bone. Only used when bAddCustomAnimationRoot is true. */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, EditCondition = "bAddCustomAnimationRoot"))
	FVector CustomAnimationRootPosition = FVector::ZeroVector;

	/** Binding method to use for skin weights */
	UPROPERTY(EditAnywhere, Category = "Binding")
	EDataflowBindSkeletonMethod BindMethod = EDataflowBindSkeletonMethod::DirectDistance;

	/** Stiffness of binding. Lower values allow more distant bones to contribute more */
	UPROPERTY(EditAnywhere, Category = "Binding", meta = (DataflowInput, ClampMin = 0, ClampMax = 1))
	float Stiffness = 0.2f;

	/** Maximum bones that will influence each vertex */
	UPROPERTY(EditAnywhere, Category = "Binding", meta = (DataflowInput, ClampMin = 1, UIMax = 10))
	int32 MaxInfluences = 5;

	/** The resolution of the voxel grid if doing geodesic voxel binding */
	UPROPERTY(EditAnywhere, Category = "Binding", meta = (DataflowInput, ClampMin = 1, UIMin = 8, ClampMax = 1024))
	int32 VoxelResolution = 256;

	/**
	 * How far to search through medial skeleton neighbors when building bone groups for constrained skin binding.
	 * 1 = immediate neighbors only, 2 = neighbors of neighbors, etc.
	 * Higher values allow smoother blending across more bones but reduce locality.
	 */
	UPROPERTY(EditAnywhere, Category = "Binding", meta = (DataflowInput, ClampMin = 1, UIMax = 5, ClampMax = 100))
	int32 ClusterNeighborSearchRange = 1;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;


public:
	FSkinMeshViaMedialSkeleton_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

namespace UE::Dataflow
{
	void RegisterMeshMedialSkeletonConversionNodes();
}



