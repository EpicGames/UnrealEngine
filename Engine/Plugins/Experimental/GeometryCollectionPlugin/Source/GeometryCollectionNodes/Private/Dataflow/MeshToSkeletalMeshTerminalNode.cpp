// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/MeshToSkeletalMeshTerminalNode.h"

#include "BoneWeights.h"
#include "DynamicMeshToMeshDescription.h"
#include "UDynamicMesh.h"
#include "Rendering/SkeletalMeshModel.h"
#include "SkeletalMeshAttributes.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/PackageName.h"

#include "Engine/SkeletalMesh.h"
#include "Animation/Skeleton.h"
#include "Engine/SkinnedAssetCommon.h"
#include "SkeletalMeshTypes.h"

#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowDebugDraw.h"
#include "Dataflow/DataflowMesh.h"

#if WITH_EDITOR
#include "StaticToSkeletalMeshConverter.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#include "Dataflow/DataflowSkeletalMeshNodes.h"
#endif

#include "DynamicMeshToSkeleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshToSkeletalMeshTerminalNode)

#define LOCTEXT_NAMESPACE "MeshToSkeletalMeshTerminalNode"

namespace UE::Dataflow
{

	void RegisterMeshToSkeletalMeshTerminalNode()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMeshToSkeletalMeshTerminalNode_v2);
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSkeletalMeshTerminalNode);

		// Deprecated
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FMeshToSkeletalMeshTerminalNode);
	}

}

namespace MeshToSkeletalMeshAssetLocals
{
#if WITH_EDITOR
	// Helper to convert a mesh description to a skeletal mesh.
	// editor-only due to FStaticToSkeletalMeshConverter usage
	void MeshDescriptionToSkeletalMesh(const FMeshDescription& MeshDescription, FSkeletalMeshAttributes& Attributes, const TArray<TObjectPtr<UMaterialInterface>>& InMaterials, USkeletalMesh* OutSkeletalMesh, USkeleton* OutSkeleton)
	{
		// Copy bones to skeleton
		if (Attributes.GetNumBones() &&
			// Note FReferenceSkeletonModifier does not allow us to remove a root bone, so we can't clear an existing skeleton here; need to be passed a fresh/empty skeleton
			ensureMsgf(OutSkeleton->GetReferenceSkeleton().GetRefBoneInfo().IsEmpty(), TEXT("Expected an empty skeleton to fill, but skeleton already had bones")))
		{
			FReferenceSkeletonModifier Modifier(OutSkeleton);
			// For now, we assume the bone hierarchy is consistent and can construct a well-formed ref skeleton.
			FSkeletalMeshAttributesShared::FBoneNameAttributesConstRef BoneNames(Attributes.GetBoneNames());
			FSkeletalMeshAttributesShared::FBoneParentIndexAttributesConstRef BoneParents(Attributes.GetBoneParentIndices());
			FSkeletalMeshAttributesShared::FBonePoseAttributesConstRef BonePoses(Attributes.GetBonePoses());

			for (int32 BoneIndex = 0; BoneIndex < Attributes.GetNumBones(); BoneIndex++)
			{
				Modifier.Add(FMeshBoneInfo(BoneNames.Get(BoneIndex), BoneNames.Get(BoneIndex).ToString(), BoneParents.Get(BoneIndex)),
					BonePoses.Get(BoneIndex));
			}
		}

		// Set skeletal materials from InMaterials
		TArray<FSkeletalMaterial> SkeletalMaterials;
		for (int32 MatIndex = 0; MatIndex < InMaterials.Num(); ++MatIndex)
		{
			const FName SlotName = FName("Material", MatIndex);

			FSkeletalMaterial NewMaterial;
			NewMaterial.MaterialInterface = InMaterials.IsValidIndex(MatIndex) ? InMaterials[MatIndex] : nullptr;
			NewMaterial.MaterialSlotName = SlotName;
			NewMaterial.ImportedMaterialSlotName = SlotName;
			SkeletalMaterials.Add(NewMaterial);
		}

		TArray<const FMeshDescription*> MeshDescriptions = { &MeshDescription };

		// Update the skeletal mesh from the description
		FStaticToSkeletalMeshConverter::FInitializationParams Parameters;
		Parameters.Materials = SkeletalMaterials;
		Parameters.bRecomputeNormals = false;
		Parameters.bRecomputeTangents = false;

		FStaticToSkeletalMeshConverter::InitializeSkeletalMeshFromMeshDescriptions(
			OutSkeletalMesh,
			MeshDescriptions,
			OutSkeleton->GetReferenceSkeleton(),
			Parameters);

		OutSkeletalMesh->SetSkeleton(OutSkeleton);
		OutSkeleton->SetPreviewMesh(OutSkeletalMesh);
		OutSkeleton->RecreateBoneTree(OutSkeletalMesh);
	}

	bool CanReplaceSkeletonAsset(const USkeleton* SkeletonAsset, const USkeletalMesh* AllowedToReference)
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

		TArray<FAssetIdentifier> Referencers;
		AssetRegistry.GetReferencers(SkeletonAsset->GetPackage()->GetFName(), Referencers);

		FName AllowedPackage = AllowedToReference ? AllowedToReference->GetPackage()->GetFName() : FName();

		for (const FAssetIdentifier& Referencer : Referencers)
		{
			TArray<FAssetData> Assets;
			AssetRegistry.GetAssetsByPackageName(Referencer.PackageName, Assets, false);

			// handle the no-assets-allowed case
			if (!AllowedToReference && !Assets.IsEmpty())
			{
				return false;
			}

			for (const FAssetData& Asset : Assets)
			{
				// if allowed to reference a specific skeletal mesh, return false if it's not that mesh
				if (!Asset.IsInstanceOf(USkeletalMesh::StaticClass()) || Asset.PackageName != AllowedPackage)
				{
					return false;
				}
			}
		}
		return true;
	}

	template<typename AssetType>
	AssetType* MakeAssetFromPath(TFunctionRef<void(const FString&)> LogErr, const FString& InAssetPath, bool& bUsingExisting, bool bMustOverwriteExisting = false, TFunctionRef<bool(UObject* Existing)> CanOverwiteExisting = [](UObject*) {return true;})
	{
		if (!FPackageName::IsValidObjectPath(InAssetPath))
		{
			LogErr(FString::Format(TEXT("Asset Path input is not a valid path : {0}"), { InAssetPath }));
			return nullptr;
		}
		const FString PackageName = FPackageName::ObjectPathToPackageName(InAssetPath);

		UPackage* Package = Cast<UPackage>(FindPackage(nullptr, *PackageName));
		if (Package == nullptr)
		{
			Package = CreatePackage(*PackageName);
		}
		if (Package == nullptr)
		{
			LogErr(FString::Format(TEXT("Failed to find or create package {0}"), { PackageName }));
			return nullptr;
		}

		const FName AssetName = FName(FPackageName::GetLongPackageAssetName(PackageName));
		UObject* ExistingObject = StaticFindObjectFastInternal( /*Class=*/ NULL, Package, AssetName, EFindObjectFlags::ExactClass);
		if (ExistingObject && !ExistingObject->GetClass()->IsChildOf<AssetType>())
		{
			LogErr(FString::Format(TEXT("Asset {0} already exists but is not a compatible type"), { InAssetPath }));
			return nullptr;
		}
		// It is an error if we are trying to replace an object of a different class
		TObjectPtr<AssetType> OutAsset = Cast<AssetType>(ExistingObject);
		if (ExistingObject && bMustOverwriteExisting)
		{
			if (!CanOverwiteExisting(ExistingObject))
			{
				LogErr(FString::Format(TEXT("Asset {0} already exists, and could not overwrite it"), { InAssetPath }));
				return nullptr;
			}
			// Overwrite is ok, create a fresh package for it
			Package = CreatePackage(*PackageName);
		}
		if (!OutAsset || bMustOverwriteExisting)
		{
			bUsingExisting = false;
			OutAsset = NewObject<AssetType>(Package, AssetType::StaticClass(), AssetName, RF_Public | RF_Standalone | RF_Transactional);
			if (!OutAsset)
			{
				LogErr(FString::Format(TEXT("Failed to create asset {0}"), { InAssetPath }));
				return nullptr;
			}
		}
		else
		{
			bUsingExisting = true;
		}

		// make sure the asset is set properly
		OutAsset->MarkPackageDirty();
		FAssetRegistryModule::AssetCreated(OutAsset.Get());
		return OutAsset;
	}
#endif
}

void FMeshToSkeletalMeshTerminalNode::Evaluate(UE::Dataflow::FContext& Context) const
{
	// Asset is created and stored by the SetAssetValue
	// Evaluate only return an existing one or nullptr
	const FString& InAssetPath = GetValue(Context, &SkeletalMeshAssetPath);
	TObjectPtr<USkeletalMesh> OutSkeletalMesh = ::LoadObject<USkeletalMesh>(nullptr, InAssetPath);
	SetValue(Context, OutSkeletalMesh, &SkeletalMeshAsset);
	TObjectPtr<USkeleton> OutSkeleton = nullptr;
	if (OutSkeletalMesh)
	{
		OutSkeleton = OutSkeletalMesh->GetSkeleton();
	}
	SetValue(Context, OutSkeleton, &SkeletonAsset);
}

void FMeshToSkeletalMeshTerminalNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	auto SetOutputsOnFailure = [this, &Context]()
	{
		TObjectPtr<USkeletalMesh> NullSkeletalMeshPtr = nullptr;
		TObjectPtr<USkeleton> NullSkeletonPtr = nullptr;
		SetValue(Context, NullSkeletalMeshPtr, &SkeletalMeshAsset);
		SetValue(Context, NullSkeletonPtr, &SkeletonAsset);
	};

#if WITH_EDITOR

	const FString& InAssetPath = GetValue(Context, &SkeletalMeshAssetPath);
	auto LogErr = [&Context, this](const FString& Msg) { Context.Error(Msg, this); };
	bool bUsingExistingSkeletalMesh = false;


	// validate that the mesh actually exists and has bones before trying to create/clear any assets
	TObjectPtr<const UDynamicMesh> InMesh = GetValue(Context, &Mesh);
	if (!InMesh)
	{
		Context.Error(LOCTEXT("MeshToSkeletalMeshNullMeshError", "MeshToSkeletalMeshTerminalNode: Null input mesh cannot be converted to a skeletal mesh."), this);
		SetOutputsOnFailure();
		return;
	}
	if (!InMesh->GetMeshRef().HasAttributes() || InMesh->GetMeshRef().Attributes()->GetNumBones() == 0)
	{
		Context.Error(LOCTEXT("MeshToSkeletalMeshBonelessMeshError", "MeshToSkeletalMeshTerminalNode: Cannot convert boneless mesh to a skeletal mesh."), this);
			SetOutputsOnFailure();
		return;
	}

	TObjectPtr<USkeletalMesh> OutSkeletalMesh = nullptr;
	if (!InAssetPath.IsEmpty())
	{
		// We try to re-use existing skeletal mesh assets when possible
		constexpr bool bMustReplaceExistingSkeletalMesh = false;
		OutSkeletalMesh = MeshToSkeletalMeshAssetLocals::MakeAssetFromPath<USkeletalMesh>(LogErr, InAssetPath, bUsingExistingSkeletalMesh, bMustReplaceExistingSkeletalMesh);
	}
	if (!OutSkeletalMesh)
	{
		// use the one passed as parameter
		OutSkeletalMesh = Asset ? Cast<USkeletalMesh>(Asset) : nullptr;
		if (OutSkeletalMesh)
		{
			bUsingExistingSkeletalMesh = true;
		}
	}
	if (!OutSkeletalMesh)
	{
		// check the outer ( case where Asset is a Dataflow Attachement )
		OutSkeletalMesh = Asset ? Cast<USkeletalMesh>(Asset->GetOuter()) : nullptr;
		if (OutSkeletalMesh)
		{
			bUsingExistingSkeletalMesh = true;
		}
	}
	if (!OutSkeletalMesh)
	{
		// Out of options
		Context.Error(LOCTEXT("MeshToSkeletalMeshInvalidSkeletalMesh", "Invalid destination Skeletal Mesh, check the asset path or if the asset bound to this dataflow is a skeletal mesh."), this);
		SetOutputsOnFailure();
		return;
	}

	// Note we try to create a fresh skeleton at the target asset path, rather than re-use an existing skeleton
	// because we have no guarantee about the skeleton topology being safe to re-use if it is referenced by other assets
	// and we don't want to solve a re-targeting problem here.
	// TODO: We could try to verify if the skeleton is compatible and re-use it in that case?
	TObjectPtr<USkeleton> OutSkeleton = nullptr;
	auto CanOverwriteSkeletonAsset = [OutSkeletalMesh](UObject* SkelObject) -> bool
	{
		if (USkeleton* TestSkeleton = Cast<USkeleton>(SkelObject))
		{
			return MeshToSkeletalMeshAssetLocals::CanReplaceSkeletonAsset(TestSkeleton, OutSkeletalMesh);
		}
		return false;
	};
	bool bUsingExistingSkeleton = false;

	const FString& InSkelAssetPath = GetValue(Context, &SkeletonAssetPath);
	if (!InSkelAssetPath.IsEmpty())
	{
		OutSkeleton = MeshToSkeletalMeshAssetLocals::MakeAssetFromPath<USkeleton>(LogErr, InSkelAssetPath, bUsingExistingSkeleton, true, CanOverwriteSkeletonAsset);
	}
	else if (!OutSkeleton)
	{
		const FString BaseName = InAssetPath.IsEmpty() ? OutSkeletalMesh->GetPathName() : InAssetPath;
		OutSkeleton = MeshToSkeletalMeshAssetLocals::MakeAssetFromPath<USkeleton>(LogErr, FString::Printf(TEXT("%s_Skeleton"), *BaseName), bUsingExistingSkeleton, true, CanOverwriteSkeletonAsset);
	}
	if (!OutSkeleton)
	{
		LogErr(TEXT("Failed to find or create a skeleton for the skeletal mesh"));
		// TODO: could we set a null skeleton here rather than failing?
		SetOutputsOnFailure();
		return;
	}

	const FDynamicMesh3& DynMesh = InMesh->GetMeshRef();
	const TArray<TObjectPtr<UMaterialInterface>>& InMaterials = GetValue(Context, &Materials);
	// convert the collection to a mesh description including skeletal mesh attributes
	FMeshDescription MeshDescription;

	FSkeletalMeshAttributes Attributes(MeshDescription);
	Attributes.Register();

	FConversionToMeshDescriptionOptions ConversionOptions;
	FDynamicMeshToMeshDescription Converter(ConversionOptions);
	Converter.Convert(&DynMesh, MeshDescription);

	TOptional<FScopedSkeletalMeshPostEditChange> ScopedPostEditChange;
	if (bUsingExistingSkeletalMesh)
	{
		ScopedPostEditChange.Emplace(OutSkeletalMesh);

		OutSkeletalMesh->Clear();
		// Make sure that we create non-transient assets
		OutSkeletalMesh->ClearFlags(RF_Transient);
	}

	MeshToSkeletalMeshAssetLocals::MeshDescriptionToSkeletalMesh(MeshDescription, Attributes, InMaterials, OutSkeletalMesh, OutSkeleton);

	// finally set the skeletal mesh to the output
	SetValue(Context, OutSkeletalMesh, &SkeletalMeshAsset);
	SetValue(Context, OutSkeleton, &SkeletonAsset);

#else
	Context.Error(TEXT("Creation of skeletal mesh asset only supported in Editor"), this);
	SetOutputsOnFailure();
#endif
}

FMeshToSkeletalMeshTerminalNode::FMeshToSkeletalMeshTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&Materials);
	RegisterInputConnection(&SkeletalMeshAssetPath);
	RegisterInputConnection(&SkeletonAssetPath);
	RegisterOutputConnection(&SkeletalMeshAsset);
	RegisterOutputConnection(&SkeletonAsset);
}

#if WITH_EDITOR
void FMeshToSkeletalMeshTerminalNode::DebugDraw(UE::Dataflow::FContext& Context,
	IDataflowDebugDrawInterface& DataflowRenderingInterface,
	const FDebugDrawParameters& DebugDrawParameters) const
{
	if (const FDataflowOutput* Output = FindOutput(&SkeletalMeshAsset))
	{
		if (const TObjectPtr<const USkeletalMesh> InSkeletalMeshAsset = Output->ReadValue(Context, SkeletalMeshAsset))
		{
			TRefCountPtr<IDataflowDebugDrawObject> SkeletonObject(MakeDebugDrawObject<FDataflowDebugDrawSkeletonObject>(
				DataflowRenderingInterface.ModifyDataflowElements(), InSkeletalMeshAsset->GetRefSkeleton()));

			DataflowRenderingInterface.DrawObject(SkeletonObject);
		}
	}
}

bool FMeshToSkeletalMeshTerminalNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}
#endif

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FSkeletalMeshTerminalNode::FSkeletalMeshTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&AssetPath).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterOutputConnection(&SkeletalMeshAsset);
}

void FSkeletalMeshTerminalNode::Evaluate(UE::Dataflow::FContext& Context) const
{
	// Asset is created and stored by the SetAssetValue
	// Evaluate returns this created asset
	const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>();
	TObjectPtr<UObject> BoundAsset = EngineContext ? EngineContext->Owner : nullptr;

	const FString InAssetPath = GetAssetPath(GetValue(Context, &AssetPath), BoundAsset);
	TObjectPtr<USkeletalMesh> OutSkeletalMeshAsset = ::LoadObject<USkeletalMesh>(nullptr, InAssetPath);
	SetValue(Context, OutSkeletalMeshAsset, &SkeletalMeshAsset);
}

void FSkeletalMeshTerminalNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	const TObjectPtr<USkeletalMesh> NullSkeletalMeshPtr = nullptr;

#if WITH_EDITOR
	const TObjectPtr<const UDataflowMesh> InMesh = GetValue(Context, &Mesh);
	if (!IsValid(InMesh) || !InMesh->GetDynamicMesh())
	{
		Context.Error(LOCTEXT("InvalidInputMesh", "Input dynamic mesh mesh is empty"), this);
		SetValue(Context, NullSkeletalMeshPtr, &SkeletalMeshAsset);
		return;
	}

	UObject* BoundAsset = UE::Dataflow::Private::FindBoundAsset(Asset);
	const FString InAssetPath = GetAssetPath(GetValue(Context, &AssetPath), BoundAsset);
	if (InAssetPath.IsEmpty())
	{
		Context.Error(FText::Format(
			LOCTEXT("InvalidPathAndBoundAsset", "Asset Path input is not a valid path and no SkeletalMesh asset is bound to this dataflow: path = {0}"),
			FText::FromString(InAssetPath)),
			this);
		SetValue(Context, NullSkeletalMeshPtr, &SkeletalMeshAsset);
		return;
	}

	UObject* OutAsset = GetOrCreateAsset(Context, InAssetPath, USkeletalMesh::StaticClass());
	TObjectPtr<USkeletalMesh> OutSkeletalMesh = Cast<USkeletalMesh>(OutAsset);
	if (!OutSkeletalMesh)
	{
		Context.Error(FText::Format(
			LOCTEXT("InvalidOutputAsset", "Failed to convert the new asset to a SkeletalMesh mesh: path = {0}"),
			FText::FromString(InAssetPath)),
			this);
		SetValue(Context, NullSkeletalMeshPtr, &SkeletalMeshAsset);
		return;
	}

	USkeleton* Skeleton = OutSkeletalMesh->GetSkeleton();
	if (!Skeleton)
	{
		Context.Error(FText::Format(
			LOCTEXT("InvalidSkeleton", "The skeletal mesh asset does not have a valid skeleton : path = {0}"),
			FText::FromString(InAssetPath)),
			this);
		SetValue(Context, NullSkeletalMeshPtr, &SkeletalMeshAsset);
		return;
	}

	const bool bIsMeshCompatibleWithSkeleton = UE::Conversion::IsDynamicMeshCompatibleWithSkeleton(*InMesh->GetDynamicMesh(), *Skeleton, true);
	if (!bIsMeshCompatibleWithSkeleton)
	{
		Context.Error(FText::Format(
			LOCTEXT("IncompatibleMeshWithSkeleton", "The mesh is not compatible with the asset skeleton: path = {0}"),
			FText::FromString(InAssetPath)),
			this);
		SetValue(Context, NullSkeletalMeshPtr, &SkeletalMeshAsset);
		return;
	}

	// now convert
	FMeshDescription MeshDescription;
	FSkeletalMeshAttributes Attributes(MeshDescription);
	Attributes.Register();

	FConversionToMeshDescriptionOptions ConversionOptions;
	FDynamicMeshToMeshDescription Converter(ConversionOptions);
	Converter.Convert(InMesh->GetDynamicMesh(), MeshDescription);

	TOptional<FScopedSkeletalMeshPostEditChange> ScopedPostEditChange;
	ScopedPostEditChange.Emplace(OutSkeletalMesh);
	OutSkeletalMesh->Clear();
	OutSkeletalMesh->ClearFlags(RF_Transient);

	// set back a few things since Clear is quite thorough :)
	OutSkeletalMesh->SetSkeleton(Skeleton);
	OutSkeletalMesh->SetRefSkeleton(Skeleton->GetReferenceSkeleton());

	// Copy bones to skeleton
	if (Attributes.GetNumBones())
	{
		FReferenceSkeletonModifier Modifier(OutSkeletalMesh->GetRefSkeleton(), /* Skeleton */ nullptr);
		// For now, we assume the bone hierarchy is consistent and can construct a well-formed ref skeleton.
		FSkeletalMeshAttributesShared::FBoneNameAttributesConstRef BoneNames(Attributes.GetBoneNames());
		FSkeletalMeshAttributesShared::FBoneParentIndexAttributesConstRef BoneParents(Attributes.GetBoneParentIndices());
		FSkeletalMeshAttributesShared::FBonePoseAttributesConstRef BonePoses(Attributes.GetBonePoses());

		// Assuming that the bones are in order (from parent to children) 
		TMap<int32, int32> MeshToSkeletonBoneMapping;
		for (int32 MeshBoneIndex = 0; MeshBoneIndex < Attributes.GetNumBones(); MeshBoneIndex++)
		{
			const FName BoneName = BoneNames.Get(MeshBoneIndex);
			int32 SkeletonBoneIndex = Modifier.FindBoneIndex(BoneName);
			if (SkeletonBoneIndex == INDEX_NONE)
			{
				const int32 MeshBoneParent = BoneParents.Get(MeshBoneIndex);
				const int32 SkeletonBoneParent = MeshToSkeletonBoneMapping.FindRef(MeshBoneParent, INDEX_NONE);
				if (SkeletonBoneParent != INDEX_NONE)
				{
					const FTransform BonePose = BonePoses.Get(MeshBoneIndex);
					Modifier.Add(FMeshBoneInfo(BoneName, BoneName.ToString(), SkeletonBoneParent), BonePose);
					SkeletonBoneIndex = Modifier.FindBoneIndex(BoneName);
				}
			}
			if (SkeletonBoneIndex != INDEX_NONE)
			{
				MeshToSkeletonBoneMapping.Add(MeshBoneIndex, SkeletonBoneIndex);
			}
		}
	}

	const TArray<TObjectPtr<UMaterialInterface>>& InMaterials = InMesh->GetMaterials();

	// Set skeletal materials from InMaterials
	TArray<FSkeletalMaterial> SkeletalMaterials;
	for (int32 MatIndex = 0; MatIndex < InMaterials.Num(); ++MatIndex)
	{
		const FName SlotName = FName("Material", MatIndex);

		FSkeletalMaterial NewMaterial;
		NewMaterial.MaterialInterface = InMaterials.IsValidIndex(MatIndex) ? InMaterials[MatIndex] : nullptr;
		NewMaterial.MaterialSlotName = SlotName;
		NewMaterial.ImportedMaterialSlotName = SlotName;
		SkeletalMaterials.Add(NewMaterial);
	}

	TArray<const FMeshDescription*> MeshDescriptions = { &MeshDescription };

	// Update the skeletal mesh from the description
	FStaticToSkeletalMeshConverter::FInitializationParams Parameters;
	Parameters.Materials = SkeletalMaterials;
	Parameters.bRecomputeNormals = false;
	Parameters.bRecomputeTangents = false;

	FStaticToSkeletalMeshConverter::InitializeSkeletalMeshFromMeshDescriptions(
	OutSkeletalMesh,
	MeshDescriptions,
	OutSkeletalMesh->GetRefSkeleton(),
	Parameters);

	SetValue(Context, OutSkeletalMesh, &SkeletalMeshAsset);
#else
	Context.Error(TEXT("Creation of skeletal mesh asset only supported in Editor"), this);
	SetValue(Context, NullSkeletalMeshPtr, &SkeletalMeshAsset);
#endif

}


/////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FMeshToSkeletalMeshTerminalNode_v2

FMeshToSkeletalMeshTerminalNode_v2::FMeshToSkeletalMeshTerminalNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&SkeletalMeshAssetPath);
	RegisterInputConnection(&SkeletonAssetPath);
	RegisterOutputConnection(&SkeletalMeshAsset);
	RegisterOutputConnection(&SkeletonAsset);
}

void FMeshToSkeletalMeshTerminalNode_v2::Evaluate(UE::Dataflow::FContext& Context) const
{
	// Asset is created and stored by the SetAssetValue
	// Evaluate only return an existing one or nullptr
	const FString& InAssetPath = GetValue(Context, &SkeletalMeshAssetPath);
	TObjectPtr<USkeletalMesh> OutSkeletalMesh = ::LoadObject<USkeletalMesh>(nullptr, InAssetPath);
	SetValue(Context, OutSkeletalMesh, &SkeletalMeshAsset);
	TObjectPtr<USkeleton> OutSkeleton = nullptr;
	if (OutSkeletalMesh)
	{
		OutSkeleton = OutSkeletalMesh->GetSkeleton();
	}
	SetValue(Context, OutSkeleton, &SkeletonAsset);
}

void FMeshToSkeletalMeshTerminalNode_v2::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	auto SetOutputsOnFailure = [this, &Context]()
	{
		TObjectPtr<USkeletalMesh> NullSkeletalMeshPtr = nullptr;
		TObjectPtr<USkeleton> NullSkeletonPtr = nullptr;
		SetValue(Context, NullSkeletalMeshPtr, &SkeletalMeshAsset);
		SetValue(Context, NullSkeletonPtr, &SkeletonAsset);
	};

#if WITH_EDITOR

	const FString& InAssetPath = GetValue(Context, &SkeletalMeshAssetPath);
	auto LogErr = [&Context, this](const FString& Msg) { Context.Error(Msg, this); };
	bool bUsingExistingSkeletalMesh = false;

	// validate that the mesh actually exists and has bones before trying to create/clear any assets
	const TObjectPtr<const UDataflowMesh> InMesh = GetValue(Context, &Mesh);
	if (!IsValid(InMesh) || !InMesh->GetDynamicMesh())
	{
		Context.Error(LOCTEXT("MeshToSkeletalMeshV2NullMeshError", "MeshToSkeletalMeshTerminal: Null input mesh cannot be converted to a skeletal mesh."), this);
		SetOutputsOnFailure();
		return;
	}
	const FDynamicMesh3& DynMesh = InMesh->GetDynamicMeshRef();
	if (!DynMesh.HasAttributes() || DynMesh.Attributes()->GetNumBones() == 0)
	{
		Context.Error(LOCTEXT("MeshToSkeletalMeshV2BonelessMeshError", "MeshToSkeletalMeshTerminal: Cannot convert boneless mesh to a skeletal mesh."), this);
		SetOutputsOnFailure();
		return;
	}

	TObjectPtr<USkeletalMesh> OutSkeletalMesh = nullptr;
	if (!InAssetPath.IsEmpty())
	{
		// We try to re-use existing skeletal mesh assets when possible
		constexpr bool bMustReplaceExistingSkeletalMesh = false;
		OutSkeletalMesh = MeshToSkeletalMeshAssetLocals::MakeAssetFromPath<USkeletalMesh>(LogErr, InAssetPath, bUsingExistingSkeletalMesh, bMustReplaceExistingSkeletalMesh);
	}
	if (!OutSkeletalMesh)
	{
		// use the one passed as parameter
		OutSkeletalMesh = Asset ? Cast<USkeletalMesh>(Asset) : nullptr;
		if (OutSkeletalMesh)
		{
			bUsingExistingSkeletalMesh = true;
		}
	}
	if (!OutSkeletalMesh)
	{
		// check the outer ( case where Asset is a Dataflow Attachement )
		OutSkeletalMesh = Asset ? Cast<USkeletalMesh>(Asset->GetOuter()) : nullptr;
		if (OutSkeletalMesh)
		{
			bUsingExistingSkeletalMesh = true;
		}
	}
	if (!OutSkeletalMesh)
	{
		// Out of options
		Context.Error(LOCTEXT("MeshToSkeletalMeshV2InvalidSkeletalMesh", "Invalid destination Skeletal Mesh, check the asset path or if the asset bound to this dataflow is a skeletal mesh."), this);
		SetOutputsOnFailure();
		return;
	}

	// Note we try to create a fresh skeleton at the target asset path, rather than re-use an existing skeleton
	// because we have no guarantee about the skeleton topology being safe to re-use if it is referenced by other assets
	// and we don't want to solve a re-targeting problem here.
	TObjectPtr<USkeleton> OutSkeleton = nullptr;
	auto CanOverwriteSkeletonAsset = [OutSkeletalMesh](UObject* SkelObject) -> bool
	{
		if (USkeleton* TestSkeleton = Cast<USkeleton>(SkelObject))
		{
			return MeshToSkeletalMeshAssetLocals::CanReplaceSkeletonAsset(TestSkeleton, OutSkeletalMesh);
		}
		return false;
	};
	bool bUsingExistingSkeleton = false;

	const FString& InSkelAssetPath = GetValue(Context, &SkeletonAssetPath);
	if (!InSkelAssetPath.IsEmpty())
	{
		OutSkeleton = MeshToSkeletalMeshAssetLocals::MakeAssetFromPath<USkeleton>(LogErr, InSkelAssetPath, bUsingExistingSkeleton, true, CanOverwriteSkeletonAsset);
	}
	else
	{
		const FString BaseName = InAssetPath.IsEmpty() ? OutSkeletalMesh->GetPathName() : InAssetPath;
		OutSkeleton = MeshToSkeletalMeshAssetLocals::MakeAssetFromPath<USkeleton>(LogErr, FString::Printf(TEXT("%s_Skeleton"), *BaseName), bUsingExistingSkeleton, true, CanOverwriteSkeletonAsset);
	}
	if (!OutSkeleton)
	{
		LogErr(TEXT("Failed to find or create a skeleton for the skeletal mesh"));
		SetOutputsOnFailure();
		return;
	}

	// Materials travel with the DataflowMesh (no separate input pin).
	const TArray<TObjectPtr<UMaterialInterface>>& InMaterials = InMesh->GetMaterials();

	// convert the dynamic mesh to a mesh description including skeletal mesh attributes
	FMeshDescription MeshDescription;
	FSkeletalMeshAttributes Attributes(MeshDescription);
	Attributes.Register();

	FConversionToMeshDescriptionOptions ConversionOptions;
	FDynamicMeshToMeshDescription Converter(ConversionOptions);
	Converter.Convert(&DynMesh, MeshDescription);

	TOptional<FScopedSkeletalMeshPostEditChange> ScopedPostEditChange;
	if (bUsingExistingSkeletalMesh)
	{
		ScopedPostEditChange.Emplace(OutSkeletalMesh);

		OutSkeletalMesh->Clear();
		// Make sure that we create non-transient assets
		OutSkeletalMesh->ClearFlags(RF_Transient);
	}

	MeshToSkeletalMeshAssetLocals::MeshDescriptionToSkeletalMesh(MeshDescription, Attributes, InMaterials, OutSkeletalMesh, OutSkeleton);

	// finally set the skeletal mesh to the output
	SetValue(Context, OutSkeletalMesh, &SkeletalMeshAsset);
	SetValue(Context, OutSkeleton, &SkeletonAsset);

#else
	Context.Error(TEXT("Creation of skeletal mesh asset only supported in Editor"), this);
	SetOutputsOnFailure();
#endif
}

#if WITH_EDITOR
void FMeshToSkeletalMeshTerminalNode_v2::DebugDraw(UE::Dataflow::FContext& Context,
	IDataflowDebugDrawInterface& DataflowRenderingInterface,
	const FDebugDrawParameters& DebugDrawParameters) const
{
	if (const FDataflowOutput* Output = FindOutput(&SkeletalMeshAsset))
	{
		if (const TObjectPtr<const USkeletalMesh> InSkeletalMeshAsset = Output->ReadValue(Context, SkeletalMeshAsset))
		{
			TRefCountPtr<IDataflowDebugDrawObject> SkeletonObject(MakeDebugDrawObject<FDataflowDebugDrawSkeletonObject>(
				DataflowRenderingInterface.ModifyDataflowElements(), InSkeletalMeshAsset->GetRefSkeleton()));

			DataflowRenderingInterface.DrawObject(SkeletonObject);
		}
	}
}

bool FMeshToSkeletalMeshTerminalNode_v2::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name;
}
#endif


#undef LOCTEXT_NAMESPACE

