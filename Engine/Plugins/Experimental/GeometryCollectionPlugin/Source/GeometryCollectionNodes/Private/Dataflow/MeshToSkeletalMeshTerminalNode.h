// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "UDynamicMesh.h"


#include "MeshToSkeletalMeshTerminalNode.generated.h"


class UDataflowMesh;

/*
* Terminal node to a save a skeletal mesh asset, as converted from a dynamic mesh.
* Note the mesh must have associated skeletal mesh attributes.
* The terminal node will create an asset if it does not exist yet
*/
USTRUCT(meta = (DataflowTerminal, Icon = "Icons.SkeletalMesh", Deprecated = "5.8"))
struct FMeshToSkeletalMeshTerminalNode : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMeshToSkeletalMeshTerminalNode, "MeshToSkeletalMeshTerminal", "Terminal", "Path Asset Mesh Skeletal Skeleton")

private:
	/** Surface mesh to convert */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<const UDynamicMesh> Mesh;

	/** Materials to use on the skeletal mesh */
	UPROPERTY(meta = (DataflowInput))
	TArray<TObjectPtr<UMaterialInterface>> Materials;


	/** Path to the skeletal mesh asset to create or update */
	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	FString SkeletalMeshAssetPath;

	/** Path to the skeleton asset to create or update */
	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	FString SkeletonAssetPath;

	/** Created Skeletal Mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<USkeletalMesh> SkeletalMeshAsset = nullptr;

	/** Created Skeleton */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<USkeleton> SkeletonAsset = nullptr;

	virtual void Evaluate(UE::Dataflow::FContext& Context) const override;
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;

#if WITH_EDITOR
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
#endif 

public:
	FMeshToSkeletalMeshTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/*
* Terminal node to save a skeletal mesh asset, as converted from a Dataflow mesh.
* The mesh must carry skeletal mesh attributes (bones). The terminal node will
* create the asset if it does not already exist.
*/
USTRUCT(meta = (DataflowTerminal, Icon = "Icons.SkeletalMesh"))
struct FMeshToSkeletalMeshTerminalNode_v2 : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMeshToSkeletalMeshTerminalNode_v2, "MeshToSkeletalMeshTerminal", "Terminal", "Path Asset Mesh Skeletal Skeleton")

private:
	/** Surface mesh to convert (carries materials inline) */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<const UDataflowMesh> Mesh;

	/** Path to the skeletal mesh asset to create or update */
	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	FString SkeletalMeshAssetPath;

	/** Path to the skeleton asset to create or update */
	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	FString SkeletonAssetPath;

	/** Created Skeletal Mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<USkeletalMesh> SkeletalMeshAsset = nullptr;

	/** Created Skeleton */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<USkeleton> SkeletonAsset = nullptr;

	virtual void Evaluate(UE::Dataflow::FContext& Context) const override;
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;

#if WITH_EDITOR
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
#endif

public:
	FMeshToSkeletalMeshTerminalNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/*
* Terminal to save to skeletal mesh from a mesh
* Note : the mesh must have associated skeletal mesh attributes and vbones must be compatible with the existing skeleton
*/
USTRUCT(meta = (DataflowTerminal, Experimental, Icon = "Icons.SkeletalMesh"))
struct FSkeletalMeshTerminalNode : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSkeletalMeshTerminalNode, "SkeletalMeshTerminal", "Terminal", "Asset")

private:
	/** Surface mesh to convert */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<const UDataflowMesh> Mesh;

	/** Path to the skeletal mesh asset to create or update */
	UPROPERTY(EditAnywhere, Category = Asset, meta = (DataflowInput))
	FString AssetPath;

	/** Created Skeletal Mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<USkeletalMesh> SkeletalMeshAsset = nullptr;

	virtual void Evaluate(UE::Dataflow::FContext& Context) const override;
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;

//#if WITH_EDITOR
//	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
//	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
//#endif 

public:
	FSkeletalMeshTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

namespace UE::Dataflow
{
	void RegisterMeshToSkeletalMeshTerminalNode();
}



