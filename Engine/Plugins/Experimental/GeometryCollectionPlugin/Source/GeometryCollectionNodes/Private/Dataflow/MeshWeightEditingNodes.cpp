// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/MeshWeightEditingNodes.h"

#include "Operations/SmoothBoneWeights.h"

#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshWeightEditingNodes)

#define LOCTEXT_NAMESPACE "MeshWeightEditingNodes"

namespace UE::Dataflow
{
	void RegisterMeshWeightEditingNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSmoothMeshBoneWeightsDataflowNode);
	}
}

FSmoothMeshBoneWeightsDataflowNode::FSmoothMeshBoneWeightsDataflowNode(
	const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&Strength).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&NumIterations).SetCanHidePin(true).SetPinIsHidden(true);
	RegisterInputConnection(&WeightProfile).SetCanHidePin(true).SetPinIsHidden(true);

	RegisterOutputConnection(&Mesh, &Mesh);
}

void FSmoothMeshBoneWeightsDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (Out->IsA(&Mesh))
	{
		TObjectPtr<const UDynamicMesh> InMesh = GetValue(Context, &Mesh);
		TObjectPtr<UDynamicMesh> OutMesh = NewObject<UDynamicMesh>();

		if (InMesh)
		{
			OutMesh->SetMesh(InMesh->GetMeshRef());
			FDynamicMesh3& DynMesh = OutMesh->GetMeshRef();

			const FName UseProfile = GetValue(Context, &WeightProfile);
			FSmoothDynamicMeshVertexSkinWeights Smoother(&DynMesh, UseProfile);
			if (Smoother.Validate() == EOperationValidationResult::Ok)
			{
				// Select all vertices to smooth
				TArray<int32> VerticesToSmooth;
				VerticesToSmooth.Reserve(DynMesh.VertexCount());
				for (int32 VID : DynMesh.VertexIndicesItr())
				{
					VerticesToSmooth.Add(VID);
				}

				const float UseStrength = FMath::Clamp(GetValue(Context, &Strength), 0.f, 1.f);
				const int32 UseIterations = FMath::Max(1, GetValue(Context, &NumIterations));
				constexpr double FloodFillUpToDistance = 0.0; // TODO: can expose this parameter if/when adding selection support, not needed when smoothing all vertices

				Smoother.SmoothWeightsAtVerticesWithinDistance(VerticesToSmooth, UseStrength, FloodFillUpToDistance, UseIterations);
			}
			else
			{
				Context.Warning(LOCTEXT("SmoothBoneWeightsNoWeightsError",
					"SmoothBoneWeights: Mesh has no skin weights on the specified profile; nothing to smooth."), this);
			}
		}
		else
		{
			Context.Error(LOCTEXT("SmoothBoneWeightsNullMeshError",
				"SmoothBoneWeights: Cannot smooth weights on a null input mesh"), this);
		}

		SetValue(Context, OutMesh, &Mesh);
	}
}

#undef LOCTEXT_NAMESPACE
