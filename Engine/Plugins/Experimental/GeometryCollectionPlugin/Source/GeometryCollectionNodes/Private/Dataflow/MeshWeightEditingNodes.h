// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowSelection.h"
#include "SkeletalMeshAttributes.h"

#include "MeshWeightEditingNodes.generated.h"

class UDynamicMesh;

/** Smooth bone weights on a dynamic mesh using Laplacian weight smoothing. */
USTRUCT(meta = (DataflowGeometryCollection))
struct FSmoothMeshBoneWeightsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSmoothMeshBoneWeightsDataflowNode, "SmoothBoneWeights", "SkeletonUtil", "Relax Blend")

private:
	/** Mesh with skin weights to smooth */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowIntrinsic, DataflowPassthrough = "Mesh"))
	TObjectPtr<UDynamicMesh> Mesh;

	/** Blend factor between original and smoothed weights (0=no change, 1=fully smoothed) */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, ClampMin = 0.0, ClampMax = 1.0))
	float Strength = 0.5f;

	/** Number of smoothing iterations to apply */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput, ClampMin = 1, UIMax = 20))
	int32 NumIterations = 1;

	/** Name of the skin weight profile to smooth. */
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput))
	FName WeightProfile = FSkeletalMeshAttributes::DefaultSkinWeightProfileName;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FSmoothMeshBoneWeightsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

namespace UE::Dataflow
{
	void RegisterMeshWeightEditingNodes();
}
