// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SetVertexColorFromFloatArrayNode.h"
#include "Dataflow/DataflowCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SetVertexColorFromFloatArrayNode)

void FSetVertexColorFromFloatArrayDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TArray<float>& InFloatArray = GetValue<TArray<float>>(Context, &FloatArray);

		const int32 NumVertices = InCollection.NumElements(FGeometryCollection::VerticesGroup);

		if (InFloatArray.Num() == NumVertices)
		{
			float MaxValue = 0;
			if (bNormalizeInput)
			{
				for (int32 Idx = 0; Idx < NumVertices; ++Idx)
				{
					MaxValue = FMath::Max(MaxValue, InFloatArray[Idx]);
				}
			}
			else
			{
				MaxValue = 1.0;
			}

			if (TManagedArray<FLinearColor>* VertexColors = InCollection.FindAttribute<FLinearColor>("Color", FGeometryCollection::VerticesGroup))
			{
				if (FMath::IsNearlyZero(MaxValue))
				{
					VertexColors->Fill(FLinearColor::Black);
				}
				else
				{
					for (int32 Idx = 0; Idx < NumVertices; ++Idx)
					{
						(*VertexColors)[Idx] = InFloatArray[Idx]/MaxValue * Color;
					}
				}
			}
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}
