// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SetVertexColorFromVertexIndicesNode.h"
#include "Dataflow/DataflowCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SetVertexColorFromVertexIndicesNode)

void FSetVertexColorFromVertexIndicesDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		auto IsValidVertex = [](int32 Index, int32 Num) {return 0 <= Index && Index < Num; };

		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		const int32 NumVertices = InCollection.NumElements(FGeometryCollection::VerticesGroup);
		const TArray<int32>& VertexIndices = GetValue<TArray<int32>>(Context, &VertexIndicesIn, VertexIndicesIn);

		if (TManagedArray<FLinearColor>* VertexColors = InCollection.FindAttribute<FLinearColor>("Color", FGeometryCollection::VerticesGroup))
		{
			for (int32 Idx : VertexIndices)
			{
				if (IsValidVertex(Idx, NumVertices))
				{
					(*VertexColors)[Idx] = SelectedColor;
				}
			}
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}
