// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SetVertexColorFromVertexSelectionNode.h"
#include "Dataflow/DataflowCore.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SetVertexColorFromVertexSelectionNode)

void FSetVertexColorFromVertexSelectionDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const FDataflowVertexSelection& InVertexSelection = GetValue<FDataflowVertexSelection>(Context, &VertexSelection);

		if (InCollection.NumElements(FGeometryCollection::VerticesGroup) == InVertexSelection.Num())
		{
			const int32 NumVertices = InCollection.NumElements(FGeometryCollection::VerticesGroup);
			if (TManagedArray<FLinearColor>* VertexColors = InCollection.FindAttribute<FLinearColor>("Color", FGeometryCollection::VerticesGroup))
			{
				for (int32 Idx = 0; Idx < NumVertices; ++Idx)
				{
					if (InVertexSelection.IsSelected(Idx))
					{
						(*VertexColors)[Idx] = SelectedColor;
					}
				}
			}
		}

		SetValue(Context, MoveTemp(InCollection), &Collection);
	}
}
