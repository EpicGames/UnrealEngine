// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dataflow/SubdivideMeshDataflowNodes.h"

#include "Dataflow/DataflowMesh.h"
#include "DynamicMesh/DynamicMesh3.h"

#if WITH_EDITOR
#include "Operations/SubdividePoly.h"
#include "Polygroups/PolygroupSet.h"
#include "Polygroups/PolygroupsGenerator.h"
#include "GroupTopology.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubdivideMeshDataflowNodes)

namespace UE::Dataflow
{
	void RegisterSubdivideMeshDataflowNodes()
	{
		DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FSubdivideMeshCatmullClarkDataflowNode);
	}
}


FSubdivideMeshCatmullClarkDataflowNode::FSubdivideMeshCatmullClarkDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Mesh);
	RegisterInputConnection(&Subdivisions);
	RegisterOutputConnection(&Mesh, &Mesh);
}


void FSubdivideMeshCatmullClarkDataflowNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;

	if (!Out->IsA(&Mesh))
	{
		return;
	}

	if (TObjectPtr<UDataflowMesh> InMesh = GetValue(Context, &Mesh))
	{
		if (const FDynamicMesh3* const InDynamicMesh = InMesh->GetDynamicMesh())
		{
			int32 SubdivLevel = GetValue(Context, &Subdivisions);
			if (SubdivLevel > 6)
			{
				Context.Warning(TEXT("Clamping Subdivision to 6 levels to avoid catastrophic memory usage. Use repeated Subdivisions if higher levels are necessary."), this, Out);
				SubdivLevel = 6;
			}

			if (SubdivLevel > 0)
			{
#if WITH_EDITOR
				FDynamicMesh3 OutDynamicMesh;
				OutDynamicMesh.Copy(*InDynamicMesh);

				const FPolygroupLayer InputGroupLayer{ bDefaultLayer, ExtendedLayerIndex };

				bool bGeneratedQuads = false;
				auto GenerateQuads = [&InputGroupLayer, &OutDynamicMesh, &bGeneratedQuads]()
				{
					InputGroupLayer.EnableOnMesh(OutDynamicMesh);

					FPolygroupSet OutputGroups(&OutDynamicMesh, InputGroupLayer);
					FPolygroupsGenerator Generator(&OutDynamicMesh);
					Generator.bApplyPostProcessing = false;
					Generator.bCopyToMesh = false;
					Generator.FindSourceMeshPolygonPolygroups();
					Generator.CopyPolygroupsToPolygroupSet(OutputGroups, OutDynamicMesh);
					bGeneratedQuads = true;
				};

				const bool bInitiallyMissing = !InputGroupLayer.CheckExists(&OutDynamicMesh)
					// consider single-group for the entire mesh as missing groups, for default layer groups
					|| (InputGroupLayer.bIsDefaultLayer && OutDynamicMesh.MaxGroupID() < 2);
				if (AutogenPolygroups == EDataflowSubdivideAutogenPolygroupsMode::Always ||
					(bInitiallyMissing && 
					(AutogenPolygroups == EDataflowSubdivideAutogenPolygroupsMode::IfMissing || AutogenPolygroups == EDataflowSubdivideAutogenPolygroupsMode::IfMissingOrInvalid)))
				{
					GenerateQuads();
				}

				if (!InputGroupLayer.CheckExists(&OutDynamicMesh))
				{
					Context.Error(TEXT("Target Polygroup Layer does not exist"), this, Out);
				}
				else
				{
					bool bShouldRetry = false;
					auto ApplySubdiv = [this, &OutDynamicMesh, SubdivLevel, &bGeneratedQuads, &Context, Out, &InMesh, &bShouldRetry]() -> bool
					{
						constexpr bool bAutoBuild = false;
						TUniquePtr<FGroupTopology> Topo = bDefaultLayer
							? MakeUnique<FGroupTopology>(&OutDynamicMesh, bAutoBuild)
							: MakeUnique<FGroupTopology>(&OutDynamicMesh, OutDynamicMesh.Attributes()->GetPolygroupLayer(ExtendedLayerIndex), bAutoBuild);

						Topo->ShouldAddExtraCornerAtVert = 
							[&OutDynamicMesh](const FGroupTopology& GroupTopology, int32 Vid, const FIndex2i& AttachedGroupEdgeEids)
								{
									// Help avoid creating group boundary with only 2 group edges
									return OutDynamicMesh.IsBoundaryVertex(Vid);
								};
							
						Topo->RebuildTopology();
							
						FSubdividePoly SubD(*Topo, OutDynamicMesh, SubdivLevel);
						if (SubD.ValidateTopology() != FSubdividePoly::ETopologyCheckResult::Ok)
						{
							bShouldRetry = AutogenPolygroups == EDataflowSubdivideAutogenPolygroupsMode::IfMissingOrInvalid && !bGeneratedQuads;
							if (!bShouldRetry)
							{
								Context.Error(TEXT("Target Polygroup Layer does not define valid Polygons for Subdivision"), this, Out);
							}
							return false;
						}

						SubD.SubdivisionScheme = ESubdivisionScheme::CatmullClark;
						SubD.NormalComputationMethod = ESubdivisionOutputNormals::Interpolated;
						SubD.UVComputationMethod = ESubdivisionOutputUVs::Interpolated;

						FDynamicMesh3 SubDMesh;
						if (!SubD.ComputeTopologySubdivision() || !SubD.ComputeSubdividedMesh(SubDMesh))
						{
							Context.Error(TEXT("Subdivision Failed"), this, Out);
							return false;
						}

						TObjectPtr<UDataflowMesh> OutMesh = NewObject<UDataflowMesh>();
						OutMesh->SetDynamicMesh(MoveTemp(SubDMesh));
						OutMesh->SetMaterials(InMesh->GetMaterials());
						SetValue(Context, OutMesh, &Mesh);
						return true;
					};
					bool bSucceeded = ApplySubdiv();
					if (!bSucceeded && bShouldRetry)
					{
						GenerateQuads();
						bSucceeded = ApplySubdiv();
					}

					if (bSucceeded)
					{
						return;
					}
				}
#else
				Context.Error(TEXT("OpenSubdiv-based subdivision is editor-only"), this, Out);
#endif
			}
		}
		else
		{
			Context.Error(TEXT("Mesh is missing DynamicMesh object"), this, Out);
		}
	}

	SafeForwardInput(Context, &Mesh, &Mesh);
}
