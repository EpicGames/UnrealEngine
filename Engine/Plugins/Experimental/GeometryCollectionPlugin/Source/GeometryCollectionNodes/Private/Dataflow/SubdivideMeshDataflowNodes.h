// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowEngine.h"

#include "SubdivideMeshDataflowNodes.generated.h"

class UDataflowMesh;

UENUM()
enum class EDataflowSubdivideAutogenPolygroupsMode : uint8
{
	/** Never auto-generate; error out if the configured polygroup layer is unsuitable. */
	Never,

	/** Always run polygon detection and overwrite the configured layer before subdividing. */
	Always,

	/** Run polygon detection only when the configured polygroup layer does not exist on the mesh. */
	IfMissing,

	/** Same as IfMissing, plus re-run generation if the subdivider rejects the topology on the first attempt. */
	IfMissingOrInvalid
};

/**
 * Apply Catmull-Clark subdivision to a DataflowMesh using its polygroups as the control cage.
 * The subdivision is editor-only; in non-editor builds the input mesh is passed through unchanged.
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FSubdivideMeshCatmullClarkDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSubdivideMeshCatmullClarkDataflowNode, "SubdivideMeshCatmullClark", "Mesh|Utilities", "Polygroup OpenSubdiv")

public:
	FSubdivideMeshCatmullClarkDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	/** DataflowMesh input/output */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Mesh", DataflowIntrinsic))
	TObjectPtr<UDataflowMesh> Mesh;

	/** Number of subdivision levels to apply */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 1, ClampMax = 6))
	int32 Subdivisions = 1;

	/** If true, use the mesh's default polygroup layer. If false, uses the extended polygroup layer at ExtendedLayerIndex. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bDefaultLayer = true;

	/** Index of the extended polygroup layer to use when bDefaultLayer is false */
	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "!bDefaultLayer", ClampMin = 0))
	int32 ExtendedLayerIndex = 0;

	/** Whether to auto-generate polygroups (by finding likely quads) when the configured polygroup layer is missing or not subdividable. */
	UPROPERTY(EditAnywhere, Category = Options)
	EDataflowSubdivideAutogenPolygroupsMode AutogenPolygroups = EDataflowSubdivideAutogenPolygroupsMode::IfMissing;
};

namespace UE::Dataflow
{
	void RegisterSubdivideMeshDataflowNodes();
}
