// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "CreateColorArrayFromFloatArrayNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class FGeometryCollection;

/** Set the vertex color on the collection based on the normalized float array. */
USTRUCT()
struct FCreateColorArrayFromFloatArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCreateColorArrayFromFloatArrayDataflowNode, "CreateColorArrayFromFloatArray", "Collection|Utilities", "")

public:
	/** Float array to use as a scalar for the color */
	UPROPERTY(meta = (DataflowInput, DisplayName  = "Float Array"))
	TArray<float> FloatArray;

	/** Color array output */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Linear Color Array"))
	TArray<FLinearColor> ColorArray;

	/** Enable normalization of input array */
	UPROPERTY(EditAnywhere, Category = "Color", meta = (DisplayName = "Normalize Input"))
	bool bNormalizeInput = true;
	
	/** Base color for the normalized float array */
	UPROPERTY(EditAnywhere, Category = "Color", meta = (DisplayName = "Color"))
	FLinearColor Color = FLinearColor(FColor::White);


	FCreateColorArrayFromFloatArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatArray);
		RegisterOutputConnection(&ColorArray);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};
