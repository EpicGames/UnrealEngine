// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowSelection.h"

#include "GeometryCollectionArrayNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class FGeometryCollection;


/**
 *
 * Returns the specified element from an array
 * DEPRECATED 5.6 - use the generic GetArrayElement node instead
 *
 */
USTRUCT(meta = (Deprecated = 5.6))
struct FGetFloatArrayElementDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetFloatArrayElementDataflowNode, "GetFloatArrayElement", "Utilities|Array", "")

public:
	/** Element index */
	UPROPERTY(EditAnywhere, Category = "Index");
	int32 Index = 0;

	/** Array to get the element from */
	UPROPERTY(meta = (DataflowInput))
	TArray<float> FloatArray;

	/** Specified element */
	UPROPERTY(meta = (DataflowOutput))
	float FloatValue = 0;

	FGetFloatArrayElementDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatArray);
		RegisterInputConnection(&Index);
		RegisterOutputConnection(&FloatValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


UENUM(BlueprintType)
enum class EFloatArrayToIntArrayFunctionEnum : uint8
{
	Dataflow_FloatToInt_Function_Floor UMETA(DisplayName = "Floor()"),
	Dataflow_FloatToInt_Function_Ceil UMETA(DisplayName = "Ceil()"),
	Dataflow_FloatToInt_Function_Round UMETA(DisplayName = "Round()"),
	Dataflow_FloatToInt_Function_Truncate UMETA(DisplayName = "Truncate()"),
	Dataflow_FloatToInt_NonZeroToIndex UMETA(DisplayName = "Non-zero to Index"),
	Dataflow_FloatToInt_ZeroToIndex UMETA(DisplayName = "Zero to Index"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 * Converts a Float array to Int array using the specified method.
 *
 */
USTRUCT()
struct FFloatArrayToIntArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFloatArrayToIntArrayDataflowNode, "FloatArrayToIntArray", "Math|Conversions", "")

public:
	/** Conversion method:
	* Floor takes the floor of each input float value - 1.1 turns into 1.
	* Ceil takes the ceil - 1.1 turns into 2.
	* Round rounds to the nearest integer - 1.1 turns into 1.
	* Tuncate trucates like a type cast - 1.1 turns into 1.
	* Non-zero to Index appends the index of all non-zero values to the output array.
	* Zero to Index appends the index of all zero values to the output array.
	*/
	UPROPERTY(EditAnywhere, Category = "Float");
	EFloatArrayToIntArrayFunctionEnum Function = EFloatArrayToIntArrayFunctionEnum::Dataflow_FloatToInt_NonZeroToIndex;

	/** Float array value to convert */
	UPROPERTY(EditAnywhere, Category = "Float", meta = (DataflowInput))
	TArray<float> FloatArray;

	/** Int array output */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> IntArray;

	FFloatArrayToIntArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatArray);
		RegisterOutputConnection(&IntArray);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Returns the specified element from an array
 * DEPRECATED 5.6 - use the generic GetArrayElement node instead
 *
 */
USTRUCT(meta = (Deprecated = 5.6))
struct FGetArrayElementDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetArrayElementDataflowNode, "GetArrayElement", "Utilities|Array", "")

public:
	/** Element index */
	UPROPERTY(EditAnywhere, Category = "Index");
	int32 Index = 0;

	/** Array to get the element from */
	UPROPERTY(meta = (DataflowInput))
	TArray<FVector> Points;

	/** Specified element */
	UPROPERTY(meta = (DataflowOutput))
	FVector Point = FVector(0.0);

	FGetArrayElementDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Points);
		RegisterInputConnection(&Index);
		RegisterOutputConnection(&Point);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Returns the number of elements in an array
 * DEPRECATED 5.6 - use the generic GetArraySize node instead
 * 
 */
USTRUCT(meta = (Deprecated = 5.6))
struct FGetNumArrayElementsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetNumArrayElementsDataflowNode, "GetNumArrayElements", "Utilities|Array", "")

public:
	/** Float array input */
	UPROPERTY(meta = (DataflowInput, DisplayName = "FloatArray"))
	TArray<float> FloatArray;

	/** Int32 array input */
	UPROPERTY(meta = (DataflowInput, DisplayName = "IntArray"))
	TArray<int32> IntArray;

	/** FVector array input */
	UPROPERTY(meta = (DataflowInput, DisplayName = "VectorArray"))
	TArray<FVector> Points;

	/** FVector3f array input */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Vector3fArray"))
	TArray<FVector3f> Vector3fArray;

	/** Number of elements in the array */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumElements = 0;

	FGetNumArrayElementsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatArray);
		RegisterInputConnection(&IntArray);
		RegisterInputConnection(&Points);
		RegisterInputConnection(&Vector3fArray);
		RegisterOutputConnection(&NumElements);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Converts a TArray<bool> to a FDataflowFaceSelection
 *
 */
USTRUCT()
struct FBoolArrayToFaceSelectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBoolArrayToFaceSelectionDataflowNode, "BoolArrayToFaceSelection", "Utilities|Array", "")

public:
	/** TArray<bool> data */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic));
	TArray<bool> BoolAttributeData;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "FaceSelection"))
	FDataflowFaceSelection FaceSelection;

	FBoolArrayToFaceSelectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&BoolAttributeData);
		RegisterOutputConnection(&FaceSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


UENUM(BlueprintType)
enum class ECompareOperation1Enum : uint8
{
	Dataflow_Compare_Equal UMETA(DisplayName = "=="),
	Dataflow_Compare_Smaller UMETA(DisplayName = "<"),
	Dataflow_Compare_SmallerOrEqual UMETA(DisplayName = "<="),
	Dataflow_Compare_Greater UMETA(DisplayName = ">"),
	Dataflow_Compare_GreaterOrEqual UMETA(DisplayName = ">="),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 * Converts a TArray<float> to a FDataflowVertexSelection
 *
 */
USTRUCT()
struct FFloatArrayToVertexSelectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFloatArrayToVertexSelectionDataflowNode, "FloatArrayToVertexSelection", "Utilities|Array", "")

public:
	/** TArray<floatl> array */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic));
	TArray<float> FloatArray;

	/** Comparison operation */
	UPROPERTY(EditAnywhere, Category = "Compare");
	ECompareOperation1Enum Operation = ECompareOperation1Enum::Dataflow_Compare_Greater;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Compare")
	float Threshold = 0.f;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "VertexSelection"))
	FDataflowVertexSelection VertexSelection;

	FFloatArrayToVertexSelectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatArray);
		RegisterOutputConnection(&VertexSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Normalize the selected float data in a FloatArray
 *
 */
USTRUCT()
struct FFloatArrayNormalizeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFloatArrayNormalizeDataflowNode, "FloatArrayNormalize", "Math|Float", "")

public:
	/** Input VectorArray */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<float> InFloatArray;

	/** Selection for the operation */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVertexSelection Selection;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float MinRange = 0.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float MaxRange = 1.f;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<float> OutFloatArray;

	FFloatArrayNormalizeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&InFloatArray);
		RegisterInputConnection(&Selection);
		RegisterInputConnection(&MinRange);
		RegisterInputConnection(&MaxRange);
		RegisterOutputConnection(&OutFloatArray);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Normalize all the selected vectors in a VectorArray
 *
 */
USTRUCT()
struct FVectorArrayNormalizeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVectorArrayNormalizeDataflowNode, "VectorArrayNormalize", "Math|Vector", "")

public:
	/** Input VectorArray */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector> InVectorArray;

	/** Selection for the operation */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVertexSelection Selection;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float Magnitude = 1.f;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> OutVectorArray;

	FVectorArrayNormalizeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&InVectorArray);
		RegisterInputConnection(&Selection);
		RegisterInputConnection(&Magnitude);
		RegisterOutputConnection(&OutVectorArray);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


USTRUCT()
struct FUnionIntArraysDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FUnionIntArraysDataflowNode, "UnionIntArrays", "Utilities", "")

public:
	UPROPERTY(meta = (DataflowInput, DisplayName = "InArray1"));
	TArray<int32> InArray1;

	UPROPERTY(meta = (DataflowInput, DisplayName = "InArray2"));
	TArray<int32> InArray2;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "OutArray", DataflowPassthrough = "InArray1"));
	TArray<int32> OutArray;

	FUnionIntArraysDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&InArray1);
		RegisterInputConnection(&InArray2);
		RegisterOutputConnection(&OutArray, &InArray1);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


/**
 *
 * Removes the specified element from an array
 *
 */
USTRUCT()
struct FRemoveFloatArrayElementDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRemoveFloatArrayElementDataflowNode, "RemoveFloatArrayElement", "Utilities|Array", "")

public:
	/** Element index */
	UPROPERTY(EditAnywhere, Category = "Index");
	int32 Index = 0;

	/** Preserve order, if order not important set it to false for faster computation */
	UPROPERTY(EditAnywhere, Category = "Order");
	bool bPreserveOrder = true;

	/** Array to remove the element from */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "FloatArray", DataflowIntrinsic))
	TArray<float> FloatArray;

	FRemoveFloatArrayElementDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatArray);
		RegisterInputConnection(&Index);
		RegisterOutputConnection(&FloatArray, &FloatArray);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


UENUM(BlueprintType)
enum class EStatisticsOperationEnum : uint8
{
	Dataflow_EStatisticsOperationEnum_Min		UMETA(DisplayName = "Min", ToolTip = ""),
	Dataflow_EStatisticsOperationEnum_Max		UMETA(DisplayName = "Max", ToolTip = ""),
	Dataflow_EStatisticsOperationEnum_Mean		UMETA(DisplayName = "Mean", ToolTip = ""),
	Dataflow_EStatisticsOperationEnum_Median	UMETA(DisplayName = "Median", ToolTip = ""),
	Dataflow_EStatisticsOperationEnum_Mode		UMETA(DisplayName = "Mode", ToolTip = ""),
	Dataflow_EStatisticsOperationEnum_Sum		UMETA(DisplayName = "Sum", ToolTip = ""),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 * Computes statistics of a float array
 *
 */
USTRUCT()
struct FFloatArrayComputeStatisticsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFloatArrayComputeStatisticsDataflowNode, "FloatArrayComputeStatistics", "Utilities|Array", "")

public:
	/** Array to compute values from */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<float> FloatArray;

	/** TransformSelection describes which values to use, if not connected all the elements will be used */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/** Statistics Operation */
	UPROPERTY(EditAnywhere, Category = "Operation", meta = (DisplayName = "Operation"))
	EStatisticsOperationEnum OperationName = EStatisticsOperationEnum::Dataflow_EStatisticsOperationEnum_Min;

	/** Computed value */
	UPROPERTY(meta = (DataflowOutput))
	float Value = 0.f;

	/** Indices of elements with the computed value */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> Indices;

	FFloatArrayComputeStatisticsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatArray);
		RegisterInputConnection(&TransformSelection);
		RegisterOutputConnection(&Value);
		RegisterOutputConnection(&Indices);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Randomize elements in a float array (Random value will be in (RandomRangeMin, RandomRangeMax)
 *
 */
USTRUCT()
struct FRandomizeFloatArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRandomizeFloatArrayDataflowNode, "RandomizeFloatArray", "Utilities|Array", "")

public:
	/** Array to randomize */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "FloatArray", DataflowIntrinsic))
	TArray<float> FloatArray;

	/** Random range min */
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput, DisplayName = "Random Range Min"))
	float RandomRangeMin = 0.f;

	/** Random range max */
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput, DisplayName = "Random Range Max"))
	float RandomRangeMax = 1.f;

	/** Seed for random */
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput, UIMin = "0"));
	int32 RandomSeed = 0;

	FRandomizeFloatArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatArray);
		RegisterInputConnection(&RandomRangeMin);
		RegisterInputConnection(&RandomRangeMax);
		RegisterInputConnection(&RandomSeed);
		RegisterOutputConnection(&FloatArray, &FloatArray);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Get size of an array
 */
USTRUCT()
struct FDataflowGetArraySizeNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowGetArraySizeNode, "GetArraySize", "Utilities|Array", "Count Length Num")

private:
	/** Array to get size from */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowArrayTypes Array;

	/** Computed value */
	UPROPERTY(meta = (DataflowOutput))
	int32 Size = 0;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FDataflowGetArraySizeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Get an element from an array
 */
USTRUCT()
struct FDataflowGetArrayElementNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowGetArrayElementNode, "GetArrayElement", "Utilities|Array", "Index At")

private:
	/** Array to get the element from */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Array", DataflowIntrinsic))
	FDataflowArrayTypes Array;

	/** Index to get the element at */
	UPROPERTY(EditAnywhere, Category = "Array", meta = (DataflowInput))
	int32 Index = 0;

	/** Element from the array at the specified index */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowAllTypes Element;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	virtual FName GetDependentConnectionType(const FDataflowConnection& SourceConnection, const FDataflowConnection& DependentConnection) const override;

public:
	FDataflowGetArrayElementNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};


/**
 * convert a single element to an array 
 */
USTRUCT()
struct FDataflowConvertToArrayNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowConvertToArrayNode, "ConvertToArray", "Utilities|Array", "Element")

public:
	FDataflowConvertToArrayNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Array to Convert to */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowArrayTypes Array;

	/** Element to convert to an array */
	UPROPERTY(meta = (DataflowInput))
	FDataflowAllTypes Element;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	virtual FName GetDependentConnectionType(const FDataflowConnection& SourceConnection, const FDataflowConnection& DependentConnection) const override;
};

/**
 * Make an array from multiple elements of the same type. When elements are array type, the arrays are appended (nested arrays are not supported).
 */
USTRUCT()
struct FDataflowMakeArrayNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMakeArrayNode, "MakeArray", "Utilities|Array", "Element")

public:
	FDataflowMakeArrayNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Array to Convert to */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowArrayTypes Array;

	/** Elements to convert to an array */
	UPROPERTY()
	TArray<FDataflowAllTypes> Elements;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	virtual FName GetDependentConnectionType(const FDataflowConnection& SourceConnection, const FDataflowConnection& DependentConnection) const override;

	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override { return true; }
	virtual bool CanRemovePin() const override { return Elements.Num() > NumInitialElements; }
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;

	UE::Dataflow::TConnectionReference<FDataflowAllTypes> GetConnectionReference(int32 Index) const;

	static constexpr int32 NumRequiredInputs = 0;
	static constexpr int32 NumInitialElements = 2; 

	inline static FName ArrayElementDependencyGroup = FName("ArrayElement");
};

/**
 * Append an element to an array of ManagedArrayCollections.
 */
USTRUCT()
struct FDataflowMakeManagedArrayCollectionArrayNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMakeManagedArrayCollectionArrayNode, "MakeManagedArrayCollectionArray", "Utilities|Array", "Make Managed Array Collection")

private:
	/** Array to append to. If no input connection, a new array will be created */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Array", DataflowIntrinsic))
	TArray<FManagedArrayCollection> Array;

	/** The element to append.*/
	UPROPERTY(meta = (DataflowInput))
	FManagedArrayCollection Element;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FDataflowMakeManagedArrayCollectionArrayNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

namespace UE::Dataflow
{
	void GeometryCollectionArrayNodes();
}

