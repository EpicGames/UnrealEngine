// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "Engine/Blueprint.h"

#include "GeometryCollectionAssetNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class UGeometryCollection;
class UBlueprint;

USTRUCT()
struct FDataflowRootProxyMesh
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = RootProxyMesh);
	TObjectPtr<UStaticMesh> Mesh;

	UPROPERTY(EditAnywhere, Category = RootProxyMesh);
	FTransform Transform;

	UPROPERTY(EditAnywhere, Category = RootProxyMesh);
	TArray<TObjectPtr<UMaterialInterface>> OverrideMaterials;
};

/**
* Create a RootProxyMesh object
* (used by geometry collection assets)
*/
USTRUCT()
struct FMakeRootProxyMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeRootProxyMeshDataflowNode, "MakeRootProxyMesh", "GeometryCollection", "")

private:
	/** mesh to use as a proxy */
	UPROPERTY(EditAnywhere, Category = RootProxyMesh, meta = (DataflowInput));
	TObjectPtr<UStaticMesh> Mesh;

	/** transform to use for the proxy, relative to the asset it will be used for */
	UPROPERTY(EditAnywhere, Category = RootProxyMesh, meta = (DataflowInput));
	FTransform Transform;

	UPROPERTY(EditAnywhere, Category = RootProxyMesh, meta = (DataflowInput));
	TArray<TObjectPtr<UMaterialInterface>> OverrideMaterials;

	/** mesh to use as a proxy */
	UPROPERTY(meta = (DataflowOutput));
	FDataflowRootProxyMesh RootProxyMesh;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FMakeRootProxyMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Create a RootProxyMesh Array
* (used by geometry collection assets)
*/
USTRUCT()
struct FMakeRootProxyMeshArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeRootProxyMeshArrayDataflowNode, "MakeRootProxyMeshArray", "GeometryCollection", "")

private:
	/** newly created array */
	UPROPERTY(EditAnywhere, Category = RootProxyMesh, meta = (DataflowOutput));
	TArray<FDataflowRootProxyMesh> RootProxyMeshes;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FMakeRootProxyMeshArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/*
* Add a root proxy object to an array of root proxy mesh
* * (used by geometry collection assets)
*/
USTRUCT(meta = (DataflowGeometryCollection, DataflowTerminal))
struct FAddRootProxyMeshToArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAddRootProxyMeshToArrayDataflowNode, "AddRootProxyMeshToArray", "GeometryCollection", "")

private:
	/** Root proxy array to add the mesh to */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = RootProxyMeshes));
	TArray<FDataflowRootProxyMesh> RootProxyMeshes;

	UPROPERTY(EditAnywhere, Category = RootProxyMesh, meta = (DataflowInput));
	FDataflowRootProxyMesh RootProxyMesh;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FAddRootProxyMeshToArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/*
* Geometry Collection asset terminal node
*/
USTRUCT(meta = (DataflowGeometryCollection, DataflowTerminal))
struct FGeometryCollectionTerminalDataflowNode_v2 : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGeometryCollectionTerminalDataflowNode_v2, "GeometryCollectionTerminal", "Terminal", "")

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Materials to set on this asset */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Materials"))
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	/** array of instanced meshes*/
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "InstancedMeshes"))
	TArray<FGeometryCollectionAutoInstanceMesh> InstancedMeshes;

	UPROPERTY(meta = (DataflowInput));
	TArray<FDataflowRootProxyMesh> RootProxyMeshes;

	virtual void Evaluate(UE::Dataflow::FContext& Context) const override;
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;

public:
	FGeometryCollectionTerminalDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/*
* Geometry Collection asset terminal node
* Deprecated (5.6) - Use version 2 of the same node that only support material interface array as materials input 
*/
USTRUCT(meta = (DataflowTerminal, Deprecated = "5.6"))
struct FGeometryCollectionTerminalDataflowNode : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGeometryCollectionTerminalDataflowNode, "GeometryCollectionTerminal", "Terminal", "")

public:

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DisplayName = "Collection"))
	FManagedArrayCollection Collection;

	/** Materials array to use for this asset */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Materials", DisplayName = "Materials"))
	TArray<TObjectPtr<UMaterial>> Materials;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "MaterialInstances", DisplayName = "MaterialInstances"))
	TArray<TObjectPtr<UMaterialInterface>> MaterialInstances;

	/** array of instanced meshes*/
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "InstancedMeshes", DisplayName = "InstancedMeshes"))
	TArray<FGeometryCollectionAutoInstanceMesh> InstancedMeshes;

	FGeometryCollectionTerminalDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context) const override;
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;
};


/**
 * Get Current geometry collection asset 
 * Note : Use with caution as this may get replaced in a near future for a more generic getAsset node
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGetGeometryCollectionAssetDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetGeometryCollectionAssetDataflowNode, "GetGeometryCollectionAsset", "GeometryCollection|Asset", "")

public:
	FGetGeometryCollectionAssetDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	/** Asset this data flow graph instance is assigned to */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Asset"))
	TObjectPtr<UGeometryCollection> Asset;
};


/**
 * Get the list of the original mesh information used to create a specific geometryc collection asset
 * each entry contains a mesh, a transform and a list of override materials
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGetGeometryCollectionSourcesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetGeometryCollectionSourcesDataflowNode, "GetGeometryCollectionSources", "GeometryCollection|Asset", "")

public:
	FGetGeometryCollectionSourcesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	
	/** Asset to get geometry sources from */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Asset"))
	TObjectPtr<UGeometryCollection> Asset;
	
	/** array of geometry sources */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Sources"))
	TArray<FGeometryCollectionSource> Sources;
};


/**
 * create a geometry collection from a set of geometry sources    
 * DEPRECATED 5.6 : use the new node version with a single material array output 
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FCreateGeometryCollectionFromSourcesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCreateGeometryCollectionFromSourcesDataflowNode, "CreateGeometryCollectionFromSources", "GeometryCollection|Asset", "")

public:
	FCreateGeometryCollectionFromSourcesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	
	/** array of geometry sources */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Sources"))
	TArray<FGeometryCollectionSource> Sources;

	/** Geometry collection newly created */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Collection"))
	FManagedArrayCollection Collection;

	/** Materials array to use for this asset */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Materials"))
	TArray<TObjectPtr<UMaterial>> Materials;

	/** Materials array to use for this asset */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "MaterialInstances"))
	TArray<TObjectPtr<UMaterialInterface>> MaterialInstances;

	/** array of instanced meshes*/
	UPROPERTY(meta = (DataflowOutput, DisplayName = "InstancedMeshes"))
	TArray<FGeometryCollectionAutoInstanceMesh> InstancedMeshes;
};

/**
 * create a geometry collection from a set of geometry sources
 * if the source array is not connected, the source array from the current asset is going to be used
  */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCreateGeometryCollectionFromSourcesDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCreateGeometryCollectionFromSourcesDataflowNode_v2, "CreateGeometryCollectionFromSources", "GeometryCollection|Asset", "")

public:
	FCreateGeometryCollectionFromSourcesDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	/** array of geometry sources */
	UPROPERTY(meta = (DataflowInput))
	TArray<FGeometryCollectionSource> Sources;

	/** Geometry collection newly created */
	UPROPERTY(meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	/** Materials array to use for this asset */
	UPROPERTY(meta = (DataflowOutput))
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	/** array of instanced meshes*/
	UPROPERTY(meta = (DataflowOutput))
	TArray<FGeometryCollectionAutoInstanceMesh> InstancedMeshes;

	/** corresponding source proxies */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FDataflowRootProxyMesh> RootProxyMeshes;
};

/**
 * Converts a UGeometryCollection asset to an FManagedArrayCollection
 * DEPRECATED 5.6 : use the new version  that only has one material array output 
 */
USTRUCT(meta = (DataflowContext = "GeometryCollection", DataflowGeometryCollection, Deprecated = "5.6"))
struct FGeometryCollectionToCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGeometryCollectionToCollectionDataflowNode, "GeometryCollectionToCollection", "GeometryCollection|Asset", "")

public:
	/** Asset input */
	UPROPERTY(EditAnywhere, Category = "Asset");
	TObjectPtr<UGeometryCollection> GeometryCollection;

	/** Geometry collection newly created */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Collection"))
	FManagedArrayCollection Collection;

	/** Materials array to use for this asset */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Materials"))
	TArray<TObjectPtr<UMaterial>> Materials;

	/** Material instances array from the static mesh */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "MaterialInstances"))
	TArray<TObjectPtr<UMaterialInterface>> MaterialInstances;

	/** Array of instanced meshes*/
	UPROPERTY(meta = (DataflowOutput, DisplayName = "InstancedMeshes"))
	TArray<FGeometryCollectionAutoInstanceMesh> InstancedMeshes;

	FGeometryCollectionToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Converts a UGeometryCollection asset to an FManagedArrayCollection
 */
USTRUCT(meta = (DataflowContext = "GeometryCollection", DataflowGeometryCollection))
struct FGeometryCollectionToCollectionDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGeometryCollectionToCollectionDataflowNode_v2, "GeometryCollectionToCollection", "GeometryCollection|Asset", "GC Convert")

public:
	FGeometryCollectionToCollectionDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Asset input */
	UPROPERTY(EditAnywhere, Category = "Asset", meta = (DataflowInput));
	TObjectPtr<UGeometryCollection> GeometryCollection;

	/** Geometry collection newly created */
	UPROPERTY(meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	/** Material instances array from the static mesh */
	UPROPERTY(meta = (DataflowOutput))
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	/** Array of instanced meshes*/
	UPROPERTY(meta = (DataflowOutput))
	TArray<FGeometryCollectionAutoInstanceMesh> InstancedMeshes;

	/** corresponding source proxies */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FDataflowRootProxyMesh> RootProxyMeshes;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Create a geometry collection from an asset
 * DEPRECATED 5.6 : use the new version  that only has one material array output 
 */
 USTRUCT(meta = (DataflowContext = "GeometryCollection", DataflowGeometryCollection, Deprecated = "5.6"))
 struct FBlueprintToCollectionDataflowNode : public FDataflowNode
 {
 	GENERATED_USTRUCT_BODY()
 	DATAFLOW_NODE_DEFINE_INTERNAL(FBlueprintToCollectionDataflowNode, "BlueprintToCollection", "GeometryCollection|Asset", "")

 public:
 	/** Asset input */
 	UPROPERTY(EditAnywhere, Category = "Asset");
	TObjectPtr<UBlueprint> Blueprint;
 
 	/** Split components */
 	UPROPERTY(EditAnywhere, Category = "Asset");
 	bool bSplitComponents = false;
 
 	/** Geometry collection newly created */
 	UPROPERTY(meta = (DataflowOutput, DisplayName = "Collection"))
 	FManagedArrayCollection Collection;
 
 	/** Materials array to use for this asset */
 	UPROPERTY(meta = (DataflowOutput, DisplayName = "Materials"))
 	TArray<TObjectPtr<UMaterial>> Materials;
 
	/** Material instances array from the static mesh */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "MaterialInstances"))
	TArray<TObjectPtr<UMaterialInterface>> MaterialInstances;

 	/** Array of instanced meshes*/
 	UPROPERTY(meta = (DataflowOutput, DisplayName = "InstancedMeshes"))
 	TArray<FGeometryCollectionAutoInstanceMesh> InstancedMeshes;
 
	FBlueprintToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
 	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
 };


/**
* Create a geometry collection from an asset
*/
USTRUCT(meta = (DataflowContext = "GeometryCollection", DataflowGeometryCollection))
	struct FBlueprintToCollectionDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBlueprintToCollectionDataflowNode_v2, "BlueprintToCollection", "GeometryCollection|Asset", "")

public:
	FBlueprintToCollectionDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Asset input */
	UPROPERTY(EditAnywhere, Category = "Asset");
	TObjectPtr<UBlueprint> Blueprint;

	/** Split components */
	UPROPERTY(EditAnywhere, Category = "Asset");
	bool bSplitComponents = false;

	/** Geometry collection newly created */
	UPROPERTY(meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	/** Material instances array from the static mesh */
	UPROPERTY(meta = (DataflowOutput))
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	/** Array of instanced meshes*/
	UPROPERTY(meta = (DataflowOutput))
	TArray<FGeometryCollectionAutoInstanceMesh> InstancedMeshes;

	/** corresponding source proxies */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FDataflowRootProxyMesh> RootProxyMeshes;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
* Create a random seed value from an asset
*/
USTRUCT(meta = (DataflowContext = "GeometryCollection", DataflowGeometryCollection))
struct FAssetRandomSeedDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAssetRandomSeedDataflowNode, "AssetRandomSeed", "GeometryCollection|Asset", "asset random seed")

public:
	FAssetRandomSeedDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Seed to combine with the seed generated from the asset path */
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput));
	int32 Seed = 0;

	/** Random seed value */
	UPROPERTY(meta = (DataflowOutput))
	int32 RandomSeed = 0;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void GeometryCollectionEngineAssetNodes();
}