// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowSelection.h"

#include "GeometryCollectionClusteringNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class FGeometryCollection;


UENUM(BlueprintType)
enum class EClusterSizeMethodEnum : uint8
{
	Dataflow_ClusterSizeMethod_ByNumber UMETA(DisplayName = "By Number"),
	Dataflow_ClusterSizeMethod_ByFractionOfInput UMETA(DisplayName = "By Fraction Of Input"),
	Dataflow_ClusterSizeMethod_BySize UMETA(DisplayName = "By Size"),
	Dataflow_ClusterSizeMethod_ByGrid UMETA(DisplayName = "By Grid"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 * Automatically group pieces of a fractured Collection into a specified number of clusters
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FAutoClusterDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAutoClusterDataflowNode, "AutoCluster", "GeometryCollection|Cluster", "")

private:
	/** How to choose the size of the clusters to create */
	UPROPERTY(EditAnywhere, Category = ClusterSize);
	EClusterSizeMethodEnum ClusterSizeMethod = EClusterSizeMethodEnum::Dataflow_ClusterSizeMethod_ByNumber;

	/** Use a Voronoi diagram with this many Voronoi sites as a guide for deciding cluster boundaries */
	UPROPERTY(EditAnywhere, Category = ClusterSize, meta = (DataflowInput, UIMin = 2, UIMax = 5000, EditCondition = "ClusterSizeMethod == EClusterSizeMethodEnum::Dataflow_ClusterSizeMethod_ByNumber"))
	int32 ClusterSites = 10;

	/** Choose the number of Voronoi sites used for clustering as a fraction of the number of child bones to process */
	UPROPERTY(EditAnywhere, Category = ClusterSize, meta = (DataflowInput, UIMin = 0.f, UIMax = 0.5f, EditCondition = "ClusterSizeMethod == EClusterSizeMethodEnum::Dataflow_ClusterSizeMethod_ByFractionOfInput"))
	float ClusterFraction = 0.25;

	/** Choose the Edge-Size of the cube used to groups bones under a cluster (in cm). */
	UPROPERTY(EditAnywhere, Category = ClusterSize, meta = (DataflowInput, DisplayName = "Cluster Size", UIMin = ".01", UIMax = "100", ClampMin = ".0001", ClampMax = "10000", EditCondition = "ClusterSizeMethod == EClusterSizeMethodEnum::Dataflow_ClusterSizeMethod_BySize"))
	float SiteSize = 1;

	/** Choose the number of cluster sites to distribute along the X axis */
	UPROPERTY(EditAnywhere, Category = ClusterSize, meta = (DataflowInput, ClampMin = "1", UIMax = "20", EditCondition = "ClusterSizeMethod == EClusterSizeMethodEnum::Dataflow_ClusterSizeMethod_ByGrid"))
	int ClusterGridWidth = 2;

	/** Choose the number of cluster sites to distribute along the Y axis */
	UPROPERTY(EditAnywhere, Category = ClusterSize, meta = (DataflowInput, ClampMin = "1", UIMax = "20", EditCondition = "ClusterSizeMethod == EClusterSizeMethodEnum::Dataflow_ClusterSizeMethod_ByGrid"))
	int ClusterGridDepth = 2;
	
	/** Choose the number of cluster sites to distribute along the Z axis */
	UPROPERTY(EditAnywhere, Category = ClusterSize, meta = (DataflowInput, ClampMin = "1", UIMax = "20", EditCondition = "ClusterSizeMethod == EClusterSizeMethodEnum::Dataflow_ClusterSizeMethod_ByGrid"))
	int ClusterGridHeight = 2;

	/** For a grid distribution, optionally iteratively recenter the grid points to the center of the cluster geometry (technically: applying K-Means iterations) to balance the shape and distribution of the clusters */
	UPROPERTY(EditAnywhere, Category = ClusterSize, meta = (DisplayName = "Grid Drift Iterations", ClampMin = "0", UIMax = "5", EditCondition = "ClusterSizeMethod == EClusterSizeMethodEnum::Dataflow_ClusterSizeMethod_ByGrid"))
	int DriftIterations = 0;

	/** If a cluster has volume less than this value (in cm) cubed, then the auto-cluster process will attempt to merge it into a neighboring cluster. */
	UPROPERTY(EditAnywhere, Category = ClusterSize, meta = (DataflowInput, ClampMin = "0"))
	float MinimumSize = 0;

	/** Whether to favor clusters that have a convex shape. (Note: Does not support ByGrid clustering.)  */
	UPROPERTY(EditAnywhere, Category = AutoCluster, meta = (EditCondition = "ClusterSizeMethod != EClusterSizeMethodEnum::Dataflow_ClusterSizeMethod_ByGrid"))
	bool bPreferConvexity = false;

	/** If > 0, cube root of maximum concave volume to add per cluster (ignoring concavity of individual parts) */
	UPROPERTY(EditAnywhere, Category = AutoCluster, meta = (EditCondition = "bPreferConvexity && ClusterSizeMethod != EClusterSizeMethodEnum::Dataflow_ClusterSizeMethod_ByGrid"))
	float ConcavityTolerance = 0;

	/** If true, bones will only be added to the same cluster if they are physically connected (either directly, or via other bones in the same cluster) */
	UPROPERTY(EditAnywhere, Category = AutoCluster, meta = (DisplayName = "Enforce Cluster Connectivity"))
	bool AutoCluster = true;

	/** If true, make sure the site parameters are matched as close as possible ( bEnforceConnectivity can make the number of site larger than the requested input may produce without it ) */
	UPROPERTY(EditAnywhere, Category = AutoCluster, meta = (EditCondition = "AutoCluster == true"))
	bool EnforceSiteParameters = true;

	/** If true, prevent the creation of clusters with only a single child. Either by merging into a neighboring cluster, or not creating the cluster. */
	UPROPERTY(EditAnywhere, Category = AutoCluster)
	bool AvoidIsolated = true;

	/** Fractured GeometryCollection to cluster */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Bone selection for the clustering */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection", DataflowIntrinsic))
	FDataflowTransformSelection TransformSelection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FAutoClusterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};


/**
 *
 * Flattens selected bones. If no selection is provided, flattens all bones to level 1
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FClusterFlattenDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FClusterFlattenDataflowNode, "Flatten", "GeometryCollection|Cluster", "")

public:

	/** Fractured GeometryCollection to flatten */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** If connected, clusters under the selected bones will be flattened. If no selection is provided, all bones will be flattened to level 1. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection OptionalTransformSelection;

	FClusterFlattenDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&OptionalTransformSelection);
		
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 * Uncluster selected nodes
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FClusterUnclusterDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FClusterUnclusterDataflowNode, "Uncluster", "GeometryCollection|Cluster", "")

public:

	/** Fractured GeometryCollection to uncluster */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Bone selection */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FClusterUnclusterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 * Cluster selected nodes under a new parent
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FClusterDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FClusterDataflowNode, "Cluster", "GeometryCollection|Cluster", "")

public:

	/** Collection on which to cluster nodes */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Bone selection */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FClusterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};



UENUM(BlueprintType)
enum class EClusterNeighborSelectionMethodEnum : uint8
{
	Dataflow_ClusterNeighborSelectionMethod_LargestNeighbor UMETA(DisplayName = "Largest Neighbor"),
	Dataflow_ClusterNeighborSelectionMethod_NearestCenter UMETA(DisplayName = "Nearest Center")
};

/**
 * Merge selected bones to their neighbors
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FClusterMergeToNeighborsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FClusterMergeToNeighborsDataflowNode, "ClusterMergeToNeighbors", "GeometryCollection|Cluster", "")

public:

	/** Collection on which to merge bones into a neighboring cluster */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Bone selection */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/** Method to choose which neighbor to merge */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Merge To"))
	EClusterNeighborSelectionMethodEnum NeighborSelectionMethod = EClusterNeighborSelectionMethodEnum::Dataflow_ClusterNeighborSelectionMethod_LargestNeighbor;

	/** Size (cube root of volume) of minimum desired post-merge clusters; if > 0, selected clusters may be merged multiple times until the cluster size is above this value */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, DisplayName = "Min Cluster Size", ClampMin = "0"))
	float MinVolumeCubeRoot = 0.0f;

	/** Whether to only allow clusters to merge if their bones are connected in the proximity graph */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	bool bOnlyToConnected = true;

	/** Whether to only allow clusters to merge if they have the same parent bone */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	bool bOnlySameParent = true;

	FClusterMergeToNeighborsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterInputConnection(&MinVolumeCubeRoot);
		RegisterInputConnection(&bOnlyToConnected);
		RegisterInputConnection(&bOnlySameParent);

		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 * Merge selected bones under a new parent cluster
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FClusterMergeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FClusterMergeDataflowNode, "ClusterMerge", "GeometryCollection|Cluster", "")

public:

	/** Collection on which to merge bones into a cluster */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Bone selection */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FClusterMergeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 * Add a single cluster to the Geometry Collection if it only has a single transform with no clusters
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FClusterIsolatedRootsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FClusterIsolatedRootsDataflowNode, "ClusterIsolatedRoots", "GeometryCollection|Cluster", "")

public:

	/** Collection to modify */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	FClusterIsolatedRootsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 * Cluster by grouping the selected bones with their adjacent, neighboring bones.
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FClusterMagnetDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FClusterMagnetDataflowNode, "ClusterMagnet", "GeometryCollection|Cluster", "")

public:

	/** Collection on which to merge bones into a cluster */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Bone selection */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/** How many layers of neighbors to include in the clusters -- i.e. if 1, only direct neighbors are clustered; if 2, neighbors of neighbors are included, etc. */
	UPROPERTY(EditAnywhere, Category = "Cluster Magnet", meta = (ClampMin = "1", DataflowInput, DisplayName = "Iterations"))
	int32 Iterations = 1;

	FClusterMagnetDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterInputConnection(&Iterations);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


namespace UE::Dataflow
{
	void GeometryCollectionClusteringNodes();
}

