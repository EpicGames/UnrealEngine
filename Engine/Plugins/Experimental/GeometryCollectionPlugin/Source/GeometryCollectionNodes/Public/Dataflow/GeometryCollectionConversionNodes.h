// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"

#include "GeometryCollectionConversionNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/**
 *
 * Description for this node
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FVectorToStringDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVectorToStringDataflowNode, "VectorToString", "Utilities|String", "")

public:
	UPROPERTY(EditAnywhere, Category = "Vector", meta = (DataflowInput))
	FVector Vector = FVector(0.0);

	UPROPERTY(meta = (DataflowOutput))
	FString String = FString("");

	FVectorToStringDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Vector);
		RegisterOutputConnection(&String);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Description for this node
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FFloatToStringDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFloatToStringDataflowNode, "FloatToString", "Utilities|String", "")

public:
	UPROPERTY(EditAnywhere, Category = "Float", meta = (DataflowInput))
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	FString String = FString("");

	FFloatToStringDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&String);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Converts an Int to a String
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FIntToStringDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FIntToStringDataflowNode, "IntToString", "Utilities|String", "")

public:
	UPROPERTY(EditAnywhere, Category = "Int", meta = (DataflowInput))
	int32 Int = 0;

	UPROPERTY(meta = (DataflowOutput))
	FString String = FString("");

	FIntToStringDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Int);
		RegisterOutputConnection(&String);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Converts a Bool to a String in a form of ("true", "false")
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FBoolToStringDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBoolToStringDataflowNode, "BoolToString", "Utilities|String", "")

public:
	UPROPERTY(EditAnywhere, Category = "Bool", meta = (DataflowInput))
	bool Bool = false;

	UPROPERTY(meta = (DataflowOutput))
	FString String = FString("");

	FBoolToStringDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Bool);
		RegisterOutputConnection(&String);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Converts an Int to a Float
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FIntToFloatDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FIntToFloatDataflowNode, "IntToFloat", "Math|Conversions", "")

public:
	UPROPERTY(EditAnywhere, Category = "Int", meta = (DataflowInput))
	int32 Int = 0;

	UPROPERTY(meta = (DataflowOutput))
	float Float = 0.f;

	FIntToFloatDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Int);
		RegisterOutputConnection(&Float);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Converts an Int to a Double
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FIntToDoubleDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FIntToDoubleDataflowNode, "IntToDouble", "Math|Conversions", "")

public:
	UPROPERTY(EditAnywhere, Category = "Int", meta = (DataflowInput))
	int32 Int = 0;

	UPROPERTY(meta = (DataflowOutput))
	double Double = 0.0;

	FIntToDoubleDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Int);
		RegisterOutputConnection(&Double);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Converts an Float to a Double
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FFloatToDoubleDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFloatToDoubleDataflowNode, "FloatToDouble", "Math|Conversions", "")

public:
	UPROPERTY(EditAnywhere, Category = "Int", meta = (DataflowInput))
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	double Double = 0.0;

	FFloatToDoubleDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&Double);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


UENUM(BlueprintType)
enum class EFloatToIntFunctionEnum : uint8
{
	Dataflow_FloatToInt_Function_Floor UMETA(DisplayName = "Floor()"),
	Dataflow_FloatToInt_Function_Ceil UMETA(DisplayName = "Ceil()"),
	Dataflow_FloatToInt_Function_Round UMETA(DisplayName = "Round()"),
	Dataflow_FloatToInt_Function_Truncate UMETA(DisplayName = "Truncate()"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 * Converts a Float to Int using the specified method
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FFloatToIntDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFloatToIntDataflowNode, "FloatToInt", "Math|Conversions", "")

public:
	/** Method to convert */
	UPROPERTY(EditAnywhere, Category = "Float");
	EFloatToIntFunctionEnum Function = EFloatToIntFunctionEnum::Dataflow_FloatToInt_Function_Round;

	/** Float value to convert */
	UPROPERTY(EditAnywhere, Category = "Float", meta = (DataflowInput))
	float Float = 0.f;

	/** Int output */
	UPROPERTY(meta = (DataflowOutput))
	int32 Int = 0;

	FFloatToIntDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&Int);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Converts an Int to a Bool
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FIntToBoolDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FIntToBoolDataflowNode, "IntToBool", "Math|Conversions", "")

public:
	UPROPERTY(EditAnywhere, Category = "Int", meta = (DataflowInput))
	int32 Int = 0;

	UPROPERTY(meta = (DataflowOutput))
	bool Bool = false;

	FIntToBoolDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Int);
		RegisterOutputConnection(&Bool);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Converts a Bool to an Int
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FBoolToIntDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBoolToIntDataflowNode, "BoolToInt", "Math|Conversions", "")

public:
	UPROPERTY(EditAnywhere, Category = "Bool", meta = (DataflowInput))
	bool Bool = false;

	UPROPERTY(meta = (DataflowOutput))
	int32 Int = 0;

	FBoolToIntDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Bool);
		RegisterOutputConnection(&Int);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

namespace UE::Dataflow
{
	void GeometryCollectionConversionNodes();
}
