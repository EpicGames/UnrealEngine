// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "UDynamicMesh.h"
#include "GeometryCollectionUtilityNodes.h"

#include "GeometryCollectionDebugNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

//~ Dataflow nodes that are useful for debugging a geometry collection-related dataflow graph


// Convert convex hulls on a geometry collection to a dynamic mesh
USTRUCT(meta = (DataflowGeometryCollection))
struct FConvexHullToMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FConvexHullToMeshDataflowNode, "Convex Hull to Mesh", "GeometryCollection|Mesh", "")

private:
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Optional transform selection to convert hulls from -- if not provided, all convex hulls will be converted. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection OptionalSelectionFilter;

	/** Whether to robustly extract valid/manifold meshes to represent the convex hulls. Note: Not necessary for simple visualization, but useful for downstream processing. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bUseRobustHulls = true;

	/** Single mesh aggregating all the convex hulls together */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

	/** Array of meshes for each convex hull found */
	UPROPERTY(meta = (DataflowOutput))
	TArray<TObjectPtr<UDynamicMesh>> Meshes;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FConvexHullToMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

// Convert a sphere covering, as generated by the 'protect negative space' option on the "Generate Cluster Convex Hull" nodes, to a dynamic mesh
USTRUCT(meta = (DataflowGeometryCollection))
struct FSphereCoveringToMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSphereCoveringToMeshDataflowNode, "Sphere Covering to Mesh", "GeometryCollection|Mesh", "")

public:
	// The sphere covering to convert to a mesh
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowSphereCovering SphereCovering;

	// Number of vertices to use along each axis when creating a mesh for each sphere
	UPROPERTY(EditAnywhere, Category = Options, meta = (ClampMin = 2))
	int32 VerticesAlongEachSide = 8;

	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

	FSphereCoveringToMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SphereCovering);
		RegisterOutputConnection(&Mesh);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

// Report the number of spheres in a sphere covering
USTRUCT(meta = (DataflowGeometryCollection))
struct FSphereCoveringCountSpheresNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSphereCoveringCountSpheresNode, "Get Sphere Covering Sphere Count", "SphereCovering", "")

public:
	// The sphere covering to evaluate
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowSphereCovering SphereCovering;

	// Number of spheres in the sphere covering
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSpheres = 0;

	FSphereCoveringCountSpheresNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SphereCovering);
		RegisterOutputConnection(&NumSpheres);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override
	{
		if (Out->IsA(&NumSpheres))
		{
			const FDataflowSphereCovering& InSphereCovering = GetValue(Context, &SphereCovering);
			SetValue(Context, InSphereCovering.Spheres.Num(), &NumSpheres);
		}
	}
};

// Convert a mesh to a string formatted as an OBJ file, which can be read by many external mesh viewers
USTRUCT(meta = (DataflowGeometryCollection))
struct FMeshToOBJStringDebugDataflowNode: public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMeshToOBJStringDebugDataflowNode, "Convert Mesh to OBJ String", "GeometryCollection|Development", "")

public:
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDynamicMesh> Mesh;

	// Whether to flip the orientation of the triangles in the OBJ output
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	bool bInvertFaces = false;

	// A string representing the input mesh in the OBJ file format
	UPROPERTY(meta = (DataflowOutput))
	FString StringOBJ;

	FMeshToOBJStringDebugDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Mesh);
		RegisterInputConnection(&bInvertFaces);
		RegisterOutputConnection(&StringOBJ);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

// Write a string to a file
USTRUCT(meta = (DataflowGeometryCollection))
struct FWriteStringToFile : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FWriteStringToFile, "Write String to File", "GeometryCollection|Development", "")

public:

	// Where file should be written on disk
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	FString FilePath;

	// Contents of the file to write
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	FString FileContents;

	FWriteStringToFile(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FileContents);
		RegisterInputConnection(&FilePath);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

// Convert a TArray<FShere> to a dynamic mesh
USTRUCT(meta = (DataflowGeometryCollection))
struct FSpheresToMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSpheresToMeshDataflowNode, "SpheresToMesh", "GeometryCollection|Mesh", "")

private:
	// Array of spheres to convert to a mesh/meshes
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FSphere> Spheres;

	// Number of vertices to use along each axis when creating a mesh for each sphere
	UPROPERTY(EditAnywhere, Category = Options, meta = (ClampMin = 2))
	int32 VerticesAlongEachSide = 4;

	/** Single mesh aggregating all the spheres together */
	UPROPERTY(meta = (DataflowOutput, DataflowRenderGroups = ""))
	TObjectPtr<UDynamicMesh> Mesh;

	/** Array of meshes for each spheres found */
	UPROPERTY(meta = (DataflowOutput))
	TArray<TObjectPtr<UDynamicMesh>> Meshes;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FSpheresToMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

};

namespace UE::Dataflow
{
	void GeometryCollectionDebugNodes();
}
