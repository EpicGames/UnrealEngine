// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowSelection.h"

#include "GeometryCollectionEditNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class FGeometryCollection;

/**
 *
 * Deletes selected bones from Collection. Empty clusters will be eliminated
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FPruneInCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FPruneInCollectionDataflowNode, "PruneInCollection", "GeometryCollection|Edit", "")

public:
	/** Fractured GeometryCollection to prune */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Transform selection for pruning */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection", DataflowIntrinsic))
	FDataflowTransformSelection TransformSelection;

	FPruneInCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


UENUM(BlueprintType)
enum class EVisibiltyOptionsEnum : uint8
{
	Dataflow_VisibilityOptions_Visible UMETA(DisplayName = "Visible"),
	Dataflow_VisibilityOptions_Invisible UMETA(DisplayName = "Hidden"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 * Sets all selected bone's visibilty to Visible/Hidden
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FSetVisibilityInCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetVisibilityInCollectionDataflowNode, "SetVisibilityInCollection", "GeometryCollection|Edit", "")

public:
	/** What to set the visibility of the selected bones */
	UPROPERTY(EditAnywhere, Category = "Visibility");
	EVisibiltyOptionsEnum Visibility = EVisibiltyOptionsEnum::Dataflow_VisibilityOptions_Invisible;

	/** Fractured GeometryCollection to set visibility */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Transform selection for setting visibility */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowTransformSelection TransformSelection;

	/** Face selection for setting visibility */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowFaceSelection FaceSelection;

	FSetVisibilityInCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterInputConnection(&FaceSelection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Merges selected bones into a single bone
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FMergeInCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMergeInCollectionDataflowNode, "MergeInCollection", "GeometryCollection|Edit", "")

public:
	/** Fractured GeometryCollection to merge */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Transform selection for merging */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection", DataflowIntrinsic))
	FDataflowTransformSelection TransformSelection;

	FMergeInCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


namespace UE::Dataflow
{
	void GeometryCollectionEditNodes();
}

