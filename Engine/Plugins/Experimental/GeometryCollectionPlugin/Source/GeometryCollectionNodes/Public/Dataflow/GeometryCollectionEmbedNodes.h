// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowSelection.h"

//#include "GeometryCollectionEmbedNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class FGeometryCollection;




namespace UE::Dataflow
{
	void GeometryCollectionEmbedNodes();
}

