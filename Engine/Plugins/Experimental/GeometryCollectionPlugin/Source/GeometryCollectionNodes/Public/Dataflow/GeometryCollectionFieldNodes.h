// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowSelection.h"
#include "Field/FieldSystemTypes.h"

#include "GeometryCollectionFieldNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/**
 *
 * 
 *
 */
UENUM(BlueprintType)
enum class EDataflowFieldFalloffType : uint8
{
	Dataflow_FieldFalloffType_None			UMETA(DisplayName = "None", ToolTip = "No falloff function is used"),
	Dataflow_FieldFalloffType_Linear		UMETA(DisplayName = "Linear", ToolTip = "The falloff function will be proportional to x"),
	Dataflow_FieldFalloffType_Inverse		UMETA(DisplayName = "Inverse", ToolTip = "The falloff function will be proportional to 1.0/x"),
	Dataflow_FieldFalloffType_Squared		UMETA(DisplayName = "Squared", ToolTip = "The falloff function will be proportional to x*x"),
	Dataflow_FieldFalloffType_Logarithmic	UMETA(DisplayName = "Logarithmic", ToolTip = "The falloff function will be proportional to log(x)"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};


/**
 *
 * RadialFalloff Field Dataflow node
 *
 */
USTRUCT()
struct FRadialFalloffFieldDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRadialFalloffFieldDataflowNode, "RadialFalloffField", "Fields", "")

public:
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector3f> SamplePositions;

	/**  */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVertexSelection SampleIndices;

	/**  */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FSphere Sphere = FSphere(ForceInit);

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	FVector Translation = FVector(0.f);

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float Magnitude = 1.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float MinRange = 0.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float MaxRange = 1.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float Default = 0.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field")
	EDataflowFieldFalloffType FalloffType = EDataflowFieldFalloffType::Dataflow_FieldFalloffType_Linear;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<float> FieldFloatResult;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> FieldRemap;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVertexSelection FieldSelectionMask;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePositions = 0;

	FRadialFalloffFieldDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SamplePositions);
		RegisterInputConnection(&SampleIndices);
		RegisterInputConnection(&Sphere);
		RegisterInputConnection(&Translation);
		RegisterInputConnection(&Magnitude);
		RegisterInputConnection(&MinRange);
		RegisterInputConnection(&MaxRange);
		RegisterInputConnection(&Default);
		RegisterOutputConnection(&FieldFloatResult);
		RegisterOutputConnection(&FieldRemap);
		RegisterOutputConnection(&FieldSelectionMask);
		RegisterOutputConnection(&NumSamplePositions);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * BoxFalloff Field Dataflow node
 *
 */
USTRUCT()
struct FBoxFalloffFieldDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBoxFalloffFieldDataflowNode, "BoxFalloffField", "Fields", "")

public:
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector3f> SamplePositions;

	/**  */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVertexSelection SampleIndices;

	/**  */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FBox Box = FBox(ForceInit);

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	FTransform Transform = FTransform::Identity;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field")
	float Magnitude = 1.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field")
	float MinRange = 0.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field")
	float MaxRange = 1.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field")
	float Default = 0.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field")
	EDataflowFieldFalloffType FalloffType = EDataflowFieldFalloffType::Dataflow_FieldFalloffType_Linear;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<float> FieldFloatResult;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> FieldRemap;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVertexSelection FieldSelectionMask;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePositions = 0;

	FBoxFalloffFieldDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SamplePositions);
		RegisterInputConnection(&SampleIndices);
		RegisterInputConnection(&Box);
		RegisterInputConnection(&Transform);
		RegisterInputConnection(&Magnitude);
		RegisterInputConnection(&MinRange);
		RegisterInputConnection(&MaxRange);
		RegisterInputConnection(&Default);
		RegisterOutputConnection(&FieldFloatResult);
		RegisterOutputConnection(&FieldRemap);
		RegisterOutputConnection(&FieldSelectionMask);
		RegisterOutputConnection(&NumSamplePositions);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * PlaneFalloff Field Dataflow node
 *
 */
USTRUCT()
struct FPlaneFalloffFieldDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FPlaneFalloffFieldDataflowNode, "PlaneFalloffField", "Fields", "")

public:
	/**  */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector3f> SamplePositions;

	/**  */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVertexSelection SampleIndices;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	FVector Position = FVector(0.f);

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	FVector Normal = FVector(0.f, 0.f, 1.f);

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float Distance = 0.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	FVector Translation = FVector(0.f);

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float Magnitude = 1.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float MinRange = 0.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float MaxRange = 1.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float Default = 0.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field")
	EDataflowFieldFalloffType FalloffType = EDataflowFieldFalloffType::Dataflow_FieldFalloffType_Linear;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<float> FieldFloatResult;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> FieldRemap;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVertexSelection FieldSelectionMask;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePositions = 0;

	FPlaneFalloffFieldDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SamplePositions);
		RegisterInputConnection(&SampleIndices);
		RegisterInputConnection(&Position);
		RegisterInputConnection(&Normal);
		RegisterInputConnection(&Distance);
		RegisterInputConnection(&Translation);
		RegisterInputConnection(&Magnitude);
		RegisterInputConnection(&MinRange);
		RegisterInputConnection(&MaxRange);
		RegisterInputConnection(&Default);
		RegisterOutputConnection(&FieldFloatResult);
		RegisterOutputConnection(&FieldRemap);
		RegisterOutputConnection(&FieldSelectionMask);
		RegisterOutputConnection(&NumSamplePositions);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 *
 *
 */
UENUM(BlueprintType)
enum class EDataflowSetMaskConditionType : uint8
{
	Dataflow_SetMaskConditionType_Always			UMETA(DisplayName = "Set Always", ToolTip = "The particle output value will be equal to Interior-value if the particle position is inside a sphere / Exterior-value otherwise."),
	Dataflow_SetMaskConditionType_IFF_NOT_Interior	UMETA(DisplayName = "Merge Interior", ToolTip = "The particle output value will be equal to Interior-value if the particle position is inside the sphere or if the particle input value is already Interior-Value / Exterior-value otherwise."),
	Dataflow_SetMaskConditionType_IFF_NOT_Exterior  UMETA(DisplayName = "Merge Exterior", ToolTip = "The particle output value will be equal to Exterior-value if the particle position is outside the sphere or if the particle input value is already Exterior-Value / Interior-value otherwise."),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 * RadialIntMask Field Dataflow node
 *
 */
USTRUCT()
struct FRadialIntMaskFieldDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRadialIntMaskFieldDataflowNode, "RadialIntMaskField", "Fields", "")

public:
	/**  */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector3f> SamplePositions;

	/**  */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVertexSelection SampleIndices;

	/**  */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FSphere Sphere = FSphere(ForceInit);

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	FVector Translation = FVector(0.f);

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	int32 InteriorValue = 1;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	int32 ExteriorValue = 0;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field")
	EDataflowSetMaskConditionType SetMaskConditionType = EDataflowSetMaskConditionType::Dataflow_SetMaskConditionType_Always;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> FieldIntResult;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> FieldRemap;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePositions = 0;

	FRadialIntMaskFieldDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SamplePositions);
		RegisterInputConnection(&SampleIndices);
		RegisterInputConnection(&Sphere);
		RegisterInputConnection(&Translation);
		RegisterInputConnection(&InteriorValue);
		RegisterInputConnection(&ExteriorValue);
		RegisterOutputConnection(&FieldIntResult);
		RegisterOutputConnection(&FieldRemap);
		RegisterOutputConnection(&NumSamplePositions);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * UniformScalar Field Dataflow node
 *
 */
USTRUCT()
struct FUniformScalarFieldDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FUniformScalarFieldDataflowNode, "UniformScalarField", "Fields", "")

public:
	/**  */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector3f> SamplePositions;

	/**  */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVertexSelection SampleIndices;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float Magnitude = 1.f;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<float> FieldFloatResult;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> FieldRemap;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePositions = 0;

	FUniformScalarFieldDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SamplePositions);
		RegisterInputConnection(&SampleIndices);
		RegisterInputConnection(&Magnitude);
		RegisterOutputConnection(&FieldFloatResult);
		RegisterOutputConnection(&FieldRemap);
		RegisterOutputConnection(&NumSamplePositions);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * UniformVector Field Dataflow node
 *
 */
USTRUCT()
struct FUniformVectorFieldDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FUniformVectorFieldDataflowNode, "UniformVectorField", "Fields", "")

public:
	/**  */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector3f> SamplePositions;

	/**  */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVertexSelection SampleIndices;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float Magnitude = 1.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	FVector Direction = FVector(1.f, 0.f, 0.f);

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> FieldVectorResult;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> FieldRemap;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePositions = 0;

	FUniformVectorFieldDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SamplePositions);
		RegisterInputConnection(&SampleIndices);
		RegisterInputConnection(&Magnitude);
		RegisterInputConnection(&Direction);
		RegisterOutputConnection(&FieldVectorResult);
		RegisterOutputConnection(&FieldRemap);
		RegisterOutputConnection(&NumSamplePositions);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * RadialVector Field Dataflow node
 *
 */
USTRUCT()
struct FRadialVectorFieldDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRadialVectorFieldDataflowNode, "RadialVectorField", "Fields", "")

public:
	/**  */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector3f> SamplePositions;

	/**  */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVertexSelection SampleIndices;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float Magnitude = 1.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	FVector Position = FVector(0.f);

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> FieldVectorResult;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> FieldRemap;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePositions = 0;

	FRadialVectorFieldDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SamplePositions);
		RegisterInputConnection(&SampleIndices);
		RegisterInputConnection(&Magnitude);
		RegisterInputConnection(&Position);
		RegisterOutputConnection(&FieldVectorResult);
		RegisterOutputConnection(&FieldRemap);
		RegisterOutputConnection(&NumSamplePositions);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * RandomVector Field Dataflow node
 *
 */
USTRUCT()
struct FRandomVectorFieldDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRandomVectorFieldDataflowNode, "RandomVectorField", "Fields", "")

public:
	/**  */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector3f> SamplePositions;

	/**  */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVertexSelection SampleIndices;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float Magnitude = 1.f;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> FieldVectorResult;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> FieldRemap;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePositions = 0;

	FRandomVectorFieldDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SamplePositions);
		RegisterInputConnection(&SampleIndices);
		RegisterInputConnection(&Magnitude);
		RegisterOutputConnection(&FieldVectorResult);
		RegisterOutputConnection(&FieldRemap);
		RegisterOutputConnection(&NumSamplePositions);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Noise Field Dataflow node
 *
 */
USTRUCT()
struct FNoiseFieldDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FNoiseFieldDataflowNode, "NoiseField", "Fields", "")

public:
	/**  */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector3f> SamplePositions;

	/**  */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVertexSelection SampleIndices;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float MinRange = 0.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float MaxRange = 1.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	FTransform Transform = FTransform::Identity;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<float> FieldFloatResult;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> FieldRemap;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePositions = 0;

	FNoiseFieldDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SamplePositions);
		RegisterInputConnection(&SampleIndices);
		RegisterInputConnection(&MinRange);
		RegisterInputConnection(&MaxRange);
		RegisterInputConnection(&Transform);
		RegisterOutputConnection(&FieldFloatResult);
		RegisterOutputConnection(&FieldRemap);
		RegisterOutputConnection(&NumSamplePositions);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * UniformInteger Field Dataflow node
 *
 */
USTRUCT()
struct FUniformIntegerFieldDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FUniformIntegerFieldDataflowNode, "UniformIntegerField", "Fields", "")

public:
	/**  */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector3f> SamplePositions;

	/**  */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVertexSelection SampleIndices;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	int32 Magnitude = 0;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> FieldIntResult;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> FieldRemap;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePositions = 0;

	FUniformIntegerFieldDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SamplePositions);
		RegisterInputConnection(&SampleIndices);
		RegisterInputConnection(&Magnitude);
		RegisterOutputConnection(&FieldIntResult);
		RegisterOutputConnection(&FieldRemap);
		RegisterOutputConnection(&NumSamplePositions);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 *
 *
 */
UENUM(BlueprintType)
enum class EDataflowWaveFunctionType : uint8
{
	Dataflow_WaveFunctionType_Cosine	UMETA(DisplayName = "Cosine", ToolTip = "Cosine wave that will move in time."),
	Dataflow_WaveFunctionType_Gaussian  UMETA(DisplayName = "Gaussian", ToolTip = "Gaussian wave that will move in time."),
	Dataflow_WaveFunctionType_Falloff	UMETA(DisplayName = "Falloff", ToolTip = "The radial falloff radius will move along temporal wave."),
	Dataflow_WaveFunctionType_Decay		UMETA(DisplayName = "Decay", ToolTip = "The magnitude of the field will decay in time."),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 * WaveScalar Field Dataflow node v2
 *
 */
USTRUCT()
struct FWaveScalarFieldDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FWaveScalarFieldDataflowNode, "WaveScalarField", "Fields", "")

public:
	/**  */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector3f> SamplePositions;

	/**  */
	UPROPERTY(meta = (DataflowInput))
	FDataflowVertexSelection SampleIndices;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float Magnitude = 1.f;

	/**  */
	UPROPERTY(meta = (DataflowInput))
	FVector Position = FVector(0.f);

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	FVector Translation = FVector(0.f);

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float Wavelength = 1000.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float Period = 1.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field")
	EDataflowWaveFunctionType FunctionType = EDataflowWaveFunctionType::Dataflow_WaveFunctionType_Cosine;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field")
	EDataflowFieldFalloffType FalloffType = EDataflowFieldFalloffType::Dataflow_FieldFalloffType_Linear;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<float> FieldFloatResult;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> FieldRemap;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePositions = 0;

	FWaveScalarFieldDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SamplePositions);
		RegisterInputConnection(&SampleIndices);
		RegisterInputConnection(&Magnitude);
		RegisterInputConnection(&Position);
		RegisterInputConnection(&Translation);
		RegisterInputConnection(&Wavelength);
		RegisterInputConnection(&Period);
		RegisterOutputConnection(&FieldFloatResult);
		RegisterOutputConnection(&FieldRemap);
		RegisterOutputConnection(&NumSamplePositions);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 *
 *
 */
UENUM(BlueprintType)
enum class EDataflowFloatFieldOperationType : uint8
{
	Dataflow_FloatFieldOperationType_Multiply	UMETA(DisplayName = "Multiply", ToolTip = "Multiply the fields output values : Output = Left * Right"),
	Dataflow_FloatFieldFalloffType_Divide		UMETA(DisplayName = "Divide", ToolTip = "Divide the fields output values : Output = Left / Right"),
	Dataflow_FloatFieldFalloffType_Add			UMETA(DisplayName = "Add", ToolTip = "Add the fields output values : Output = Left + Right"),
	Dataflow_FloatFieldFalloffType_Substract	UMETA(DisplayName = "Subtract", ToolTip = "Subtract the fields output : Output = Left - Right"),
	Dataflow_FloatFieldFalloffType_Min			UMETA(DisplayName = "Min", ToolTip = "Min of the fields output values: Output = Min(Left, Right)"),
	Dataflow_FloatFieldFalloffType_Max			UMETA(DisplayName = "Max", ToolTip = "Max of the fields output values: Output = Max(Left, Right)"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};


/**
 *
 *
 *
 */
USTRUCT()
struct FSumScalarFieldDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSumScalarFieldDataflowNode, "SumScalarField", "Fields", "")

public:
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<float> FieldFloatLeft;

	/**  */
	UPROPERTY(meta = (DataflowInput))
	TArray<int32> FieldRemapLeft;

	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<float> FieldFloatRight;

	/**  */
	UPROPERTY(meta = (DataflowInput))
	TArray<int32> FieldRemapRight;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field")
	float Magnitude = 1.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field")
	EDataflowFloatFieldOperationType Operation = EDataflowFloatFieldOperationType::Dataflow_FloatFieldFalloffType_Add;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field")
	bool bSwapInputs = false;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<float> FieldFloatResult;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> FieldRemap;

	FSumScalarFieldDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FieldFloatLeft);
		RegisterInputConnection(&FieldRemapLeft);
		RegisterInputConnection(&FieldFloatRight);
		RegisterInputConnection(&FieldRemapRight);
		RegisterOutputConnection(&FieldFloatResult);
		RegisterOutputConnection(&FieldRemap);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 *
 *
 */
UENUM(BlueprintType)
enum class EDataflowVectorFieldOperationType : uint8
{
	Dataflow_VectorFieldOperationType_Multiply		UMETA(DisplayName = "Multiply", ToolTip = "Multiply the fields output values : Output = Left * Right"),
	Dataflow_VectorFieldFalloffType_Divide			UMETA(DisplayName = "Divide", ToolTip = "Divide the fields output values : Output = Left / Right"),
	Dataflow_VectorFieldFalloffType_Add				UMETA(DisplayName = "Add", ToolTip = "Add the fields output values : Output = Left + Right"),
	Dataflow_VectorFieldFalloffType_Substract		UMETA(DisplayName = "Subtract", ToolTip = "Subtract the fields output : Output = Left - Right"),
	Dataflow_VectorFieldFalloffType_CrossProduct	UMETA(DisplayName = "Cross product", ToolTip = "Cross product of the fields output values: Output = Left x Right"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 *
 *
 */
USTRUCT()
struct FSumVectorFieldDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSumVectorFieldDataflowNode, "SumVectorField", "Fields", "")

public:
	UPROPERTY(meta = (DataflowInput))
	TArray<float> FieldFloat;

	/**  */
	UPROPERTY(meta = (DataflowInput))
	TArray<int32> FieldFloatRemap;

	UPROPERTY(meta = (DataflowInput))
	TArray<FVector> FieldVectorLeft;

	/**  */
	UPROPERTY(meta = (DataflowInput))
	TArray<int32> FieldRemapLeft;

	UPROPERTY(meta = (DataflowInput))
	TArray<FVector> FieldVectorRight;

	/**  */
	UPROPERTY(meta = (DataflowInput))
	TArray<int32> FieldRemapRight;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput))
	float Magnitude = 1.f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field")
	EDataflowVectorFieldOperationType Operation = EDataflowVectorFieldOperationType::Dataflow_VectorFieldFalloffType_Add;

	UPROPERTY(EditAnywhere, Category = "Field")
	bool bSwapVectorInputs = false;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> FieldVectorResult;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> FieldRemap;

	FSumVectorFieldDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FieldFloat);
		RegisterInputConnection(&FieldFloatRemap);
		RegisterInputConnection(&FieldVectorLeft);
		RegisterInputConnection(&FieldRemapLeft);
		RegisterInputConnection(&FieldVectorRight);
		RegisterInputConnection(&FieldRemapRight);
		RegisterInputConnection(&Magnitude);
		RegisterOutputConnection(&FieldVectorResult);
		RegisterOutputConnection(&FieldRemap);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Converts a sparse FloatArray (a selected subset of the whole incoming array) into a dense FloatArray
 * (same number of elements as the incoming array using NumSamplePositions) using the Remap input
 * NumSamplePositions controls the size of the output array, only indices smaller than l to than NumSamplePositions
 * will be processed
 *
 */
USTRUCT()
struct FFieldMakeDenseFloatArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFieldMakeDenseFloatArrayDataflowNode, "FieldMakeDenseFloatArray", "Fields", "")

public:
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<float> FieldFloatInput;

	/**  */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<int32> FieldRemap;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (DataflowInput, DisplayName = "Number of Sample Positions"))
	int32 NumSamplePositions = 0;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Field")
	float Default = 0.f;

	/**  */
	UPROPERTY(meta = (DataflowOutput))
	TArray<float> FieldFloatResult;

	FFieldMakeDenseFloatArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FieldFloatInput);
		RegisterInputConnection(&FieldRemap);
		RegisterInputConnection(&NumSamplePositions);
		RegisterOutputConnection(&FieldFloatResult);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

namespace UE::Dataflow
{
	void GeometryCollectionFieldNodes();
}
