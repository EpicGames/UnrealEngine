// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowSelection.h"
#include "FractureEngineFracturing.h"
#include "Dataflow/DataflowDebugDraw.h"

#include "GeometryCollectionFracturingNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class FGeometryCollection;
class UStaticMesh;
class UDynamicMesh;

USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.5"))
struct FUniformScatterPointsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FUniformScatterPointsDataflowNode, "UniformScatterPoints", "Generators|Point", "")

public:
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = 1));
	int32 MinNumberOfPoints = 20;

	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = 1));
	int32 MaxNumberOfPoints = 20;

	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput));
	float RandomSeed = -1.f;

	UPROPERTY(meta = (DataflowInput))
	FBox BoundingBox = FBox(ForceInit);

	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> Points;

	FUniformScatterPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&BoundingBox);
		RegisterInputConnection(&MinNumberOfPoints);
		RegisterInputConnection(&MaxNumberOfPoints);
		RegisterInputConnection(&RandomSeed);
		RegisterOutputConnection(&Points);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

USTRUCT(meta = (DataflowGeometryCollection))
struct FUniformScatterPointsDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FUniformScatterPointsDataflowNode_v2, "UniformScatterPoints", "Generators|Point", "")

public:
	/** Minimum for the random range */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = 1));
	int32 MinNumberOfPoints = 20;

	/** Maximum for the random range */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = 1));
	int32 MaxNumberOfPoints = 20;

	/** Seed for random */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = "0"));
	int32 RandomSeed = 0;

	/** BoundingBox to generate points inside of */
	UPROPERTY(meta = (DataflowInput))
	FBox BoundingBox = FBox(ForceInit);

	/** Generated points */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> Points;

	FUniformScatterPointsDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&BoundingBox);
		RegisterInputConnection(&MinNumberOfPoints) .SetCanHidePin(true) .SetPinIsHidden(true);
		RegisterInputConnection(&MaxNumberOfPoints) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&RandomSeed) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterOutputConnection(&Points);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

USTRUCT(meta = (DataflowGeometryCollection))
struct FClusterScatterPointsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FClusterScatterPointsDataflowNode, "ClusterScatterPoints", "Generators|Point", "")

private:

	/** Minimum number of clusters of points to create. The amount of clusters created will be chosen at random between Min and Max */
	UPROPERTY(EditAnywhere, Category = Scatter, meta = (DisplayName = "Min Num Clusters", UIMax = "200", ClampMin = "1", DataflowInput))
	int32 NumberClustersMin = 8;

	/** Maximum number of clusters of points to create. The amount of clusters created will be chosen at random between Min and Max */
	UPROPERTY(EditAnywhere, Category = Scatter, meta = (DisplayName = "Max Num Clusters", UIMax = "200", ClampMin = "1", DataflowInput))
	int32 NumberClustersMax = 8;

	/** Minimum number of points per cluster. The amount of points in each cluster will be chosen at random between Min and Max */
	UPROPERTY(EditAnywhere, Category = Scatter, meta = (DisplayName = "Min Points Per Cluster", UIMax = "200", ClampMin = "0", DataflowInput))
	int32 PointsPerClusterMin = 2;

	/** Maximum number of points per cluster. The amount of points in each cluster will be chosen at random between Min and Max */
	UPROPERTY(EditAnywhere, Category = Scatter, meta = (DisplayName = "Max Points Per Cluster", UIMax = "200", ClampMin = "0", DataflowInput))
	int32 PointsPerClusterMax = 30;

	/**
	 * Minimum cluster radius (as fraction of the overall bounding box size). Cluster Radius Offset will be added to this.
	 * Each point will be placed at least this far (plus the Cluster Radius Offset) from its cluster center.
	 */
	UPROPERTY(EditAnywhere, Category = Scatter, meta = (DisplayName = "Min Dist from Center (as Frac of Bounds)", ClampMin = "0.0", UIMax = "1.0", DataflowInput))
	float ClusterRadiusFractionMin = .1f;

	/**
	 * Maximum cluster radius (as fraction of the overall bounding box size). Cluster Radius Offset will be added to this.
	 * Each point will be placed at most this far (plus the Cluster Radius Offset) from its cluster center.
	 */
	UPROPERTY(EditAnywhere, Category = Scatter, meta = (DisplayName = "Max Dist from Center (as Frac of Bounds)", ClampMin = "0.0", UIMax = "1.0", DataflowInput))
	float ClusterRadiusFractionMax = .2f;

	/** Cluster radius offset (in cm). This offset will be added to the 'Min/Max Dist from Center' distance */
	UPROPERTY(EditAnywhere, Category = Scatter, meta = (Units = "cm", DataflowInput))
	float ClusterRadiusOffset = 0;

	/** Seed for random */
	UPROPERTY(EditAnywhere, Category = Scatter, meta = (DataflowInput, UIMin = "0"));
	int32 RandomSeed = 0;

	/** BoundingBox to generate points inside of */
	UPROPERTY(meta = (DataflowInput))
	FBox BoundingBox = FBox(ForceInit);

	/** Generated points */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> Points;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FClusterScatterPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.5"))
struct FRadialScatterPointsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRadialScatterPointsDataflowNode, "RadialScatterPoints", "Generators|Point", "")

public:
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput));
	FVector Center = FVector(0.0);

	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput));
	FVector Normal = FVector(0.0, 0.0, 1.0);

	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = 0.01f));
	float Radius = 50.f;

	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = 1, UIMax = 50));
	int32 AngularSteps = 5;

	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = 1, UIMax = 50));
	int32 RadialSteps = 5;

	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput));
	float AngleOffset = 0.f;

	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = 0.f));
	float Variability = 0.f;

	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput));
	float RandomSeed = -1.f;

	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> Points;

	FRadialScatterPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Center);
		RegisterInputConnection(&Normal);
		RegisterInputConnection(&Radius);
		RegisterInputConnection(&AngularSteps);
		RegisterInputConnection(&RadialSteps);
		RegisterInputConnection(&AngleOffset);
		RegisterInputConnection(&Variability);
		RegisterInputConnection(&RandomSeed);
		RegisterOutputConnection(&Points);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

USTRUCT(meta = (DataflowGeometryCollection))
struct FRadialScatterPointsDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRadialScatterPointsDataflowNode_v2, "RadialScatterPoints", "Generators|Point", "")

public:
	/** BoundingBox to generate points inside of */
	UPROPERTY(meta = (DataflowInput))
	FBox BoundingBox = FBox(ForceInit);

	/** Center of generated pattern */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput));
	FVector Center = FVector(0.0);

	/** Normal to plane in which sites are generated */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput));
	FVector Normal = FVector(0.0, 0.0, 1.0);

	/** Seed for random */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = "0"));
	int32 RandomSeed = 0;

	/** Number of angular steps */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = 1, UIMax = 50, ClampMin = "1"));
	int32 AngularSteps = 5;

	/** Angle offset at each radial step (in degrees) */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput));
	float AngleOffset = 0.f;

	/** Amount of global variation to apply to each angular step (in degrees) */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput));
	float AngularNoise = 0.f;

	/** Pattern radius (in cm) */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = "0.0", ClampMin = "0.0"));
	float Radius = 50.f;

	/** Number of radial steps */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = "1", UIMax = "50", ClampMin = "1"));
	int32 RadialSteps = 5;

	/** Radial steps will follow a distribution based on this exponent, i.e., Pow(distance from center, RadialStepExponent) */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = ".01", UIMax = "10", ClampMin = ".01", ClampMax = "20"));
	float RadialStepExponent = 1.f;

	/** Minimum radial separation between any two voronoi points (in cm) */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = ".25", ClampMin = ".01"))
	float RadialMinStep = 1.f;

	/** Amount of global variation to apply to each radial step (in cm) */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, ClampMin = "0"))
	float RadialNoise = 0.f;

	/** Amount to randomly displace each Voronoi site radially (in cm) */
	UPROPERTY(EditAnywhere, Category = "Per Point Variability", meta = (DataflowInput, ClampMin = "0.0"))
	float RadialVariability = 0.f;

	/** Amount to randomly displace each Voronoi site in angle (in degrees) */
	UPROPERTY(EditAnywhere, Category = "Per Point Variability", meta = (DataflowInput, ClampMin = "0.0"))
	float AngularVariability = 0.f;

	/** Amount to randomly displace each Voronoi site in the direction of the rotation axis (in cm) */
	UPROPERTY(EditAnywhere, Category = "Per Point Variability", meta = (DataflowInput, ClampMin = "0.0"))
	float AxialVariability = 0.f;

	/** Generated points */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> Points;

	FRadialScatterPointsDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&BoundingBox);
		RegisterInputConnection(&Center);
		RegisterInputConnection(&Normal) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&RandomSeed) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&AngularSteps) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&AngleOffset) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&AngularNoise) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Radius) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&RadialSteps) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&RadialStepExponent) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&RadialMinStep) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&RadialNoise) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&RadialVariability) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&AngularVariability) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&AxialVariability) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterOutputConnection(&Points);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

USTRUCT(meta = (DataflowGeometryCollection))
struct FGridScatterPointsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGridScatterPointsDataflowNode, "GridScatterPoints", "Generators|Point", "")

public:
	/** Number of points in X direction */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, ClampMin = "0", UIMax = "100", ClampMax = "200"));
	int32 NumberOfPointsInX = 5;

	/** Number of points in Y direction */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, ClampMin = "0", UIMax = "100", ClampMax = "200"));
	int32 NumberOfPointsInY = 5;

	/** Number of points in Z direction */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, ClampMin = "0", UIMax = "100", ClampMax = "200"));
	int32 NumberOfPointsInZ = 5;

	/** Seed for random */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = "0"));
	int32 RandomSeed = 0;

	/** Random displacement in X direction will be in the range (-MaxRandomDisplacementX, MaxRandomDisplacementX) */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = "0.0"));
	float MaxRandomDisplacementX = 0.f;

	/** Random displacement in Y direction will be in the range (-MaxRandomDisplacementY, MaxRandomDisplacementY) */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = "0.0"));
	float MaxRandomDisplacementY = 0.f;

	/** Random displacement in Z direction will be in the range (-MaxRandomDisplacementZ, MaxRandomDisplacementZ) */
	UPROPERTY(EditAnywhere, Category = "Scatter", meta = (DataflowInput, UIMin = "0.0"));
	float MaxRandomDisplacementZ = 0.f;

	/** BoundingBox to generate points inside of */
	UPROPERTY(meta = (DataflowInput))
	FBox BoundingBox = FBox(ForceInit);

	/** Generated points */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> Points;

	FGridScatterPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&BoundingBox);
		RegisterInputConnection(&NumberOfPointsInX) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&NumberOfPointsInY) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&NumberOfPointsInZ) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&RandomSeed) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&MaxRandomDisplacementX) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&MaxRandomDisplacementY) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&MaxRandomDisplacementZ) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterOutputConnection(&Points);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
* Transform an array of points
*/
USTRUCT(meta = (DataflowGeometryCollection))
struct FTransformPointsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FTransformPointsDataflowNode, "TransformPoints", "Generators|Point", "")

private:
	UPROPERTY(meta = (DataflowOutput, DataflowInput, DataflowPassthrough = Points, DataflowIntrinsic))
	TArray<FVector> Points;

	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DataflowInput));
	FTransform Transform = FTransform::Identity;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FTransformPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/** 
* Combine two arrays of points into one
*/
USTRUCT(meta = (DataflowGeometryCollection))
struct FAppendPointsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAppendPointsDataflowNode, "AppendPoints", "Generators|Point", "")

private:
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector> PointsA;

	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector> PointsB;

	UPROPERTY(meta = (DataflowOutput, DataflowPassthrough = PointsA))
	TArray<FVector> Points;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FAppendPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Voronoi fracture
 * Fracture using a Voronoi diagram with a uniform random pattern, creating fracture pieces of similar volume across the shape.
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.5"))
struct FVoronoiFractureDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVoronoiFractureDataflowNode, "VoronoiFracture", "GeometryCollection|Fracture", "")

public:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector> Points;

	/**   */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection", DataflowIntrinsic))
	FDataflowTransformSelection TransformSelection;

	UPROPERTY(EditAnywhere, Category = "Fracture", meta = (DataflowInput));
	float RandomSeed = -1.f;

	UPROPERTY(EditAnywhere, Category = "Fracture", meta = (DataflowInput, UIMin = 0.f, UIMax = 1.f));
	float ChanceToFracture = 1.f;

	UPROPERTY(EditAnywhere, Category = "Fracture");
	bool GroupFracture = true;

	UPROPERTY(EditAnywhere, Category = "Fracture", meta = (DataflowInput, UIMin = 0.f));
	float Grout = 0.f;

	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Amplitude = 0.f;

	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.00001f));
	float Frequency = 0.1f;

	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Persistence = 0.5f;

	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Lacunarity = 2.f;

	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	int32 OctaveNumber = 4;

	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float PointSpacing = 10.f;

	UPROPERTY(EditAnywhere, Category = "Collision");
	bool AddSamplesForCollision = false;

	UPROPERTY(EditAnywhere, Category = "Collision", meta = (DataflowInput, UIMin = 0.f));
	float CollisionSampleSpacing = 50.f;

	FVoronoiFractureDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&Points);
		RegisterInputConnection(&TransformSelection);
		RegisterInputConnection(&RandomSeed);
		RegisterInputConnection(&ChanceToFracture);
		RegisterInputConnection(&Grout);
		RegisterInputConnection(&Amplitude);
		RegisterInputConnection(&Frequency);
		RegisterInputConnection(&Persistence);
		RegisterInputConnection(&Lacunarity);
		RegisterInputConnection(&OctaveNumber);
		RegisterInputConnection(&PointSpacing);
		RegisterInputConnection(&CollisionSampleSpacing);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Voronoi fracture
 * Fracture using a Voronoi diagram with a uniform random pattern, creating fracture pieces of similar volume across the shape.
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FVoronoiFractureDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVoronoiFractureDataflowNode_v2, "VoronoiFracture", "GeometryCollection|Fracture", "")

public:
	/** Collection to fracture */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Voronoi source points */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector> Points;

	/** Pieces to fracture */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "TransformSelection", DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/** Transform to apply to cut planes */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Transform"))
	FTransform Transform = FTransform::Identity;

	// Made it hidden to hide this parameter, the random seed for Voronoi fracturing is on the point generation nodes
	/** Seed for random */
	UPROPERTY();
	int32 RandomSeed = 0;

	/** Chance to fracture each selected bone. If 0, no bones will fracture; if 1, all bones will fracture. */
	UPROPERTY(EditAnywhere, Category = "Common Fracture", meta = (DataflowInput, UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"));
	float ChanceToFracture = 1.f;

	/** Whether to split the fractured mesh pieces into separate islands based on geometric connectivity after fracturing */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands");
	bool SplitIslands = true;

	/** Distance threshold for connecting spatially-close vertices before computing islands */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands", meta = (EditCondition = "SplitIslands", Units = "cm", ClampMin = "0.0"));
	float CloseVertexDistance = 0.001f;

	/** If > 0, bridge separate islands whose surfaces are within this vertex-to-triangle distance. 0 = disabled. */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands", meta = (EditCondition = "SplitIslands", Units = "cm", ClampMin = "0.0"));
	float VertexToSurfaceBridgeDistance = 0.f;

	/** Amount of space to leave between cut pieces */
	UPROPERTY(EditAnywhere, Category = "Common Fracture", meta = (DataflowInput, UIMin = 0.f));
	float Grout = 0.f;

	/** Size of the Perlin noise displacement (in cm). If 0, no noise will be applied */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Amplitude = 0.f;

	/** Period of the Perlin noise.  Smaller values will create a smoother noise pattern */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.00001f));
	float Frequency = 0.1f;

	/** Persistence of the layers of Perlin noise. At each layer (octave) after the first, the amplitude of the Perlin noise is scaled by this factor */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Persistence = 0.5f;

	/** Lacunarity of the layers of Perlin noise. At each layer (octave) after the first, the frequency of the Perlin noise is scaled by this factor */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Lacunarity = 2.f;

	/**
	 * Number of fractal layers of Perlin noise to apply. Each layer is additive, with Amplitude and Frequency parameters scaled by Persistence and Lacunarity
	 * Smaller values (1 or 2) will create noise that looks like gentle rolling hills, while larger values (> 4) will tend to look more like craggy mountains
	 */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	int32 OctaveNumber = 4;

	/** Distance (in cm) between vertices on cut surfaces where noise is added.  Larger spacing between vertices will create more efficient meshes with fewer triangles, but less resolution to see the shape of the added noise  */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float PointSpacing = 10.f;

	/**
	 * If enabled, add extra vertices (without triangles) to the geometry in regions where vertices are spaced too far apart (e.g. across large triangles)
	 * These extra vertices will be used as collision samples in particle-implicit collisions, and can help the physics system detect collisions more accurately
	 *
	 * Note this is *only* useful for simulations that use particle-implicit collisions
	 */
	UPROPERTY(EditAnywhere, Category = "Collision");
	bool AddSamplesForCollision = false;

	/**
	 * The number of centimeters to allow between vertices on the mesh surface: If there are gaps larger than this, add additional vertices (without triangles) to help support particle-implicit collisions
	 * Only used if Add Samples For Collision is enabled
	 */
	UPROPERTY(EditAnywhere, Category = "Collision", meta = (DataflowInput, UIMin = 0.f, EditCondition = "AddSamplesForCollision"));
	float CollisionSampleSpacing = 50.f;

	/** Fractured Pieces */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "FracturedTransformSelection "))
	FDataflowTransformSelection NewGeometryTransformSelection;

	FVoronoiFractureDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&Points);
		RegisterInputConnection(&TransformSelection);
		RegisterInputConnection(&Transform) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&ChanceToFracture) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Grout) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Amplitude) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Frequency) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Persistence) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Lacunarity) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&OctaveNumber) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&PointSpacing) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&CollisionSampleSpacing) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection, &TransformSelection);
		RegisterOutputConnection(&NewGeometryTransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Editor Fracture Mode / Fracture / Planar tool
 * Fracture using a set of noised up planes.
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.5"))
struct FPlaneCutterDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FPlaneCutterDataflowNode, "PlaneCutter", "GeometryCollection|Fracture", "")

public:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FBox BoundingBox = FBox(ForceInit);

	/**   */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection", DataflowIntrinsic))
	FDataflowTransformSelection TransformSelection;

	UPROPERTY(EditAnywhere, Category = "Fracture", meta = (DataflowInput, UIMin = 1))
	int32 NumPlanes = 1;

	UPROPERTY(EditAnywhere, Category = "Fracture", meta = (DataflowInput))
	float RandomSeed = -1.f;

	UPROPERTY(EditAnywhere, Category = "Fracture", meta = (DataflowInput, UIMin = 0.f))
	float Grout = 0.f;

	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Amplitude = 0.f;

	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.00001f));
	float Frequency = 0.1f;

	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Persistence = 0.5f;

	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Lacunarity = 2.f;

	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	int32 OctaveNumber = 4;

	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float PointSpacing = 10.f;

	UPROPERTY(EditAnywhere, Category = "Collision");
	bool AddSamplesForCollision = false;

	UPROPERTY(EditAnywhere, Category = "Collision", meta = (DataflowInput, UIMin = 0.f));
	float CollisionSampleSpacing = 50.f;

	FPlaneCutterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&BoundingBox);
		RegisterInputConnection(&TransformSelection);
		RegisterInputConnection(&NumPlanes);
		RegisterInputConnection(&RandomSeed);
		RegisterInputConnection(&Grout);
		RegisterInputConnection(&Amplitude);
		RegisterInputConnection(&Frequency);
		RegisterInputConnection(&Persistence);
		RegisterInputConnection(&Lacunarity);
		RegisterInputConnection(&OctaveNumber);
		RegisterInputConnection(&PointSpacing);
		RegisterInputConnection(&AddSamplesForCollision);
		RegisterInputConnection(&CollisionSampleSpacing);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Editor Fracture Mode / Fracture / Planar tool
 * Fracture using a set of noised up planes.
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FPlaneCutterDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FPlaneCutterDataflowNode_v2, "PlaneCutter", "GeometryCollection|Fracture", "")

public:
	/** Collection to fracture */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Bound for plane centers */
	UPROPERTY(meta = (DataflowInput))
	FBox BoundingBox = FBox(ForceInit);

	/** Pieces to cut */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "TransformSelection", DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/** Transform to bring the collection into the space of the cutting planes */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Transform Collection to Plane Space"))
	FTransform Transform = FTransform::Identity;
	
	/** Cutting planes to use. The cut plane normal is aligned to the Z-up of the transform. */
	UPROPERTY(EditAnywhere, Category = "Plane Cut", meta = (DataflowInput))
	TArray<FTransform> CutPlanes;

	/** Number of random cutting planes to add, in addition to those specified in the Cut Planes array */
	UPROPERTY(EditAnywhere, Category = "Plane Cut", meta = (DataflowInput, DisplayName = "Add Random Cuts", ClampMin = 0))
	int32 NumPlanes = 0;

	/** Seed for random */
	UPROPERTY(EditAnywhere, Category = "Common Fracture", meta = (DataflowInput, UIMin = "0"))
	int32 RandomSeed = 0;

	/** Chance to fracture each selected bone. If 0, no bones will fracture; if 1, all bones will fracture. */
	UPROPERTY(EditAnywhere, Category = "Common Fracture", meta = (DataflowInput, UIMin = 0.f, DisplayName = "Chance To Fracture Per Bone", UIMax = 1.f));
	float ChanceToFracture = 1.f;

	/** Whether to split the fractured mesh pieces into separate islands based on geometric connectivity after fracturing */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands");
	bool SplitIslands = true;

	/** Distance threshold for connecting spatially-close vertices before computing islands */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands", meta = (EditCondition = "SplitIslands", Units = "cm", ClampMin = "0.0"));
	float CloseVertexDistance = 0.001f;

	/** If > 0, bridge separate islands whose surfaces are within this vertex-to-triangle distance. 0 = disabled. */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands", meta = (EditCondition = "SplitIslands", Units = "cm", ClampMin = "0.0"));
	float VertexToSurfaceBridgeDistance = 0.f;

	/** Amount of space to leave between cut pieces */
	UPROPERTY(EditAnywhere, Category = "Common Fracture", meta = (DataflowInput, UIMin = 0.f));
	float Grout = 0.f;

	/** Size of the Perlin noise displacement (in cm). If 0, no noise will be applied */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Amplitude = 0.f;

	/** Period of the Perlin noise.  Smaller values will create a smoother noise pattern */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.00001f));
	float Frequency = 0.1f;

	/** Persistence of the layers of Perlin noise. At each layer (octave) after the first, the amplitude of the Perlin noise is scaled by this factor */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Persistence = 0.5f;

	/** Lacunarity of the layers of Perlin noise. At each layer (octave) after the first, the frequency of the Perlin noise is scaled by this factor */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Lacunarity = 2.f;

	/**
	 * Number of fractal layers of Perlin noise to apply. Each layer is additive, with Amplitude and Frequency parameters scaled by Persistence and Lacunarity
	 * Smaller values (1 or 2) will create noise that looks like gentle rolling hills, while larger values (> 4) will tend to look more like craggy mountains
	 */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	int32 OctaveNumber = 4;

	/** Distance (in cm) between vertices on cut surfaces where noise is added.  Larger spacing between vertices will create more efficient meshes with fewer triangles, but less resolution to see the shape of the added noise  */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float PointSpacing = 20.f;

	/**
	 * If enabled, add extra vertices (without triangles) to the geometry in regions where vertices are spaced too far apart (e.g. across large triangles)
	 * These extra vertices will be used as collision samples in particle-implicit collisions, and can help the physics system detect collisions more accurately
	 *
	 * Note this is *only* useful for simulations that use particle-implicit collisions
	 */
	UPROPERTY(EditAnywhere, Category = "Collision");
	bool AddSamplesForCollision = false;

	/**
	 * The number of centimeters to allow between vertices on the mesh surface: If there are gaps larger than this, add additional vertices (without triangles) to help support particle-implicit collisions
	 * Only used if Add Samples For Collision is enabled
	 */
	UPROPERTY(EditAnywhere, Category = "Collision", meta = (DataflowInput, UIMin = 0.f, EditCondition = "AddSamplesForCollision"));
	float CollisionSampleSpacing = 50.f;

	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	EDataflowDebugDrawRenderType RenderType = EDataflowDebugDrawRenderType::Wireframe;

	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (UIMin = "1.0", UIMax = "10.0", ClampMin = "1.0", ClampMax = "10.0"));
	float PlaneSizeMultiplier = 2.2f;

	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (EditCondition = "RenderType==EDataflowDebugDrawRenderType::Shaded"));
	bool bTranslucent = true;

	UPROPERTY(EditAnywhere, Category = "Debug Draw");
	bool bRandomizeColors = true;

	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (UIMin = "0"));
	int32 ColorRandomSeed = 0;

	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (UIMin = "0.1", UIMax = "10.0", ClampMin = "0.1", ClampMax = "10.0"));
	float LineWidthMultiplier = 1.f;

	/** Fractured Pieces */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "FracturedTransformSelection "));
	FDataflowTransformSelection NewGeometryTransformSelection;

	FPlaneCutterDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&BoundingBox);
		RegisterInputConnection(&TransformSelection);
		RegisterInputConnection(&Transform) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&NumPlanes) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&CutPlanes) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&RandomSeed) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&ChanceToFracture) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Grout) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Amplitude) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Frequency) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Persistence) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Lacunarity) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&OctaveNumber) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&PointSpacing) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&CollisionSampleSpacing);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection, &TransformSelection);
		RegisterOutputConnection(&NewGeometryTransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

#if WITH_EDITOR
	virtual bool CanDebugDraw() const override
	{
		return true;
	}
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif
};

/**
 *
 * "Explodes" the pieces from the Collection for better visualization
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FExplodedViewDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FExplodedViewDataflowNode, "ExplodedView", "GeometryCollection|Fracture|Utilities", "")

public:
	/** Collection to explode */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Scale amount to expand the pieces uniformly in all directions */
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (DataflowInput))
	float UniformScale = 1.f;

	/** Scale amounts to expand the pieces in all 3 directions */
	UPROPERTY(EditAnywhere, Category = "Scale", meta = (DataflowInput))
	FVector Scale = FVector(1.0);

	/** Translate collection for exploded view */
	UPROPERTY(EditAnywhere, Category = "Exploded View", meta = (DataflowInput))
	FVector Offset = FVector(0.0);

	FExplodedViewDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&UniformScale)
			.SetCanHidePin(true)
			.SetPinIsHidden(true);
		RegisterInputConnection(&Scale)
			.SetCanHidePin(true)
			.SetPinIsHidden(true);
		RegisterInputConnection(&Offset)
			.SetCanHidePin(true)
			.SetPinIsHidden(true);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

private:
	// todo(chaos) this is a copy of a function in FractureEditorModeToolkit, we should move this to a common place  
	static bool GetValidGeoCenter(FGeometryCollection* Collection, const TManagedArray<int32>& TransformToGeometryIndex, const TArray<FTransform>& Transforms, const TManagedArray<TSet<int32>>& Children, const TManagedArray<FBox>& BoundingBox, int32 TransformIndex, FVector& OutGeoCenter);
};

/**
 *
 * Editor Fracture Mode / Fracture / Slice tool
 * Fracture with a grid of X, Y, and Z slices, with optional random variation in angle and offset.
 * 
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FSliceCutterDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSliceCutterDataflowNode, "SliceCutter", "GeometryCollection|Fracture", "")

public:
	/** Collection to fracture */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FBox BoundingBox = FBox(ForceInit);

	/** The selected pieces to cut */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "TransformSelection", DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/** Number of slices along the X axis */
	UPROPERTY(EditAnywhere, Category = Slicing, meta = (DataflowInput, UIMin = "0"))
	int32 SlicesX = 3;

	/** Number of slices along the Y axis */
	UPROPERTY(EditAnywhere, Category = Slicing, meta = (DataflowInput, UIMin = "0"))
	int32 SlicesY = 3;

	/** Number of slices along the Z axis */
	UPROPERTY(EditAnywhere, Category = Slicing, meta = (DataflowInput, UIMin = "0"))
	int32 SlicesZ = 1;

	/** Maximum angle (in degrees) to randomly rotate each slicing plane */
	UPROPERTY(EditAnywhere, Category = Slicing, meta = (DataflowInput, DisplayName = "Random Angle Variation", UIMin = "0.0", UIMax = "90.0"))
	float SliceAngleVariation = 0.f;

	/** Maximum distance (in cm) to randomly shift each slicing plane */
	UPROPERTY(EditAnywhere, Category = Slicing, meta = (DataflowInput, DisplayName = "Random Offset Variation", UIMin = "0.0"))
	float SliceOffsetVariation = 0.f;

	/** Seed for random */
	UPROPERTY(EditAnywhere, Category = "Common Fracture", meta = (DataflowInput, UIMin = "0"))
	int32 RandomSeed = 0;

	/** Chance to fracture each selected bone. If 0, no bones will fracture; if 1, all bones will fracture. */
	UPROPERTY(EditAnywhere, Category = "Common Fracture", meta = (DataflowInput, UIMin = 0.f, UIMax = 1.f));
	float ChanceToFracture = 1.f;

	/** Whether to split the fractured mesh pieces into separate islands based on geometric connectivity after fracturing */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands");
	bool SplitIslands = true;

	/** Distance threshold for connecting spatially-close vertices before computing islands */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands", meta = (EditCondition = "SplitIslands", Units = "cm", ClampMin = "0.0"));
	float CloseVertexDistance = 0.001f;

	/** If > 0, bridge separate islands whose surfaces are within this vertex-to-triangle distance. 0 = disabled. */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands", meta = (EditCondition = "SplitIslands", Units = "cm", ClampMin = "0.0"));
	float VertexToSurfaceBridgeDistance = 0.f;

	/** Amount of space to leave between cut pieces */
	UPROPERTY(EditAnywhere, Category = "Common Fracture", meta = (DataflowInput, UIMin = 0.f));
	float Grout = 0.f;

	/** Size of the Perlin noise displacement (in cm). If 0, no noise will be applied */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Amplitude = 0.f;

	/** Period of the Perlin noise.  Smaller values will create a smoother noise pattern */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.00001f));
	float Frequency = 0.1f;

	/** Persistence of the layers of Perlin noise. At each layer (octave) after the first, the amplitude of the Perlin noise is scaled by this factor */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Persistence = 0.5f;

	/** Lacunarity of the layers of Perlin noise. At each layer (octave) after the first, the frequency of the Perlin noise is scaled by this factor */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Lacunarity = 2.f;

	/**
	 * Number of fractal layers of Perlin noise to apply. Each layer is additive, with Amplitude and Frequency parameters scaled by Persistence and Lacunarity
	 * Smaller values (1 or 2) will create noise that looks like gentle rolling hills, while larger values (> 4) will tend to look more like craggy mountains
	 */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	int32 OctaveNumber = 4;

	/** Distance (in cm) between vertices on cut surfaces where noise is added.  Larger spacing between vertices will create more efficient meshes with fewer triangles, but less resolution to see the shape of the added noise  */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float PointSpacing = 10.f;

	/**
	 * If enabled, add extra vertices (without triangles) to the geometry in regions where vertices are spaced too far apart (e.g. across large triangles)
	 * These extra vertices will be used as collision samples in particle-implicit collisions, and can help the physics system detect collisions more accurately
	 *
	 * Note this is *only* useful for simulations that use particle-implicit collisions
	 */
	UPROPERTY(EditAnywhere, Category = "Collision");
	bool AddSamplesForCollision = false;

	/**
	 * The number of centimeters to allow between vertices on the mesh surface: If there are gaps larger than this, add additional vertices (without triangles) to help support particle-implicit collisions
	 * Only used if Add Samples For Collision is enabled
	 */
	UPROPERTY(EditAnywhere, Category = "Collision", meta = (DataflowInput, UIMin = 0.f, EditCondition = "AddSamplesForCollision"));
	float CollisionSampleSpacing = 50.f;

	/** Fractured Pieces */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "FracturedTransformSelection "))
	FDataflowTransformSelection NewGeometryTransformSelection;

	FSliceCutterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&BoundingBox);
		RegisterInputConnection(&TransformSelection);
		RegisterInputConnection(&SlicesX) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&SlicesY) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&SlicesZ) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&SliceAngleVariation) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&SliceOffsetVariation) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&RandomSeed) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&ChanceToFracture) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Grout) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Amplitude) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Frequency) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Persistence) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Lacunarity) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&OctaveNumber) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&PointSpacing) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&CollisionSampleSpacing) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection, &TransformSelection);
		RegisterOutputConnection(&NewGeometryTransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * 
 * Editor Fracture Mode / Fracture / Brick tool
 * Fracture with a customizable brick pattern. 
 * Note: Currently only supports fracturing with at least some (non-zero) Grout.
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FBrickCutterDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBrickCutterDataflowNode, "BrickCutter", "GeometryCollection|Fracture", "")

public:
	/** Collection to cut */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Boundingbox to create the cutting planes in */
	UPROPERTY(meta = (DataflowInput))
	FBox BoundingBox = FBox(ForceInit);

	/** The selected pieces to cut */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "TransformSelection", DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/** Transform to apply to cut planes */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Transform"))
	FTransform Transform = FTransform::Identity;

	/** The brick bond pattern defines how the bricks are arranged */
	UPROPERTY(EditAnywhere, Category = Brick)
	EFractureBrickBondEnum Bond = EFractureBrickBondEnum::Dataflow_FractureBrickBond_Stretcher;

	/** Brick length (in cm) */
	UPROPERTY(EditAnywhere, Category = Brick, meta = (DataflowInput, UIMin = "0.1", UIMax = "500.0", ClampMin = "0.001"))
	float BrickLength = 194.f;

	/** Brick height (in cm) */
	UPROPERTY(EditAnywhere, Category = Brick, meta = (DataflowInput, UIMin = "0.1", UIMax = "500.0", ClampMin = "0.001"))
	float BrickHeight = 57.f;

	/** Brick depth (in cm) */
	UPROPERTY(EditAnywhere, Category = Brick, meta = (DataflowInput, UIMin = "0.1", UIMax = "500.0", ClampMin = "0.001"))
	float BrickDepth = 92.f;

	/** Seed for random */
	UPROPERTY(EditAnywhere, Category = "Common Fracture", meta = (DataflowInput, UIMin = "0"))
	int32 RandomSeed = 0;

	/** Chance to fracture each selected bone. If 0, no bones will fracture; if 1, all bones will fracture. */
	UPROPERTY(EditAnywhere, Category = "Common Fracture", meta = (DataflowInput, UIMin = 0.f, UIMax = 1.f));
	float ChanceToFracture = 1.f;

	/** Whether to split the fractured mesh pieces into separate islands based on geometric connectivity after fracturing */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands");
	bool SplitIslands = true;

	/** Distance threshold for connecting spatially-close vertices before computing islands */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands", meta = (EditCondition = "SplitIslands", Units = "cm", ClampMin = "0.0"));
	float CloseVertexDistance = 0.001f;

	/** If > 0, bridge separate islands whose surfaces are within this vertex-to-triangle distance. 0 = disabled. */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands", meta = (EditCondition = "SplitIslands", Units = "cm", ClampMin = "0.0"));
	float VertexToSurfaceBridgeDistance = 0.f;

	/** Amount of space to leave between cut pieces */
	UPROPERTY(EditAnywhere, Category = "Common Fracture", meta = (DataflowInput, UIMin = 0.f));
	float Grout = 0.f;

	/** Size of the Perlin noise displacement (in cm). If 0, no noise will be applied */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Amplitude = 0.f;

	/** Period of the Perlin noise.  Smaller values will create a smoother noise pattern */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.00001f));
	float Frequency = 0.1f;

	/** Persistence of the layers of Perlin noise. At each layer (octave) after the first, the amplitude of the Perlin noise is scaled by this factor */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Persistence = 0.5f;

	/** Lacunarity of the layers of Perlin noise. At each layer (octave) after the first, the frequency of the Perlin noise is scaled by this factor */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Lacunarity = 2.f;

	/**
	 * Number of fractal layers of Perlin noise to apply. Each layer is additive, with Amplitude and Frequency parameters scaled by Persistence and Lacunarity
	 * Smaller values (1 or 2) will create noise that looks like gentle rolling hills, while larger values (> 4) will tend to look more like craggy mountains
	 */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	int32 OctaveNumber = 4;

	/** Distance (in cm) between vertices on cut surfaces where noise is added.  Larger spacing between vertices will create more efficient meshes with fewer triangles, but less resolution to see the shape of the added noise  */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float PointSpacing = 10.f;

	/**
	 * If enabled, add extra vertices (without triangles) to the geometry in regions where vertices are spaced too far apart (e.g. across large triangles)
	 * These extra vertices will be used as collision samples in particle-implicit collisions, and can help the physics system detect collisions more accurately
	 *
	 * Note this is *only* useful for simulations that use particle-implicit collisions
	 */
	UPROPERTY(EditAnywhere, Category = "Collision");
	bool AddSamplesForCollision = false;

	/**
	 * The number of centimeters to allow between vertices on the mesh surface: If there are gaps larger than this, add additional vertices (without triangles) to help support particle-implicit collisions
	 * Only used if Add Samples For Collision is enabled
	 */
	UPROPERTY(EditAnywhere, Category = "Collision", meta = (DataflowInput, UIMin = 0.f, EditCondition = "AddSamplesForCollision"));
	float CollisionSampleSpacing = 50.f;

	/** Fractured Pieces */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "FracturedTransformSelection "))
	FDataflowTransformSelection NewGeometryTransformSelection;

	FBrickCutterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&BoundingBox);
		RegisterInputConnection(&TransformSelection);
		RegisterInputConnection(&Transform) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&BrickLength) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&BrickHeight) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&BrickDepth) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&RandomSeed) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&ChanceToFracture) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Grout) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Amplitude) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Frequency) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Persistence) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Lacunarity) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&OctaveNumber) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&PointSpacing) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&CollisionSampleSpacing) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection, &TransformSelection);
		RegisterOutputConnection(&NewGeometryTransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * 
 * Editor Fracture Mode / Fracture / Mesh tool
 * Fracture using the shape of a chosen static mesh and/or array of dynamic meshes
 * 
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FMeshCutterDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMeshCutterDataflowNode, "MeshCutter", "GeometryCollection|Fracture", "")

public:
	/** Collection to cut */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Boundingbox to create the cutting planes in */
	UPROPERTY(meta = (DataflowInput))
	FBox BoundingBox = FBox(ForceInit);

	/** The selected pieces to cut */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "TransformSelection", DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/** Transform to apply to cut planes */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Transform"))
	FTransform Transform = FTransform::Identity;

	/** Dynamic Meshes to cut with */
	UPROPERTY(EditAnywhere, Category = "Mesh", meta = (DataflowInput))
	TArray<TObjectPtr<UDynamicMesh>> CuttingDynamicMeshes;

	/** Static Mesh to cut with */
	UPROPERTY(EditAnywhere, Category = "StaticMesh", meta = (DataflowInput))
	TObjectPtr<UStaticMesh> CuttingStaticMesh;

	/** If using a Static Mesh to cut, attempt to use the Nanite HiRes source mesh, if available and non-empty. */
	UPROPERTY(EditAnywhere, Category = "StaticMesh", meta = (DisplayName = "Use HiRes"));
	bool bUseHiRes = false;

	/** If using a Static Mesh to cut, and not using the Nanite HiRes source mesh, use this LOD level's mesh */
	UPROPERTY(EditAnywhere, Category = "StaticMesh", meta = (DisplayName = "LOD Level"));
	int32 LODLevel = 0;

	/** How to arrange the mesh cuts in space */
	UPROPERTY(EditAnywhere, Category = Distribution)
	EMeshCutterCutDistribution CutDistribution = EMeshCutterCutDistribution::SingleCut;

	/** When there are multiple cutting meshes, how to choose the cut mesh to apply at each location */
	UPROPERTY(EditAnywhere, Category = Distribution)
	EMeshCutterPerCutMeshSelection PerCutMeshSelection = EMeshCutterPerCutMeshSelection::All;

	/** Number of meshes to random scatter */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (DataflowInput, ClampMin = "1", UIMax = "5000", EditCondition = "CutDistribution == EMeshCutterCutDistribution::UniformRandom", EditConditionHides))
	int32 NumberToScatter = 10;

	/** Number of meshes to add to grid in X */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (DataflowInput, DisplayName = "Grid Width", ClampMin = "1", UIMax = "100", ClampMax = "5000", EditCondition = "CutDistribution == EMeshCutterCutDistribution::Grid", EditConditionHides))
	int32 GridX = 2;

	/** Number of meshes to add to grid in Y */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (DataflowInput, DisplayName = "Grid Depth", ClampMin = "1", UIMax = "100", ClampMax = "5000", EditCondition = "CutDistribution == EMeshCutterCutDistribution::Grid", EditConditionHides))
	int32 GridY = 2;

	/** Number of meshes to add to grid in Z */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (DataflowInput, DisplayName = "Grid Height", ClampMin = "1", UIMax = "100", ClampMax = "5000", EditCondition = "CutDistribution == EMeshCutterCutDistribution::Grid", EditConditionHides))
	int32 GridZ = 2;

	/** Magnitude of random displacement to cutting meshes */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (DataflowInput, DisplayName = "Variability", EditCondition = "CutDistribution == EMeshCutterCutDistribution::Grid", EditConditionHides, UIMin = "0.0", ClampMin = "0.0"))
	float Variability = 0.f;

	/** Minimum scale factor to apply to cutting meshes. A random scale will be chosen between Min and Max */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (DataflowInput, ClampMin = "0.001", EditCondition = "CutDistribution != EMeshCutterCutDistribution::SingleCut", EditConditionHides))
	float MinScaleFactor = .5;

	/** Maximum scale factor to apply to cutting meshes. A random scale will be chosen between Min and Max */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (DataflowInput, ClampMin = "0.001", EditCondition = "CutDistribution != EMeshCutterCutDistribution::SingleCut", EditConditionHides))
	float MaxScaleFactor = 1.5;

	/** Whether to randomly vary the orientation of the cutting meshes */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (EditCondition = "CutDistribution != EMeshCutterCutDistribution::SingleCut", EditConditionHides))
	bool bRandomOrientation = true;

	/** Roll will be chosen between -Range and +Range */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (DataflowInput, DisplayName = "+/- Roll Range", EditCondition = "CutDistribution != EMeshCutterCutDistribution::SingleCut && bRandomOrientation", EditConditionHides, ClampMin = "0", ClampMax = "180"))
	float RollRange = 180;

	/** Pitch will be chosen between -Range and +Range */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (DataflowInput, DisplayName = "+/- Pitch Range", EditCondition = "CutDistribution != EMeshCutterCutDistribution::SingleCut && bRandomOrientation", EditConditionHides, ClampMin = "0", ClampMax = "180"))
	float PitchRange = 180;

	/** Yaw will be chosen between -Range and +Range */
	UPROPERTY(EditAnywhere, Category = Distribution, meta = (DataflowInput, DisplayName = "+/- Yaw Range", EditCondition = "CutDistribution != EMeshCutterCutDistribution::SingleCut && bRandomOrientation", EditConditionHides, ClampMin = "0", ClampMax = "180"))
	float YawRange = 180;

	/** Seed for random */
	UPROPERTY(EditAnywhere, Category = "Common Fracture", meta = (DataflowInput, UIMin = "0"))
	int32 RandomSeed = 0;

	/** Chance to fracture each selected bone. If 0, no bones will fracture; if 1, all bones will fracture. */
	UPROPERTY(EditAnywhere, Category = "Common Fracture", meta = (DataflowInput, UIMin = 0.f, UIMax = 1.f));
	float ChanceToFracture = 1.f;

	/** Whether to split the fractured mesh pieces into separate islands based on geometric connectivity after fracturing */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands");
	bool SplitIslands = true;

	/** Distance threshold for connecting spatially-close vertices before computing islands */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands", meta = (EditCondition = "SplitIslands", Units = "cm", ClampMin = "0.0"));
	float CloseVertexDistance = 0.001f;

	/** If > 0, bridge separate islands whose surfaces are within this vertex-to-triangle distance. 0 = disabled. */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands", meta = (EditCondition = "SplitIslands", Units = "cm", ClampMin = "0.0"));
	float VertexToSurfaceBridgeDistance = 0.f;

	/**
	 * If enabled, add extra vertices (without triangles) to the geometry in regions where vertices are spaced too far apart (e.g. across large triangles)
	 * These extra vertices will be used as collision samples in particle-implicit collisions, and can help the physics system detect collisions more accurately
	 *
	 * Note this is *only* useful for simulations that use particle-implicit collisions
	 */
	UPROPERTY(EditAnywhere, Category = "Collision");
	bool AddSamplesForCollision = false;

	/**
	 * The number of centimeters to allow between vertices on the mesh surface: If there are gaps larger than this, add additional vertices (without triangles) to help support particle-implicit collisions
	 * Only used if Add Samples For Collision is enabled
	 */
	UPROPERTY(EditAnywhere, Category = "Collision", meta = (DataflowInput, DisplayName = "Point Spacing", UIMin = 0.f, EditCondition = "AddSamplesForCollision"));
	float CollisionSampleSpacing = 50.f;

	/** Fractured Pieces */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "FracturedTransformSelection "))
	FDataflowTransformSelection NewGeometryTransformSelection;

	FMeshCutterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&BoundingBox);
		RegisterInputConnection(&TransformSelection);
		RegisterInputConnection(&Transform) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&CuttingStaticMesh) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&CuttingDynamicMeshes) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&NumberToScatter) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&GridX) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&GridY) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&GridZ) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&Variability) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&MinScaleFactor) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&MaxScaleFactor) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&RollRange) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&PitchRange) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&YawRange) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&RandomSeed) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&ChanceToFracture) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&CollisionSampleSpacing) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection, &TransformSelection);
		RegisterOutputConnection(&NewGeometryTransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Editor Fracture Mode / Fracture / Uniform tool
 * Fracture using a Voronoi diagram with a uniform random pattern, creating fracture pieces of similar volume across the shape.
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FUniformFractureDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FUniformFractureDataflowNode, "UniformFracture", "GeometryCollection|Fracture", "")

private:
	/** Collection to fracture */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Bones to fracture, if not connected it will fracture all the bones */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "TransformSelection", DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/** Transform to apply */
	UPROPERTY(meta = (DataflowInput))
	FTransform Transform = FTransform::Identity;

	/** Minimum Number of Voronoi sites. The amount of sites per Voronoi diagram will be chosen at random between Min and Max */
	UPROPERTY(EditAnywhere, Category = "Uniform Voronoi", meta = (DataflowInput, UIMin = "1", UIMax = "5000", ClampMin = "1"));
	int32 MinVoronoiSites = 20;

	/** Maximum Number of Voronoi sites. The amount of sites per Voronoi diagram will be chosen at random between Min and Max */
	UPROPERTY(EditAnywhere, Category = "Uniform Voronoi", meta = (DataflowInput, UIMin = "1", UIMax = "5000", ClampMin = "1"));
	int32 MaxVoronoiSites = 20;

	/** ID for the material for the newly created internal faces */
	UPROPERTY(EditAnywhere, Category = "Materials", meta = (DataflowInput, UIMin = "0", UIMax = "10", ClampMin = "0"));
	int32 InternalMaterialID = 0;

	/** Random number generator seed for repeatability. If the value is -1, a different random seed will be used every time, otherwise the specified seed will always be used */
	UPROPERTY(EditAnywhere, Category = "Common Fracture", meta = (DataflowInput, UIMin = "-1", UIMax = "5000", ClampMin = "-1"));
	int32 RandomSeed = 0;

	/** Chance to fracture each selected bone. If 0, no bones will fracture; if 1, all bones will fracture. */
	UPROPERTY(EditAnywhere, Category = "Common Fracture", meta = (DataflowInput, UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"));
	float ChanceToFracture = 1.f;

	/** Generate a fracture pattern across all selected meshes.  */
	UPROPERTY(EditAnywhere, Category = "Common Fracture");
	bool GroupFracture = true;

	/** Whether to split the fractured mesh pieces into separate islands based on geometric connectivity after fracturing */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands");
	bool SplitIslands = true;

	/** Distance threshold for connecting spatially-close vertices before computing islands */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands", meta = (EditCondition = "SplitIslands", Units = "cm", ClampMin = "0.0"));
	float CloseVertexDistance = 0.001f;

	/** If > 0, bridge separate islands whose surfaces are within this vertex-to-triangle distance. 0 = disabled. */
	UPROPERTY(EditAnywhere, Category = "Common Fracture|Split Islands", meta = (EditCondition = "SplitIslands", Units = "cm", ClampMin = "0.0"));
	float VertexToSurfaceBridgeDistance = 0.f;

	/** Amount of space to leave between cut pieces */
	UPROPERTY(EditAnywhere, Category = "Common Fracture", meta = (DataflowInput, UIMin = 0.f));
	float Grout = 0.f;

	/** Size of the Perlin noise displacement (in cm). If 0, no noise will be applied */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Amplitude = 0.f;

	/** Period of the Perlin noise.  Smaller values will create a smoother noise pattern */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.00001f));
	float Frequency = 0.1f;

	/** Persistence of the layers of Perlin noise. At each layer (octave) after the first, the amplitude of the Perlin noise is scaled by this factor */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Persistence = 0.5f;

	/** Lacunarity of the layers of Perlin noise. At each layer (octave) after the first, the frequency of the Perlin noise is scaled by this factor */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float Lacunarity = 2.f;

	/**
	 * Number of fractal layers of Perlin noise to apply. Each layer is additive, with Amplitude and Frequency parameters scaled by Persistence and Lacunarity
	 * Smaller values (1 or 2) will create noise that looks like gentle rolling hills, while larger values (> 4) will tend to look more like craggy mountains
	 */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0, ClampMin = 0));
	int32 OctaveNumber = 4;

	/** Distance (in cm) between vertices on cut surfaces where noise is added.  Larger spacing between vertices will create more efficient meshes with fewer triangles, but less resolution to see the shape of the added noise  */
	UPROPERTY(EditAnywhere, Category = "Noise", meta = (DataflowInput, UIMin = 0.f));
	float PointSpacing = 10.f;

	/**
	 * If enabled, add extra vertices (without triangles) to the geometry in regions where vertices are spaced too far apart (e.g. across large triangles)
	 * These extra vertices will be used as collision samples in particle-implicit collisions, and can help the physics system detect collisions more accurately
	 *
	 * Note this is *only* useful for simulations that use particle-implicit collisions
	 */
	UPROPERTY(EditAnywhere, Category = "Collision");
	bool AddSamplesForCollision = false;

	/**
	 * The number of centimeters to allow between vertices on the mesh surface: If there are gaps larger than this, add additional vertices (without triangles) to help support particle-implicit collisions
	 * Only used if Add Samples For Collision is enabled
	 */
	UPROPERTY(EditAnywhere, Category = "Collision", meta = (DataflowInput, UIMin = 0.f, EditCondition = "AddSamplesForCollision"));
	float CollisionSampleSpacing = 50.f;

	/** Fractured Pieces */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "FracturedTransformSelection "))
	FDataflowTransformSelection NewGeometryTransformSelection;

public:
	FUniformFractureDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

UENUM(BlueprintType)
enum class EDataflowVisualizeFractureColoringType : uint8
{
	ColorByParent UMETA(DisplayName = "Color by Parent"),
	ColorByLevel UMETA(DisplayName = "Color by Level"),
	ColorByCluster UMETA(DisplayName = "Color by Cluster"),
	ColorByLeafLevel UMETA(DisplayName = "Color by Leaf Level"),
	ColorByLeaf UMETA(DisplayName = "Color by Leaf"),
	ColorByAttr UMETA(DisplayName = "Color by Attribute"),
};

USTRUCT() 
struct FMinSettings
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Color", meta = (DisplayName = "Minimum Attr Value", EditCondition = "ColoringType == EDataflowVisualizeFractureColoringType::ColorByAttr", EditConditionHides))
	float MinAttrValue = 0.f;

	UPROPERTY(EditAnywhere, Category = "Color", meta = (EditCondition = "ColoringType == EDataflowVisualizeFractureColoringType::ColorByAttr", EditConditionHides))
	FLinearColor MinColor = FLinearColor(FLinearColor::Green);
};

USTRUCT()
struct FMaxSettings
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Color", meta = (DisplayName = "Maximum Attr Value", EditCondition = "ColoringType == EDataflowVisualizeFractureColoringType::ColorByAttr", EditConditionHides))
	float MaxAttrValue = 1.f;

	UPROPERTY(EditAnywhere, Category = "Color", meta = (EditCondition = "ColoringType == EDataflowVisualizeFractureColoringType::ColorByAttr", EditConditionHides))
	FLinearColor MaxColor = FLinearColor(FLinearColor::Red);
};

/**
 * 
 * Visualizing fracture/cluster info in fractured collection
 * 
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FVisualizeFractureDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVisualizeFractureDataflowNode, "VisualizeFracture", "GeometryCollection|Fracture|Utilities", "")

private:
	/** Collection to visualize */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Level", meta = (DataflowInput, UIMin = "0", ClampMin = "0"));
	int32 Level = 0;

	/** Seed for random */
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput, UIMin = "0", ClampMin = "0"));
	int32 RandomSeed = 0;

	/** Use cluster level for coloring and explode */
	UPROPERTY(EditAnywhere, Category = "Exploded View");
	bool bApplyExplodedView = true;

	/** Scale amount to expand the pieces uniformly in all directions */
	UPROPERTY(EditAnywhere, Category = "Exploded View", meta = (DataflowInput, UIMin = "0.0", ClampMin = "0.0", UIMax = "1.0", ClampMax = "1.0", EditCondition = "bApplyExplodedView"))
	float ExplodeAmount = 0.5f;

	/** Scale amounts to expand the pieces in all 3 directions */
	UPROPERTY(EditAnywhere, Category = "Exploded View", meta = (DataflowInput, EditCondition = "bApplyExplodedView"))
	FVector Scale = FVector(1.0);

	/**  */
	UPROPERTY(EditAnywhere, Category = "Color");
	bool bApplyColor = true;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Color", meta = (EditCondition = "bApplyColor"));
	EDataflowVisualizeFractureColoringType ColoringType = EDataflowVisualizeFractureColoringType::ColorByLevel;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Color", meta = (UIMin = "0", ClampMin = "0", UIMax = "255", ClampMax = "255"))
	int32 RandomColorRangeMin = 40;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Color", meta = (UIMin = "0", ClampMin = "0", UIMax = "255", ClampMax = "255"))
	int32 RandomColorRangeMax = 190;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Color", meta = (EditCondition = "ColoringType == EDataflowVisualizeFractureColoringType::ColorByAttr", EditConditionHides))
	FString Attribute;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Color", meta = (EditCondition = "ColoringType == EDataflowVisualizeFractureColoringType::ColorByAttr", EditConditionHides))
	FMinSettings Min;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Color", meta = (EditCondition = "ColoringType == EDataflowVisualizeFractureColoringType::ColorByAttr", EditConditionHides))
	FMaxSettings Max;

	/** Translate collection for exploded view */
	UPROPERTY(EditAnywhere, Category = "Offset", meta = (DataflowInput))
	FVector Offset = FVector(0.0);

public:
	FVisualizeFractureDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

UENUM(BlueprintType)
enum class EDataflowSetFloatArrayMethod : uint8
{
	Random UMETA(DisplayName = "Random"),
	Noise UMETA(DisplayName = "Noise"),
	ByBoundingBox UMETA(DisplayName = "By BoundinBox"),
};

/**
 *
 * Set a float values in an array
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FSetFloatAttributeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetFloatAttributeDataflowNode, "SetFloatAttribute", "GeometryCollection|Utilities", "")

private:
	/** Collection to visualize */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Attribute");
	FString Attribute;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Method");
	EDataflowSetFloatArrayMethod Method = EDataflowSetFloatArrayMethod::Random;

	/** Seed for random */
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput, UIMin = "0", EditCondition = "Method == EDataflowSetFloatArrayMethod::Random", EditConditionHides));
	int32 RandomSeed = 0;

	/** Seed for random */
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput, UIMin = "0", EditCondition = "Method == EDataflowSetFloatArrayMethod::Noise", EditConditionHides));
	float NoiseScale = 1.f;

	/** Output float array */
	UPROPERTY(meta = (DataflowOutput))
	TArray<float> FloatArray;

public:
	FSetFloatAttributeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void GeometryCollectionFracturingNodes();
}

