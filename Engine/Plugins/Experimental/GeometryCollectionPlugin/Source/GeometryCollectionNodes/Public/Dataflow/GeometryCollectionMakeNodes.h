// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "UDynamicMesh.h"
#include "Dataflow/DataflowSelection.h"
#include "Math/MathFwd.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Dataflow/DataflowPlane.h"
#include "Dataflow/DataflowPoints.h"
#include "Dataflow/DataflowPrimitiveTypes.h"
#include "Dataflow/SamplerNodes/DataflowSamplerTypes.h"
#include "Dataflow/DataflowDebugDraw.h"
#include "Dataflow/DataflowGeneratorNodes.h"

#include "GeometryCollectionMakeNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/**
 * Make a literal string
 * Deprecated (5.6)
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FMakeLiteralStringDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeLiteralStringDataflowNode, "MakeLiteralString", "Utilities|String", "")

public:
	UPROPERTY(EditAnywhere, Category = "String");
	FString Value = FString("");

	UPROPERTY(meta = (DataflowOutput))
	FString String;

	FMakeLiteralStringDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&String);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a literal string
 */
USTRUCT()
struct FMakeLiteralStringDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeLiteralStringDataflowNode_v2, "MakeLiteralString", "Utilities|String", "")

public:
	UPROPERTY(EditAnywhere, Category = "String", meta = (DataflowOutput));
	FString String;

	FMakeLiteralStringDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&String);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Make a points array from specified points
 *
 */
USTRUCT()
struct FMakePointsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakePointsDataflowNode, "MakePoints", "Generators|Point", "")

public:
	UPROPERTY(EditAnywhere, Category = "Point")
	TArray<FVector> Point;

	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> Points;

	FMakePointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&Points);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


UENUM(BlueprintType)
enum class EMakeBoxDataTypeEnum : uint8
{
	Dataflow_MakeBox_DataType_MinMax UMETA(DisplayName = "Min/Max"),
	Dataflow_MakeBox_DataType_CenterSize UMETA(DisplayName = "Center/Size"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 * Make a box
 */
USTRUCT(Meta = (Icon = "Icons.MakeStaticMesh"))
struct FMakeBoxDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeBoxDataflowNode, "MakeBox", "Generators|Box", "")

public:
	FMakeBoxDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(EditAnywhere, Category = "Box", meta = (DisplayName = "Input Data Type"));
	EMakeBoxDataTypeEnum DataType = EMakeBoxDataTypeEnum::Dataflow_MakeBox_DataType_CenterSize;

	UPROPERTY(EditAnywhere, Category = "Box", meta = (DataflowInput, EditCondition = "DataType == EMakeBoxDataTypeEnum::Dataflow_MakeBox_DataType_MinMax", EditConditionHides, GizmoType = "Translate"))
	FVector Min = FVector(0.0);

	UPROPERTY(EditAnywhere, Category = "Box", meta = (DataflowInput, EditCondition = "DataType == EMakeBoxDataTypeEnum::Dataflow_MakeBox_DataType_MinMax", EditConditionHides, GizmoType = "Translate"))
	FVector Max = FVector(100.0);

	UPROPERTY(EditAnywhere, Category = "Box", meta = (DataflowInput, EditCondition = "DataType == EMakeBoxDataTypeEnum::Dataflow_MakeBox_DataType_CenterSize", EditConditionHides, GizmoType = "Translate"))
	FVector Center = FVector(0.0);

	UPROPERTY(EditAnywhere, Category = "Box", meta = (DataflowInput, EditCondition = "DataType == EMakeBoxDataTypeEnum::Dataflow_MakeBox_DataType_CenterSize", EditConditionHides))
	FVector Size = FVector(100.0);

	UPROPERTY(meta = (DataflowOutput));
	FBox Box = FBox(ForceInit);

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


/**
 *
 * Description for this node
 *
 */
USTRUCT()
struct FMakeSphereDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeSphereDataflowNode, "MakeSphere", "Generators|Sphere", "")

public:
	UPROPERTY(EditAnywhere, Category = "Sphere", meta = (DataflowInput, GizmoType = "Translate"))
	FVector Center = FVector(0.f);

	UPROPERTY(EditAnywhere, Category = "Sphere", meta = (DataflowInput));
	float Radius = 10.f;

	UPROPERTY(meta = (DataflowOutput));
	FSphere Sphere = FSphere(ForceInit);

	FMakeSphereDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Center);
		RegisterInputConnection(&Radius);
		RegisterOutputConnection(&Sphere);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a float value
 * Deprecated (5.6)
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FMakeLiteralFloatDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeLiteralFloatDataflowNode, "MakeLiteralFloat", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Float");
	float Value = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float Float = 0.f;

	FMakeLiteralFloatDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&Float);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a float value
 */
USTRUCT()
struct FMakeLiteralFloatDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeLiteralFloatDataflowNode_v2, "MakeLiteralFloat", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Float", meta = (DataflowOutput));
	float Float = 0.f;

	FMakeLiteralFloatDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&Float);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Make a double value
 *
 */
USTRUCT()
struct FMakeLiteralDoubleDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeLiteralDoubleDataflowNode, "MakeLiteralDouble", "Math|Double", "")

private:
	UPROPERTY(EditAnywhere, Category = "Double", meta = (DataflowOutput));
	double Double = 0.0;

public:
	FMakeLiteralDoubleDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make an integer value
 * Deprecated (5.6)
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FMakeLiteralIntDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeLiteralIntDataflowNode, "MakeLiteralInt", "Math|Int", "")

public:
	UPROPERTY(EditAnywhere, Category = "Int");
	int32 Value = 0;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "Int"))
	int32 Int = 0;

	FMakeLiteralIntDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&Int);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make an integer value
 */
USTRUCT()
struct FMakeLiteralIntDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeLiteralIntDataflowNode_v2, "MakeLiteralInt", "Math|Int", "")

public:
	UPROPERTY(EditAnywhere, Category = "Int", meta = (DataflowOutput));
	int32 Int = 0;

	FMakeLiteralIntDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&Int);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a bool value
 * Deprecated(5.6)
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FMakeLiteralBoolDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeLiteralBoolDataflowNode, "MakeLiteralBool", "Math|Boolean", "")

public:
	UPROPERTY(EditAnywhere, Category = "Bool");
	bool Value = false;

	UPROPERTY(meta = (DataflowOutput))
	bool Bool = false;

	FMakeLiteralBoolDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&Bool);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a bool value
 */
USTRUCT()
struct FMakeLiteralBoolDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeLiteralBoolDataflowNode_v2, "MakeLiteralBool", "Math|Boolean", "")

public:
	UPROPERTY(EditAnywhere, Category = "Bool", meta = (DataflowOutput));
	bool Bool = false;

	FMakeLiteralBoolDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&Bool);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


/**
 * Make a vector
 * Deprecated(5.6)
 * Use MakeVector3 instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FMakeLiteralVectorDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeLiteralVectorDataflowNode, "MakeLiteralVector", "Math|Vector", "")

public:
	UPROPERTY(EditAnywhere, Category = "Vector", meta = (DataflowInput));
	float X = float(0.0);

	UPROPERTY(EditAnywhere, Category = "Vector", meta = (DataflowInput));
	float Y = float(0.0);

	UPROPERTY(EditAnywhere, Category = "Vector", meta = (DataflowInput));
	float Z = float(0.0);

	UPROPERTY(meta = (DataflowOutput, DisplayName = "Vector"))
	FVector Vector = FVector(0.0);

	FMakeLiteralVectorDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&X);
		RegisterInputConnection(&Y);
		RegisterInputConnection(&Z);
		RegisterOutputConnection(&Vector);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make an FTransform
 * Note: Originaly this version was depricated and replaced with FMakeTransformDataflowNode_v2 but when AnyRotationType was
 * introduced with the ConvertAnyRotation node FMakeTransformDataflowNode_v2 became obsolete and this version became the current version again
 */
USTRUCT()
struct FMakeTransformDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeTransformDataflowNode, "MakeTransform", "Generators|Transform", "")

private:
	/** Translation */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DataflowInput, DisplayName = "Translation"));
	FVector InTranslation = FVector(0, 0, 0);

	/** Rotation as Euler */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DataflowInput, DisplayName = "Rotation"));
	FVector InRotation = FVector(0, 0, 0);

	/** Scale */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DataflowInput, DisplayName = "Scale"));
	FVector InScale = FVector(1, 1, 1);

	/** Result transform */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Transform"));
	FTransform OutTransform = FTransform::Identity;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FMakeTransformDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&InTranslation);
		RegisterInputConnection(&InRotation);
		RegisterInputConnection(&InScale);
		RegisterOutputConnection(&OutTransform);
	}
};

/**
*
* Make a FTransform
* Deprecated (5.6)
* Use FMakeTransformDataflowNode instead
*/
USTRUCT(meta = (Deprecated = "5.6"))
struct FMakeTransformDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeTransformDataflowNode_v2, "MakeTransform", "Generators|Transform", "")

private:

	/** Translation */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DataflowInput));
	FVector Translation = FVector(0.f, 0.f, 0.f);

	/** Rotation as Euler */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DataflowInput));
	FVector Rotation = FVector(0.f, 0.f, 0.f);

	/** Rotation a Rotator */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DataflowInput));
	FRotator Rotator = FRotator(0.f, 0.f, 0.f);

	/** Rotation as a quaternion */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DataflowInput));
	FQuat Quat = FQuat(ForceInit);

	/** Scale */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DataflowInput));
	FVector Scale = FVector(1.f, 1.f, 1.f);

	/** Result transform */
	UPROPERTY(meta = (DataflowOutput));
	FTransform Transform = FTransform::Identity;

public:
	FMakeTransformDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 *
 *
 */
USTRUCT()
struct FMakeQuaternionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeQuaternionDataflowNode, "MakeQuaternion", "Math|Vector", "")

public:
	UPROPERTY(EditAnywhere, Category = "Quaternion ", meta = (DataflowInput));
	float X = float(0.0);

	UPROPERTY(EditAnywhere, Category = "Quaternion", meta = (DataflowInput));
	float Y = float(0.0);

	UPROPERTY(EditAnywhere, Category = "Quaternion", meta = (DataflowInput));
	float Z = float(0.0);

	UPROPERTY(EditAnywhere, Category = "Quaternion", meta = (DataflowInput));
	float W = float(0.0);

	UPROPERTY(meta = (DataflowOutput, DisplayName = "Quaternion"))
	FQuat Quaternion = FQuat(ForceInitToZero);

	FMakeQuaternionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&X);
		RegisterInputConnection(&Y);
		RegisterInputConnection(&Z);
		RegisterInputConnection(&W);
		RegisterOutputConnection(&Quaternion);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * M
 *
 */
USTRUCT()
struct FMakeFloatArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeFloatArrayDataflowNode, "MakeFloatArray", "Math|Float", "")

public:
	/** Number of elements of the array */
	UPROPERTY(EditAnywhere, Category = "Float", meta = (DataflowInput, DisplayName = "Number of Elements", UIMin = "0"));
	int32 NumElements = 1;

	/** Value to initialize the array with */
	UPROPERTY(EditAnywhere, Category = "Float", meta = (DataflowInput));
	float Value = 0.f;

	/** Output float array */
	UPROPERTY(meta = (DataflowOutput))
	TArray<float> FloatArray;

	FMakeFloatArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&NumElements);
		RegisterInputConnection(&Value);
		RegisterOutputConnection(&FloatArray);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Make an empty ManagedArrayCollection
 *
 */
USTRUCT()
struct FMakeCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeCollectionDataflowNode, "MakeCollection", "Generators|Collection", "")

private:
	UPROPERTY(meta = (DataflowOutput));
	FManagedArrayCollection Collection;

	/** if true, create a root transform */
	UPROPERTY(EditAnyWhere, Category = "Options")
	bool bAddRootTransform = false;

public:
	FMakeCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Make a Rotator
 *
 */
USTRUCT()
struct FMakeRotatorDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeRotatorDataflowNode, "MakeRotator", "Generators|Transform", "")

private:
	/** Rotation around the right axis (around Y axis), Looking up and down (0=Straight Ahead, +Up, -Down) */
	UPROPERTY(EditAnywhere, Category = "Rotator", meta = (DataflowInput));
	float Pitch = 0.f;

	/** Rotation around the up axis (around Z axis), Turning around (0=Forward, +Right, -Left)*/
	UPROPERTY(EditAnywhere, Category = "Rotator", meta = (DataflowInput));
	float Yaw = 0.f;

	/** Rotation around the forward axis (around X axis), Tilting your head, (0=Straight, +Clockwise, -CCW) */
	UPROPERTY(EditAnywhere, Category = "Rotator", meta = (DataflowInput));
	float Roll = 0.f;

	/** Rotator output */
	UPROPERTY(meta = (DataflowOutput));
	FRotator Rotator = FRotator(0.f, 0.f, 0.f);

public:
	FMakeRotatorDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
* Break a Transform into Translation, Rotation (Euler, Rotator, Quaternion), Scale
*/
USTRUCT()
struct FBreakTransformDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBreakTransformDataflowNode, "BreakTransform", "Math|Transform", "")

private:
	/** Transform to break into components */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DataflowInput, GizmoType = "Transform"));
	FTransform Transform;

	/** Translation */
	UPROPERTY(meta = (DataflowOutput));
	FDataflowVectorTypes Translation;

	/** Rotation as Euler */
	UPROPERTY(meta = (DataflowOutput));
	FVector Rotation = FVector(0.f, 0.f, 0.f);

	/** Rotation as a rotator */
	UPROPERTY(meta = (DataflowOutput));
	FRotator Rotator = FRotator(0.f, 0.f, 0.f);

	/** Rotation as a quaternion */
	UPROPERTY(meta = (DataflowOutput));
	FQuat Quat = FQuat(ForceInit);

	/** Scale */
	UPROPERTY(meta = (DataflowOutput));
	FDataflowVectorTypes Scale;

public:
	FBreakTransformDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a plane
 * Deprecated (5.8), use FMakePlaneDataflowNode_v2 instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FMakePlaneDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakePlaneDataflowNode, "MakePlane", "Generators|Plane", "")

private:
	/** Base point */
	UPROPERTY(EditAnywhere, Category = "Plane", meta = (DataflowInput, GizmoType = "Translate"));
	FVector BasePoint = FVector(0.0);

	/** Normal vector */
	UPROPERTY(EditAnywhere, Category = "Plane", meta = (DataflowInput));
	FVector Normal = FVector::UpVector;

	/** DebugDraw settings */
	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ShowOnlyInnerProperties))
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (UIMin = "1.0", UIMax = "10.0", ClampMin = "1.0", ClampMax = "10.0"));
	float PlaneSizeMultiplier = 1.f;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput));
	FPlane Plane = FPlane(ForceInit);

public:
	FMakePlaneDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override
	{
		return true;
	}
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif
};

/**
 *
 * Make an int32 array starting with Start with Count elements
 *
 */
USTRUCT()
struct FMakeIntArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeIntArrayDataflowNode, "MakeIntArray", "Generators|Array", "integer indices start end")

private:
	/** First element of IntArray */
	UPROPERTY(EditAnywhere, Category = "Int", meta = (DataflowInput))
	int32 Start = 0;

	/** Number of elements in IntArray */
	UPROPERTY(EditAnywhere, Category = "Int", meta = (DataflowInput, UIMin = "0", ClampMin = "0"))
	int32 Count = 3;

	UPROPERTY(EditAnywhere, Category = "Int")
	bool bReverseOrder = false;

	/** IntArray */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> IntArray;

public:
	FMakeIntArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Make an int32 array from a comma separated list, e.g. "0, 2, 5-10, 12-15"
 *
 */
USTRUCT()
struct FMakeIntArrayFromListDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeIntArrayFromListDataflowNode, "MakeIntArrayFromList", "Generators|Array", "integer indices start end list")

private:
	/** First element of IntArray */
	UPROPERTY(EditAnywhere, Category = "Int")
	FString List;

	UPROPERTY(EditAnywhere, Category = "Int")
	bool bReverseOrder = false;

	/** IntArray */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> IntArray;

public:
	FMakeIntArrayFromListDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Make a plane
 */
USTRUCT()
struct FMakePlaneDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakePlaneDataflowNode_v2, "MakePlane", "Generators|Plane", "")

private:
	/** Base point */
	UPROPERTY(EditAnywhere, Category = "Plane", meta = (DataflowInput));
	FVector BasePoint = FVector(0.0);

	/** Normal vector */
	UPROPERTY(EditAnywhere, Category = "Plane", meta = (DataflowInput));
	FVector Normal = FVector::UpVector;

	/** Output plane */
	UPROPERTY(meta = (DataflowOutput));
	FDataflowPlane Plane;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FMakePlaneDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Make a points array from specified points
 *
 */
USTRUCT()
struct FMakePointsDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakePointsDataflowNode_v2, "MakePoints", "Generators|Point", "")

private:
	UPROPERTY(EditAnywhere, Category = "Point")
	TArray<FVector> Point;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowPoints Points;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FMakePointsDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Make an array of any numeric type
 *
 */
 USTRUCT(meta = (Icon = "Kismet.VariableList.ArrayTypeIcon"))
 struct FMakeNumericArrayDataflowNode : public FDataflowNode
 {
 	GENERATED_USTRUCT_BODY()
 	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeNumericArrayDataflowNode, "MakeNumericArray", "Math", "int float double numbers create")
 
 private:
 	/** Number of elements of the array */
 	UPROPERTY(EditAnywhere, Category = "Array", meta = (DataflowInput, DisplayName = "Number of Elements", UIMin = "0"));
 	int32 NumElements = 1;
 
 	/** Default value for output, whichever value doesn't get set in the output array by the generator will have the default value  */
 	UPROPERTY(EditAnywhere, Category = "Array", meta = (DataflowInput));
 	FDataflowNumericTypes DefaultValue;
 
 	/** Generator input */
 	UPROPERTY(EditAnywhere, Category = "Generator", meta = (DataflowInput));
 	FDataflowValueGenerator Generator;
 
 	/** Output float array */
 	UPROPERTY(meta = (DataflowOutput))
 	FDataflowNumericArrayTypes Array;
 
 	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
 
 public:
 	FMakeNumericArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
 };

namespace UE::Dataflow
{
	void GeometryCollectionMakeNodes();
}
