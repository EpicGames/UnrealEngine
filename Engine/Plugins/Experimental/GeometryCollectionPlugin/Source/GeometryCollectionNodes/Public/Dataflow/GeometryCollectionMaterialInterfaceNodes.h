// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowTerminalNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "GeometryCollectionMaterialInterfaceNodes.generated.h"

class UMaterialInterface;
class UTexture2D;
class UTexture;

/**
* Make a array from a user defined list of material objects
*/
USTRUCT()
struct FMakeMaterialInterfaceArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeMaterialInterfaceArrayDataflowNode, "MakeMaterialArray", "Materials", "")

private:
	/** Material array set by the user */
	UPROPERTY(EditAnywhere, Category=Materials, meta=(DataflowOutput))
	TArray<TObjectPtr<UMaterialInterface>> MaterialArray;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FMakeMaterialInterfaceArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Get number of element in an material array 
* DEPRECATED 5.6 - use the generic GetArraySize node instead
*/
USTRUCT(meta = (Deprecated = 5.6))
struct FGetMaterialInterfaceArraySizeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetMaterialInterfaceArraySizeDataflowNode, "GetMaterialArraySize", "Materials", "")

private:
	/** Material array to get size from */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = MaterialArray, DataflowIntrinsic))
	TArray<TObjectPtr<UMaterialInterface>> MaterialArray;

	/** Size of the array */
	UPROPERTY(meta = (DataflowOutput))
	int32 Size = 0;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FGetMaterialInterfaceArraySizeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Get a material interface from an existing asset
*/
USTRUCT()
struct FGetMaterialInterfaceAssetDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetMaterialInterfaceAssetDataflowNode, "GetMaterialAsset", "Materials", "")

private:
	/** Material asset to get */
	UPROPERTY(EditAnywhere, Category = Material, meta = (DataflowOutput));
	TObjectPtr<UMaterialInterface> Material;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	virtual bool SupportsAssetProperty(UObject* Asset) const override;
	virtual void SetAssetProperty(UObject* Asset) override;

public:
	FGetMaterialInterfaceAssetDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Get an element from a material array
 * (if the index does not match the range of the array, null is returned)
 * DEPRECATED 5.6 - use the generic GetArrayElement node instead
 */
USTRUCT(meta = (Deprecated = 5.6))
struct FGetFromMaterialInterfaceArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetFromMaterialInterfaceArrayDataflowNode, "GetFromMaterialsArray", "Materials", "")

private:
	/** Material array to get the material from  */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = MaterialArray, DataflowIntrinsic))
	TArray<TObjectPtr<UMaterialInterface>> MaterialArray;

	/** Material at the requested index ( may be null if index does not match the array range ) */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UMaterialInterface> Material;

	/** Index in the array to get the material from. Invalid index will return null material */
	UPROPERTY(EditAnywhere, Category = Material, meta = (DataflowInput))
	int32 Index = 0;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FGetFromMaterialInterfaceArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Set an element into a material array at a specific index
 * (if the index does not match the range of the array, the array will remain unchanged)
 */
USTRUCT()
struct FSetIntoMaterialInterfaceArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetIntoMaterialInterfaceArrayDataflowNode, "SetIntoMaterialsArray", "Materials", "")

private:
	/** Material array to modify */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = MaterialArray, DataflowIntrinsic))
	TArray<TObjectPtr<UMaterialInterface>> MaterialArray;

	/** Material to set at the specific index into the array */
	UPROPERTY(EditAnywhere, Category = Material, meta = (DataflowInput))
	TObjectPtr<UMaterialInterface> Material;

	/** Index Set the material at (if the index does not match the range of the array, the array will remain unchanged)*/
	UPROPERTY(EditAnywhere, Category = Material, meta = (DataflowInput))
	int32 Index = 0;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FSetIntoMaterialInterfaceArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
* Add material(s) to an array
*/
USTRUCT()
struct FAddToMaterialInterfaceArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAddToMaterialInterfaceArrayDataflowNode, "AddToMaterialArray", "Materials", "")

private:
	/** Material array to add to */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = MaterialArray))
	TArray<TObjectPtr<UMaterialInterface>> MaterialArray;

	UPROPERTY()
	TArray<TObjectPtr<UMaterialInterface>> MaterialsToAdd;

	//~ Begin FDataflowNode interface
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override;
	virtual bool CanRemovePin() const override;
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	UE::Dataflow::TConnectionReference<TObjectPtr<UMaterialInterface>> GetConnectionReference(int32 Index) const;

	static constexpr int32 NumOtherInputs = 1; // MaterialArray
	static constexpr int32 NumInitialVariableInputs = 1; 

public:
	FAddToMaterialInterfaceArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Assign material to a set of face in a geometry collection
 */
USTRUCT()
struct FAssignMaterialInterfaceToCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAssignMaterialInterfaceToCollectionDataflowNode, "AssignMaterialToCollection", "Materials", "")

private:
	/** Collection to assign material to */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Collection, DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Faces that will be set with this material index, if no selection is connected , all faces will be set */
	UPROPERTY(meta = (DataflowInput))
	FDataflowFaceSelection FaceSelection;

	/** Array holding the materials objects */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = MaterialArray, DataflowIntrinsic))
	TArray<TObjectPtr<UMaterialInterface>> MaterialArray;

	/** Material to assign to the selection */
	UPROPERTY(EditAnywhere, Category = Material, meta = (DataflowInput))
	TObjectPtr<UMaterialInterface> Material;

	/** Index where the material was set in the array */
	UPROPERTY(meta = (DataflowOutput))
	int32 MaterialIndex = 0;

	/** If true, detect duplicate in the material array and only add the material in the array if it does not yet exists */
	UPROPERTY(EditAnywhere, Category = Material)
	bool bMergeDuplicateMaterials = false;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	/** add or merge a material and return the index where the material was set inthe array */
	int32 AddOrMergeMaterialToArray(TArray<TObjectPtr<UMaterialInterface>>& InOutMaterials, TObjectPtr<UMaterialInterface> InMaterialToAdd) const;

public:
	FAssignMaterialInterfaceToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};


/**
 * Duplicate the given material and replace the target texture with the override texture on the newly-created material
 */
USTRUCT(Meta = (MeshResizing, Experimental))
struct FMaterialInterfaceTextureOverrideDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMaterialInterfaceTextureOverrideDataflowNode, "MaterialInterfaceTextureOverride", "Materials", "Material Texture Override")

public:

	FMaterialInterfaceTextureOverrideDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Material"))
	TObjectPtr<UMaterialInterface> Material;

	UPROPERTY(EditAnywhere, Category = "Override", meta = (DataflowInput))
	TObjectPtr<const UTexture2D> TargetTexture;

	UPROPERTY(EditAnywhere, Category = "Override", meta=(DataflowInput))
	TObjectPtr<UTexture2D> OverrideTexture;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////

USTRUCT()
struct FDataflowMaterialFloatParameter
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FName Name;

	UPROPERTY()
	float Value = 0;
};

USTRUCT()
struct FDataflowMaterialVectorParameter
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FName Name;

	UPROPERTY()
	FVector4 Value = FVector4(0,0,0,0);
};

USTRUCT()
struct FDataflowMaterialTextureParameter
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FName Name;

	UPROPERTY()
	TObjectPtr<UTexture> Value = nullptr;
};

USTRUCT()
struct FDataflowMaterialParameters
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<FDataflowMaterialFloatParameter> FloatParameters;

	UPROPERTY()
	TArray<FDataflowMaterialVectorParameter> VectorParameters;

	UPROPERTY()
	TArray<FDataflowMaterialTextureParameter> TextureParameters;
};

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Add a material float parameter to a parameters object
 * Material parameters are used to create material instances
 */
USTRUCT(Meta = (MeshResizing, Experimental))
struct FMaterialInterfaceSetFloatParameterDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMaterialInterfaceSetFloatParameterDataflowNode, "SetMaterialFloatParameter", "Materials", "Instance Value")

public:
	FMaterialInterfaceSetFloatParameterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "MaterialParameters"))
	FDataflowMaterialParameters MaterialParameters;

	UPROPERTY(EditAnywhere, Category = "Parameter", meta = (DataflowInput))
	FDataflowStringTypes ParameterName;

	UPROPERTY(EditAnywhere, Category = "Parameter", meta = (DataflowInput))
	FDataflowNumericTypes ParameterValue;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Add a material vector4 parameter to a parameters object
 * Material parameters are used to create material instances
 */
USTRUCT(Meta = (MeshResizing, Experimental))
struct FMaterialInterfaceSetVectorParameterDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMaterialInterfaceSetVectorParameterDataflowNode, "SetMaterialVectorParameter", "Materials", "Instance Value")

public:
	FMaterialInterfaceSetVectorParameterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "MaterialParameters"))
	FDataflowMaterialParameters MaterialParameters;

	UPROPERTY(EditAnywhere, Category = "Parameter", meta = (DataflowInput))
	FDataflowStringTypes ParameterName;

	UPROPERTY(EditAnywhere, Category = "Parameter", meta = (DataflowInput))
	FDataflowVectorTypes ParameterValue;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Add a material texture parameter to a parameters object
 * Material parameters are used to create material instances
 */
USTRUCT(Meta = (MeshResizing, Experimental))
struct FMaterialInterfaceSetTexture2DParameterDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMaterialInterfaceSetTexture2DParameterDataflowNode, "SetMaterialTexture2DParameter", "Materials", "Instance Value")

public:
	FMaterialInterfaceSetTexture2DParameterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "MaterialParameters"))
	FDataflowMaterialParameters MaterialParameters;

	UPROPERTY(EditAnywhere, Category = "Parameter", meta = (DataflowInput))
	FDataflowStringTypes ParameterName;

	UPROPERTY(EditAnywhere, Category = "Parameter", meta = (DataflowInput))
	TObjectPtr<UTexture2D> ParameterValue;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};

//////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 * Make a material instance from another material 
 */
USTRUCT(Meta = (MeshResizing, Experimental))
struct FMaterialInterfaceTerminalDataflowNode : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMaterialInterfaceTerminalDataflowNode, "MaterialTerminal", "Terminal", "Instance Value Parameters")

public:
	FMaterialInterfaceTerminalDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Material"))
	TObjectPtr<UMaterialInterface> Material;

	UPROPERTY(EditAnywhere, Category = "Asset", meta = (DataflowInput))
	FString AssetPath;

	UPROPERTY(EditAnywhere, Category = "Parameters", meta = (DataflowInput))
	FDataflowMaterialParameters MaterialParameters;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context) const override;
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;
	//~ End FDataflowNode interface
};

namespace UE::Dataflow
{
	void RegisterGeometryCollectionMaterialInterfaceNodes();
}
