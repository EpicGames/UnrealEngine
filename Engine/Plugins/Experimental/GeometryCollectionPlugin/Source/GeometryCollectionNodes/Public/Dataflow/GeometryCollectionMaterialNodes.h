// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
//#include "DynamicMesh/DynamicMesh3.h"
//#include "UDynamicMesh.h"
#include "Dataflow/DataflowSelection.h"
//#include "Math/MathFwd.h"
//#include "DynamicMesh/DynamicMesh3.h"

#include "GeometryCollectionMaterialNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class FGeometryCollection;
class UGeometryCollection;
class UStaticMesh;
class UMaterial;

/**
 *
 * Adds Outside/Inside Materials to Outside/Inside faces
 * Deprecated (5.6) - Use AssignMaterialToCollection node instead
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FAddMaterialToCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAddMaterialToCollectionDataflowNode, "AddMaterialToCollection", "GeometryCollection|Materials", "")

public:
	/** Collection to add material(s) to */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Face selection, the material(s) will be added to the selected faces */
	UPROPERTY(meta = (DataflowInput, DisplayName = "FaceSelection", DataflowIntrinsic))
	FDataflowFaceSelection FaceSelection;

	/** Materials array storing the materials */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Materials", DataflowIntrinsic))
	TArray<TObjectPtr<UMaterial>> Materials;

	/** Outside material to assign to the outside faces from the face selection */
	UPROPERTY(EditAnywhere, Category = "Material", meta = (DataflowInput))
	TObjectPtr<UMaterial> OutsideMaterial;

	/** Inside material to assign to the inside faces from the face selection */
	UPROPERTY(EditAnywhere, Category = "Material", meta = (DataflowInput))
	TObjectPtr<UMaterial> InsideMaterial;

	/** If true, the outside material will be assigned to the outside faces from the face selection */
	UPROPERTY(EditAnywhere, Category = "Material", meta = (DisplayName = "Assign Outside Material to Outside Faces"))
	bool bAssignOutsideMaterial = true;

	/** If true, the inside material will be assigned to the inside faces from the face selection */
	UPROPERTY(EditAnywhere, Category = "Material", meta = (DisplayName = "Assign Inside Material to Inside Faces"))
	bool bAssignInsideMaterial = false;

	FAddMaterialToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&Materials);
		RegisterInputConnection(&FaceSelection);
		RegisterInputConnection(&OutsideMaterial);
		RegisterInputConnection(&InsideMaterial);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&Materials, &Materials);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Reassign existing material(s) to Outside/Inside faces
 * Deprecated (5.6) - Use AssignMaterialToCollection node instead
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FReAssignMaterialInCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
		DATAFLOW_NODE_DEFINE_INTERNAL(FReAssignMaterialInCollectionDataflowNode, "ReAssignMaterialInCollection", "GeometryCollection|Materials", "")

public:
	/** Collection for reassign the material(s) */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Face selection, the material(s) will be assigned to the selected faces */
	UPROPERTY(meta = (DataflowInput, DisplayName = "FaceSelection", DataflowIntrinsic))
	FDataflowFaceSelection FaceSelection;

	/** Materials array storing the materials */
	UPROPERTY(EditAnywhere, Category = "Material", meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Materials", DataflowIntrinsic))
	TArray<TObjectPtr<UMaterial>> Materials;

	/** Index of the material in the Materials array to assign to the outside faces from the face selection */
	UPROPERTY(EditAnywhere, Category = "Material", meta = (DataflowInput, DisplayName = "Outside Material Index", ArrayClamp = "Materials"))
	int32 OutsideMaterialIdx = -1;

	/** Index of the material in the Materials array to assign to the inside faces from the face selection */
	UPROPERTY(EditAnywhere, Category = "Material", meta = (DataflowInput, DisplayName = "Inside Material Index", ArrayClamp = "Materials"))
	int32 InsideMaterialIdx = -1;

	/** If true, the selected material from the Materials array will be assigned to the outside faces from the face selection */
	UPROPERTY(EditAnywhere, Category = "Material", meta = (DisplayName = "Assign Outside Material to Outside Faces"))
	bool bAssignOutsideMaterial = false;

	/** If true, the selected material from the Materials array will be assigned to the inside faces from the face selection */
	UPROPERTY(EditAnywhere, Category = "Material", meta = (DisplayName = "Assign Inside Material to Inside Faces"))
	bool bAssignInsideMaterial = false;

	FReAssignMaterialInCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&Materials);
		RegisterInputConnection(&FaceSelection);
		RegisterInputConnection(&OutsideMaterialIdx);
		RegisterInputConnection(&InsideMaterialIdx);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&Materials, &Materials);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Generates a formatted string of materials from the Materials array
 * Deprecated (5.6) 
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FMaterialsInfoDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
		DATAFLOW_NODE_DEFINE_INTERNAL(FMaterialsInfoDataflowNode, "MaterialsInfo", "GeometryCollection|Materials", "")

public:
	/** Materials array storing the materials */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<TObjectPtr<UMaterial>> Materials;

	/** Formatted string of the materials */
	UPROPERTY(meta = (DataflowOutput))
	FString String;

	FMaterialsInfoDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Materials);
		RegisterOutputConnection(&String);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Get a Material from a Materials array
 * Deprecated (5.6) - Use GetFromMaterialArray instead
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FGetMaterialFromMaterialsArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
		DATAFLOW_NODE_DEFINE_INTERNAL(FGetMaterialFromMaterialsArrayDataflowNode, "GetMaterialFromMaterialsArray", "Utilities|Materials", "")

public:
	/** Materials array storing the materials */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<TObjectPtr<UMaterial>> Materials;

	/** Selected material from the Materials array */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UMaterial> Material;

	/** Index in the Materials array for the selected material */
	UPROPERTY(EditAnywhere, Category = "Material", meta = (DataflowInput, DisplayName = "Material Index"))
	int32 MaterialIdx = 0;

	FGetMaterialFromMaterialsArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Materials);
		RegisterInputConnection(&MaterialIdx);
		RegisterOutputConnection(&Material);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


UENUM(BlueprintType)
enum class ESetMaterialOperationTypeEnum : uint8
{
	Dataflow_SetMaterialOperationType_Add UMETA(DisplayName = "Add"),
	Dataflow_SetMaterialOperationType_Insert UMETA(DisplayName = "Insert"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 * Set a Material in a Materials array
 * Deprecated (5.6) - Use SetIntoMaterialArray instead
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FSetMaterialInMaterialsArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
		DATAFLOW_NODE_DEFINE_INTERNAL(FSetMaterialInMaterialsArrayDataflowNode, "SetMaterialInMaterialsArray", "Utilities|Materials", "")

public:
	/** Materials array storing the materials */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Materials", DataflowIntrinsic))
	TArray<TObjectPtr<UMaterial>> Materials;

	/** Material to add/insert to/in Materials array */
	UPROPERTY(EditAnywhere, Category = "Material", meta = (DataflowInput))
	TObjectPtr<UMaterial> Material;

	/** Operation type for setting the material, add will add the new material to the end off Materials array, insert will insert the
	new material into Materials array at the index specified by MaterialIdx	*/
	UPROPERTY(EditAnywhere, Category = "Material")
	ESetMaterialOperationTypeEnum Operation = ESetMaterialOperationTypeEnum::Dataflow_SetMaterialOperationType_Add;

	/** Index for inserting a nem material into the Materials array */
	UPROPERTY(EditAnywhere, Category = "Material", meta = (DataflowInput, DisplayName = "Material Index", EditCondition = "Operation == ESetMaterialOperationTypeEnum::Dataflow_SetMaterialOperationType_Insert", EditConditionHides))
	int32 MaterialIdx = 0;

	FSetMaterialInMaterialsArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Materials);
		RegisterInputConnection(&Material);
		RegisterInputConnection(&MaterialIdx);
		RegisterOutputConnection(&Materials, &Materials);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Makes a material
 * Deprecated (5.6)
 *
 */
USTRUCT(meta=(Deprecated = "5.6"))
struct FMakeMaterialDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
		DATAFLOW_NODE_DEFINE_INTERNAL(FMakeMaterialDataflowNode, "MakeMaterial", "Generators|Material", "")

public:
	/** Material which will be outputed */
	UPROPERTY(EditAnywhere, Category = "Material", meta = (DisplayName = "Material"))
	TObjectPtr<UMaterial> InMaterial;

	/** Output material */
	UPROPERTY(meta = (DataflowOutput));
	TObjectPtr<UMaterial> Material;

	FMakeMaterialDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&Material);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Makes an empty Materials array
 * Deprecated (5.6)
 *
 */
USTRUCT(meta = (Deprecated = "5.6"))
struct FMakeMaterialsArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeMaterialsArrayDataflowNode, "MakeMaterialsArray", "Generators|Material", "")

public:
	/** Output Matarials array */
	UPROPERTY(meta = (DataflowOutput));
	TArray<TObjectPtr<UMaterial>> Materials;

	FMakeMaterialsArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&Materials);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


namespace UE::Dataflow
{
	void GeometryCollectionMaterialNodes();
}
