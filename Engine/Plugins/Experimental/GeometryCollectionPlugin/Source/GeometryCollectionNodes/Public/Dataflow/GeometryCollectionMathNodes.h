// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"

#include "GeometryCollectionMathNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/**
*
* 
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FAddDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAddDataflowNode, "Add", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Add", meta = (DataflowInput));
	float FloatA = 0.f;

	UPROPERTY(EditAnywhere, Category = "Add", meta = (DataflowInput));
	float FloatB = 0.f;
	
	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FAddDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatA);
		RegisterInputConnection(&FloatB);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FSubtractDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSubtractDataflowNode, "Subtract", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Subtract", meta = (DataflowInput));
	float FloatA = 0.f;

	UPROPERTY(EditAnywhere, Category = "Subtract", meta = (DataflowInput));
	float FloatB = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FSubtractDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatA);
		RegisterInputConnection(&FloatB);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FMultiplyDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMultiplyDataflowNode, "Multiply", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Multiply", meta = (DataflowInput));
	float FloatA = 0.f;

	UPROPERTY(EditAnywhere, Category = "Multiply", meta = (DataflowInput));
	float FloatB = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FMultiplyDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatA);
		RegisterInputConnection(&FloatB);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FSafeDivideDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSafeDivideDataflowNode, "SafeDivide", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "SafeDivide", meta = (DataflowInput));
	float FloatA = 0.f;

	UPROPERTY(EditAnywhere, Category = "SafeDivide", meta = (DataflowInput));
	float FloatB = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FSafeDivideDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatA);
		RegisterInputConnection(&FloatB);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FDivideDataflowNode : public FSafeDivideDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDivideDataflowNode, "Divide", "Math|Float", "")

public:
	FDivideDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FSafeDivideDataflowNode(InParam, InGuid)
	{}
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection))
struct FDivisionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDivisionDataflowNode, "Division (Whole and Remainder)", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Division", meta = (DataflowInput));
	float Dividend = 0.f;

	UPROPERTY(EditAnywhere, Category = "Division", meta = (DataflowInput));
	float Divisor = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float Remainder = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	int32 ReturnValue = 0.f;

	FDivisionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Dividend);
		RegisterInputConnection(&Divisor);
		RegisterOutputConnection(&Remainder);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FSafeReciprocalDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSafeReciprocalDataflowNode, "SafeReciprocal", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "SafeReciprocal", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FSafeReciprocalDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FSquareDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSquareDataflowNode, "Square", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Square", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FSquareDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FSquareRootDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSquareRootDataflowNode, "SquareRoot", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "SquareRoot", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FSquareRootDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FInverseSqrtDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FInverseSqrtDataflowNode, "InverseSqrt", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "InverseSqrt", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FInverseSqrtDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FCubeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCubeDataflowNode, "Cube", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Cube", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FCubeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FNegateDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FNegateDataflowNode, "Negate", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Negate", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FNegateDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FAbsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAbsDataflowNode, "Abs", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Abs", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FAbsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Converts a float to a nearest less or equal integer
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FFloorDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFloorDataflowNode, "Floor", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Floor", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FFloorDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Converts a float to the nearest greater or equal integer
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FCeilDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCeilDataflowNode, "Ceil", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Ceil", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FCeilDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Converts a float to the nearest integer. Rounds up when the fraction is .5
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FRoundDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRoundDataflowNode, "Round", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Round", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FRoundDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Converts a float to the nearest integer. Rounds up when the fraction is .5
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FTruncDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FTruncDataflowNode, "Trunc", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Trunc", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FTruncDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Returns the fractional part of a float
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FFracDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFracDataflowNode, "Frac", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Frac", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FFracDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FMinDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMinDataflowNode, "Min", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Min", meta = (DataflowInput));
	float FloatA = 0.f;

	UPROPERTY(EditAnywhere, Category = "Min", meta = (DataflowInput));
	float FloatB = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FMinDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatA);
		RegisterInputConnection(&FloatB);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FMaxDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMaxDataflowNode, "Max", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Max", meta = (DataflowInput));
	float FloatA = 0.f;

	UPROPERTY(EditAnywhere, Category = "Max", meta = (DataflowInput));
	float FloatB = 0.f;

	UPROPERTY(meta = (DataflowOutput))
		float ReturnValue = 0.f;

	FMaxDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatA);
		RegisterInputConnection(&FloatB);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FMin3DataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMin3DataflowNode, "Min3", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Min3", meta = (DataflowInput));
	float FloatA = 0.f;

	UPROPERTY(EditAnywhere, Category = "Min3", meta = (DataflowInput));
	float FloatB = 0.f;

	UPROPERTY(EditAnywhere, Category = "Min3", meta = (DataflowInput));
	float FloatC = 0.f;

	UPROPERTY(meta = (DataflowOutput))
		float ReturnValue = 0.f;

	FMin3DataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatA);
		RegisterInputConnection(&FloatB);
		RegisterInputConnection(&FloatC);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FMax3DataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMax3DataflowNode, "Max3", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Max3", meta = (DataflowInput));
	float FloatA = 0.f;

	UPROPERTY(EditAnywhere, Category = "Max3", meta = (DataflowInput));
	float FloatB = 0.f;

	UPROPERTY(EditAnywhere, Category = "Max3", meta = (DataflowInput));
	float FloatC = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FMax3DataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatA);
		RegisterInputConnection(&FloatB);
		RegisterInputConnection(&FloatC);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Returns 1, 0, or -1 depending on relation of Float to 0
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FSignDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSignDataflowNode, "Sign", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Sign", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FSignDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Clamps X to be between Min and Max, inclusive
* DEPRECATED 5.6: Use the Clamp node that support all numerics types
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FClampDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FClampDataflowNode, "Clamp", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Clamp", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(EditAnywhere, Category = "Clamp", meta = (DataflowInput));
	float Min = 0.f;

	UPROPERTY(EditAnywhere, Category = "Clamp", meta = (DataflowInput));
	float Max = 1.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FClampDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterInputConnection(&Min);
		RegisterInputConnection(&Max);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Fits a value from one range to another
* Returns a number between newmin and newmax that is relative to num in the range between oldmin and oldmax. 
* If the value is outside the old range, it will be clamped to the new range.
*/
USTRUCT(meta = (DataflowGeometryCollection))
struct FFitDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFitDataflowNode, "Fit", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Fit", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(EditAnywhere, Category = "Fit", meta = (DataflowInput));
	float OldMin = 0.f;

	UPROPERTY(EditAnywhere, Category = "Fit", meta = (DataflowInput));
	float OldMax = 1.f;

	UPROPERTY(EditAnywhere, Category = "Fit", meta = (DataflowInput));
	float NewMin = 0.f;

	UPROPERTY(EditAnywhere, Category = "Fit", meta = (DataflowInput));
	float NewMax = 1.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FFitDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterInputConnection(&OldMin);
		RegisterInputConnection(&OldMax);
		RegisterInputConnection(&NewMin);
		RegisterInputConnection(&NewMax);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Fits a value from one range to another
* 
* Takes the value num from the range (oldmin, oldmax) and shifts it to the corresponding value in the new range (newmin, newmax). Unlike fit, this function does not clamp values in the given range.
*/
USTRUCT(meta = (DataflowGeometryCollection))
struct FEFitDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FEFitDataflowNode, "EFit", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "EFit", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(EditAnywhere, Category = "EFit", meta = (DataflowInput));
	float OldMin = 0.f;

	UPROPERTY(EditAnywhere, Category = "EFit", meta = (DataflowInput));
	float OldMax = 1.f;

	UPROPERTY(EditAnywhere, Category = "EFit", meta = (DataflowInput));
	float NewMin = 0.f;

	UPROPERTY(EditAnywhere, Category = "EFit", meta = (DataflowInput));
	float NewMax = 1.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FEFitDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterInputConnection(&OldMin);
		RegisterInputConnection(&OldMax);
		RegisterInputConnection(&NewMin);
		RegisterInputConnection(&NewMax);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* 
* 
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FPowDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FPowDataflowNode, "Pow", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Pow", meta = (DataflowInput));
	float Base = 0.f;

	UPROPERTY(EditAnywhere, Category = "Pow", meta = (DataflowInput));
	float Exp = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FPowDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Base);
		RegisterInputConnection(&Exp);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Returns log of A base Base (if Base ^ R == A, where R is ReturnValue)
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FLogDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FLogDataflowNode, "Log", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Log", meta = (DataflowInput));
	float Base = 1.f;

	UPROPERTY(EditAnywhere, Category = "Log", meta = (DataflowInput));
	float A = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FLogDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&A);
		RegisterInputConnection(&Base);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Returns log of A base Base (if Base ^ R == A, where R is ReturnValue)
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FLogeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FLogeDataflowNode, "Loge", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Loge", meta = (DataflowInput));
	float A = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FLogeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&A);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Linearly interpolates between A and B based on Alpha. (100% of A when Alpha = 0.0 and 100% of B when Alpha = 1.0)
*
*/
USTRUCT(meta = (DataflowGeometryCollection))
struct FLerpDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FLerpDataflowNode, "Lerp", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Lerp", meta = (DataflowInput));
	float A = 0.f;

	UPROPERTY(EditAnywhere, Category = "Lerp", meta = (DataflowInput));
	float B = 1.f;

	UPROPERTY(EditAnywhere, Category = "Lerp", meta = (DataflowInput, UIMin = 0.f, UIMax = 100.f));
	float Alpha = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FLerpDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&A);
		RegisterInputConnection(&B);
		RegisterInputConnection(&Alpha);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* 
*
*/
USTRUCT(meta = (DataflowGeometryCollection))
struct FWrapDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FWrapDataflowNode, "Wrap", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Wrap", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(EditAnywhere, Category = "Wrap", meta = (DataflowInput));
	float Min = 0.f;

	UPROPERTY(EditAnywhere, Category = "Wrap", meta = (DataflowInput));
	float Max = 1.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FWrapDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterInputConnection(&Min);
		RegisterInputConnection(&Max);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FExpDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FExpDataflowNode, "Exp", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Exp", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FExpDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Returns the sine of A. (Expects Radians)
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FSinDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSinDataflowNode, "Sin", "Math|Trigonometry", "")

public:
	UPROPERTY(EditAnywhere, Category = "Sin", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FSinDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Returns the arcsine of A. (ReturnValue is in Radians)
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FArcSinDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FArcSinDataflowNode, "ArcSin", "Math|Trigonometry", "")

public:
	UPROPERTY(EditAnywhere, Category = "ArcSin", meta = (DataflowInput, UIMin = -1.f, UIMax = 1.f));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FArcSinDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Returns the cosine of A. (Expects Radians)
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FCosDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCosDataflowNode, "Cos", "Math|Trigonometry", "")

public:
	UPROPERTY(EditAnywhere, Category = "Cos", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FCosDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Returns the arccosine of A. (ReturnValue is in Radians)
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FArcCosDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FArcCosDataflowNode, "ArcCos", "Math|Trigonometry", "")

public:
	UPROPERTY(EditAnywhere, Category = "ArcCos", meta = (DataflowInput, UIMin = -1.f, UIMax = 1.f));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FArcCosDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Returns the cosine of A. (Expects Radians)
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FTanDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FTanDataflowNode, "Tan", "Math|Trigonometry", "")

public:
	UPROPERTY(EditAnywhere, Category = "Tan", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FTanDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Returns the arccosine of A. (ReturnValue is in Radians)
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FArcTanDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FArcTanDataflowNode, "ArcTan", "Math|Trigonometry", "")

public:
	UPROPERTY(EditAnywhere, Category = "ArcTan", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FArcTanDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Returns the arccosine of A. (ReturnValue is in Radians)
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FArcTan2DataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FArcTan2DataflowNode, "ArcTan2", "Math|Trigonometry", "")

public:
	UPROPERTY(EditAnywhere, Category = "ArcTan2", meta = (DataflowInput));
	float Y = 0.f;

	UPROPERTY(EditAnywhere, Category = "ArcTan2", meta = (DataflowInput));
	float X = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FArcTan2DataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Y);
		RegisterInputConnection(&X);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Returns Float normalized to the given range
*
*/
USTRUCT(meta = (DataflowGeometryCollection))
struct FNormalizeToRangeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FNormalizeToRangeDataflowNode, "NormalizeToRange", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "NormalizeToRange", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(EditAnywhere, Category = "NormalizeToRange", meta = (DataflowInput));
	float RangeMin = 0.f;

	UPROPERTY(EditAnywhere, Category = "NormalizeToRange", meta = (DataflowInput));
	float RangeMax = 1.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FNormalizeToRangeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterInputConnection(&RangeMin);
		RegisterInputConnection(&RangeMax);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FScaleVectorDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FScaleVectorDataflowNode, "ScaleVector", "Math|Vector", "")

public:
	UPROPERTY(EditAnywhere, Category = "ScaleVector", meta = (DataflowInput));
	FVector Vector = FVector(0.f);

	UPROPERTY(EditAnywhere, Category = "ScaleVector", meta = (DataflowInput));
	float Scale = 1.f;

	UPROPERTY(meta = (DataflowOutput))
	FVector ScaledVector = FVector(0.f);

	FScaleVectorDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Vector);
		RegisterInputConnection(&Scale);
		RegisterOutputConnection(&ScaledVector);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FDotProductDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDotProductDataflowNode, "DotProduct", "Math|Vector", "")

public:
	UPROPERTY(EditAnywhere, Category = "DotProduct", meta = (DataflowInput));
	FVector VectorA = FVector(0.f);

	UPROPERTY(EditAnywhere, Category = "DotProduct", meta = (DataflowInput));
	FVector VectorB = FVector(0.f);

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FDotProductDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&VectorA);
		RegisterInputConnection(&VectorB);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FCrossProductDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCrossProductDataflowNode, "CrossProduct", "Math|Vector", "")

public:
	UPROPERTY(EditAnywhere, Category = "CrossProduct", meta = (DataflowInput));
	FVector VectorA = FVector(0.f);

	UPROPERTY(EditAnywhere, Category = "CrossProduct", meta = (DataflowInput));
	FVector VectorB = FVector(0.f);

	UPROPERTY(meta = (DataflowOutput))
	FVector ReturnValue = FVector(0.f);

	FCrossProductDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&VectorA);
		RegisterInputConnection(&VectorB);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FNormalizeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FNormalizeDataflowNode, "Normalize", "Math|Vector", "")

public:
	UPROPERTY(EditAnywhere, Category = "Normalize", meta = (DataflowInput));
	FVector VectorA = FVector(0.f);

	UPROPERTY(EditAnywhere, Category = "Normalize", meta = (DataflowInput));
	float Tolerance = 0.0001f;

	UPROPERTY(meta = (DataflowOutput))
	FVector ReturnValue = FVector(0.f);

	FNormalizeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&VectorA);
		RegisterInputConnection(&Tolerance);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FLengthDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FLengthDataflowNode, "Length", "Math|Vector", "")

public:
	UPROPERTY(EditAnywhere, Category = "Length", meta = (DataflowInput));
	FVector Vector = FVector(0.f);

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FLengthDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Vector);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
*
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FDistanceDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDistanceDataflowNode, "Distance", "Math|Points", "")

public:
	UPROPERTY(EditAnywhere, Category = "Distance", meta = (DataflowInput));
	FVector PointA = FVector(0.f);

	UPROPERTY(EditAnywhere, Category = "Distance", meta = (DataflowInput));
	FVector PointB = FVector(0.f);

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FDistanceDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&PointA);
		RegisterInputConnection(&PointB);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* 
*
*/
USTRUCT(meta = (DataflowGeometryCollection))
struct FIsNearlyZeroDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FIsNearlyZeroDataflowNode, "IsNearlyZero", "Math|Utilities", "")

public:
	UPROPERTY(EditAnywhere, Category = "IsNearlyZero", meta = (DataflowInput));
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	bool ReturnValue = false;

	FIsNearlyZeroDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Float);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Generates a random float
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FRandomFloatDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRandomFloatDataflowNode, "RandomFloat", "Math|Random", "")

public:
	UPROPERTY(EditAnywhere, Category = "Seed")
	bool bDeterministic = false;

	UPROPERTY(EditAnywhere, Category = "Seed", meta = (DataflowInput, EditCondition = "bDeterministic"))
	float RandomSeed = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FRandomFloatDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RandomSeed = FMath::FRandRange(-1e5, 1e5);
		RegisterInputConnection(&RandomSeed);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Generates a random float between Min and Max
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FRandomFloatInRangeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRandomFloatInRangeDataflowNode, "RandomFloatInRange", "Math|Random", "")

public:
	UPROPERTY(EditAnywhere, Category = "Seed")
	bool bDeterministic = false;

	UPROPERTY(EditAnywhere, Category = "Seed", meta = (DataflowInput, EditCondition = "bDeterministic"))
	float RandomSeed = 0.f;

	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput))
	float Min = 0.f;

	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput))
	float Max = 1.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FRandomFloatInRangeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RandomSeed = FMath::FRandRange(-1e5, 1e5);
		RegisterInputConnection(&RandomSeed);
		RegisterInputConnection(&Min);
		RegisterInputConnection(&Max);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Returns a random vector with length of 1
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FRandomUnitVectorDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRandomUnitVectorDataflowNode, "RandomUnitVector", "Math|Random", "")

public:
	UPROPERTY(EditAnywhere, Category = "Seed")
	bool bDeterministic = false;

	UPROPERTY(EditAnywhere, Category = "Seed", meta = (DataflowInput, EditCondition = "bDeterministic"))
	float RandomSeed = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	FVector ReturnValue = FVector(0.0);

	FRandomUnitVectorDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RandomSeed = FMath::FRandRange(-1e5, 1e5);
		RegisterInputConnection(&RandomSeed);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Returns a random vector with length of 1, within the specified cone, with uniform random distribution
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FRandomUnitVectorInConeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRandomUnitVectorInConeDataflowNode, "RandomUnitVectorInCone", "Math|Random", "")

public:
	UPROPERTY(EditAnywhere, Category = "Seed")
	bool bDeterministic = false;

	UPROPERTY(EditAnywhere, Category = "Seed", meta = (DataflowInput, EditCondition = "bDeterministic"))
	float RandomSeed = 0.f;

	/** The base "center" direction of the cone */
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput))
	FVector ConeDirection = FVector(0.0, 0.0, 1.0);

	/** The half-angle of the cone (from ConeDir to edge), in degrees */
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput))
	float ConeHalfAngle = PI / 4.f;

	UPROPERTY(meta = (DataflowOutput))
	FVector ReturnValue = FVector(0.0);

	FRandomUnitVectorInConeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RandomSeed = FMath::FRandRange(-1e5, 1e5);
		RegisterInputConnection(&RandomSeed);
		RegisterInputConnection(&ConeDirection);
		RegisterInputConnection(&ConeHalfAngle);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Converts radians to degrees
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FRadiansToDegreesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRadiansToDegreesDataflowNode, "RadiansToDegrees", "Math|Trigonometry", "")

public:
	UPROPERTY(EditAnywhere, Category = "Radians", meta = (DataflowInput))
	float Radians = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float Degrees = 0.f;

	FRadiansToDegreesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Radians);
		RegisterOutputConnection(&Degrees);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Converts degrees to radians
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FDegreesToRadiansDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDegreesToRadiansDataflowNode, "DegreesToRadians", "Math|Trigonometry", "")

public:
	UPROPERTY(EditAnywhere, Category = "Degrees", meta = (DataflowInput))
	float Degrees = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float Radians = 0.f;

	FDegreesToRadiansDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Degrees);
		RegisterOutputConnection(&Radians);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

UENUM(BlueprintType)
enum class EMathConstantsEnum : uint8
{
	Dataflow_MathConstants_Pi UMETA(DisplayName = "Pi"),
	Dataflow_MathConstants_HalfPi UMETA(DisplayName = "HalfPi"),
	Dataflow_MathConstants_TwoPi UMETA(DisplayName = "TwoPi"),
	Dataflow_MathConstants_FourPi UMETA(DisplayName = "FourPi"),
	Dataflow_MathConstants_InvPi UMETA(DisplayName = "InvPi"),
	Dataflow_MathConstants_InvTwoPi UMETA(DisplayName = "InvTwoPi"),
	Dataflow_MathConstants_Sqrt2 UMETA(DisplayName = "Sqrt2"),
	Dataflow_MathConstants_InvSqrt2 UMETA(DisplayName = "InvSqrt2"),
	Dataflow_MathConstants_Sqrt3 UMETA(DisplayName = "Sqrt3"),
	Dataflow_MathConstants_InvSqrt3 UMETA(DisplayName = "InvSqrt3"),
	Dataflow_FloatToInt_Function_E UMETA(DisplayName = "e"),
	Dataflow_FloatToInt_Function_Gamma UMETA(DisplayName = "Gamma"),
	Dataflow_FloatToInt_Function_GoldenRatio UMETA(DisplayName = "GoldenRatio"),
	Dataflow_FloatToInt_Function_ZeroTolerance UMETA(DisplayName = "ZeroTolerance"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 * Offers a selection of Math constants
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FMathConstantsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMathConstantsDataflowNode, "MathConstants", "Math|Utilities", "")

public:
	/** Math constant to output */
	UPROPERTY(EditAnywhere, Category = "Constants");
	EMathConstantsEnum Constant = EMathConstantsEnum::Dataflow_MathConstants_Pi;

	/** Selected Math constant */
	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0;

	FMathConstantsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
*
* Returns 1.0 - A
*
*/
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FOneMinusDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FOneMinusDataflowNode, "OneMinus", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "OneMinus", meta = (DataflowInput));
	float A = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FOneMinusDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&A);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Expression node for floats
*
*/
USTRUCT()
struct FFloatMathExpressionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFloatMathExpressionDataflowNode, "FloatMathExpression", "Math|Float", "")

public:
	UPROPERTY(EditAnywhere, Category = "Variables", meta = (DataflowInput));
	float A = 0.f;

	UPROPERTY(EditAnywhere, Category = "Variables", meta = (DataflowInput));
	float B = 0.f;

	UPROPERTY(EditAnywhere, Category = "Variables", meta = (DataflowInput));
	float C = 0.f;

	UPROPERTY(EditAnywhere, Category = "Variables", meta = (DataflowInput));
	float D = 0.f;

	UPROPERTY(EditAnywhere, Category = "Expression");
	FString Expression;

	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FFloatMathExpressionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
*
* Expression node for floats
*
*/
USTRUCT()
struct FMathExpressionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMathExpressionDataflowNode, "MathExpression", "Math", "")

public:
	UPROPERTY(EditAnywhere, Category = Inputs, meta = (DataflowInput));
	FDataflowNumericTypes A;

	UPROPERTY(EditAnywhere, Category = Inputs, meta = (DataflowInput));
	FDataflowNumericTypes B;

	UPROPERTY(EditAnywhere, Category = Inputs, meta = (DataflowInput));
	FDataflowNumericTypes C;

	UPROPERTY(EditAnywhere, Category = Inputs, meta = (DataflowInput));
	FDataflowNumericTypes D;

	UPROPERTY(EditAnywhere, Category = Expression);
	FString Expression;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowNumericTypes ReturnValue;

	FMathExpressionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void GeometryCollectionMathNodes();
}

