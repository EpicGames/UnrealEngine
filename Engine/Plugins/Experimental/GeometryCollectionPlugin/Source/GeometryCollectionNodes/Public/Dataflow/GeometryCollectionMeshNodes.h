// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowMesh.h"
#include "Dataflow/DataflowDynamicConnections.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowSelection.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshProcessor.h"
#include "Dataflow/DataflowMesh.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
// Some mesh-specific nodes have been moved to their own plugin
#include "Dataflow/MeshBooleanNodes.h"
#endif

#include "GeometryCollectionMeshNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class FGeometryCollection;
class UStaticMesh;
class UDynamicMesh;
class UMaterialInterface;

/**
 *
 * Converts points into a DynamicMesh
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FPointsToMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FPointsToMeshDataflowNode, "PointsToMesh", "Mesh|Utilities", "")

public:
	/** Points input */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector> Points;

	/** Mesh output */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

	/** Mesh triangle count */
	UPROPERTY(meta = (DataflowOutput))
	int32 TriangleCount = 0;

	FPointsToMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Points);
		RegisterOutputConnection(&Mesh);
		RegisterOutputConnection(&TriangleCount);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 * Base class for nodes that applying Geometry Script Mesh Processors
 */
USTRUCT()
struct FMeshProcessorDataflowNodeBase 
	: public FDataflowNode
#if CPP
	, public FDataflowDynamicConnections::IOwnerInterface
#endif
{
	GENERATED_USTRUCT_BODY()

public:

	FMeshProcessorDataflowNodeBase() = default;

	FMeshProcessorDataflowNodeBase(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid), OwningObject(InParam.OwningObject)
		, DynamicConnections(UE::Dataflow::FPin::EDirection::INPUT, this, Cast<UDataflow>(InParam.OwningObject))
	{
	}

	~FMeshProcessorDataflowNodeBase()
	{
		TeardownBlueprintEvent();
	}

protected:

	UPROPERTY(EditAnywhere, Category = "Processor Type")
	TSubclassOf<UDynamicMeshProcessorBlueprint> MeshProcessor;

	UPROPERTY(EditAnywhere, Instanced, NoClear, Category = "Instance", meta = (ShowOnlyInnerProperties))
	TObjectPtr<UDynamicMeshProcessorBlueprint> MeshProcessorInstance;

	virtual void PostSerialize(const FArchive& Ar) override final;

	void ApplyParametersToBlueprintInstance(UE::Dataflow::FContext& Context) const;

private:

	/* Begin IOwnerInterface */
	virtual FDataflowNode* GetOwner(const FDataflowDynamicConnections* Caller) override { return this; }
	virtual const FInstancedPropertyBag& GetPropertyBag(const FDataflowDynamicConnections* Caller) override { return PropertyBag; }
	/* End IOwnerInterface */

	virtual void OnPropertyChanged(UE::Dataflow::FContext& Context, const FPropertyChangedEvent& PropertyChangedEvent) override;

	// Evaluate on game thread unless the blueprint opts in to async evaluation
	virtual bool EvaluateOnGameThreadOnly() const override
	{
		return MeshProcessorInstance && MeshProcessorInstance->bRequiresGameThread;
	}

#if WITH_EDITOR
	virtual void OnDoubleClicked(UE::Dataflow::FContext* Context) const override;
#endif

	// Handling for the selected blueprint being changed under the node (e.g., recompiled)
	FDelegateHandle BlueprintChangeDelegateHandle;
	void TeardownBlueprintEvent();
	void SetupBlueprintEvent();
	void RefreshConnectionsFromBlueprint();

	// Remember the parent UObject so that we can properly parent MeshProcessorInstance when the MeshProcessor changes 
	UPROPERTY(Transient)
	TObjectPtr<UObject> OwningObject;

	UPROPERTY()
	FDataflowDynamicConnections DynamicConnections;

	UPROPERTY()
	FInstancedPropertyBag PropertyBag;

	// Cached mapping from property bag names to their source blueprint property names, which can be different due to sanitization
	TMap<FName, FName> CachedPropertyNameMap;
};

/**
 * Apply a Geometry Script Mesh Processors to an input UDynamicMesh
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FApplyMeshProcessorToMeshDataflowNode final : public FMeshProcessorDataflowNodeBase
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FApplyMeshProcessorToMeshDataflowNode, "ApplyGeometryScriptToMesh", "Mesh|Utilities", "")

public:

	/** Input/Output mesh */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Mesh", DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> Mesh;

	FApplyMeshProcessorToMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FMeshProcessorDataflowNodeBase(InParam, InGuid)
	{
		RegisterInputConnection(&Mesh);
		RegisterOutputConnection(&Mesh, &Mesh);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Apply a Geometry Script Mesh Processors to the geometry of selected transforms in a geometry collection
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FApplyMeshProcessorToGeometryCollectionDataflowNode final : public FMeshProcessorDataflowNodeBase
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FApplyMeshProcessorToGeometryCollectionDataflowNode, "ApplyGeometryScriptToCollection", "Mesh|Utilities", "")

public:

	/** Input/Output mesh */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Selected bones will have geometry script processing applied (if they have geometry). If not connected, all geometry will be processed. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection TransformSelection;

	/** Whether the processed mesh will have edges at normal/UV/color seams welded so they are treated as one edge during processing. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bWeldVertices = true;

	/** Whether to preserve isolated vertices which aren't used by any triangles. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bPreserveIsolatedVertices = true;

	FApplyMeshProcessorToGeometryCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FMeshProcessorDataflowNodeBase(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);

		RegisterInputConnection(&TransformSelection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Converts a Collection to a set of Dynamic Meshes per selected Transform
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionSelectionToMeshesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSelectionToMeshesDataflowNode, "CollectionSelectionToMeshes", "Mesh|Utilities", "")

public:
	/** Collection to convert*/
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Geometry on or under selected bones will be converted to meshes, optionally after filtering the selection to leaves. If not connected, all geometry will be processed. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection TransformSelection;

	/** Whether to convert the input selection to only leaves, which may directly store geometry. Otherwise, meshes for selected cluster nodes will be generated by appending together geometry from leaf nodes. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bConvertSelectionToLeaves = true;

	/** Whether the processed mesh will have edges at normal/UV/color seams welded so they are treated as one edge during processing. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bWeldVertices = true;

	/** Whether to preserve isolated vertices which aren't used by any triangles. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bPreserveIsolatedVertices = true;

	/** Output Array of DynamicMesh */
	UPROPERTY(meta = (DataflowOutput))
	TArray<TObjectPtr<UDynamicMesh>> Meshes;

	FCollectionSelectionToMeshesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Append Array of Meshes to Collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FAppendMeshesToCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAppendMeshesToCollectionDataflowNode, "AppendMeshesToCollection", "Mesh|Utilities", "")

public:
	/** Meshes will be appended to this collection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;
	
	/** Selection of added transforms */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowTransformSelection AddedSelection;

	/** Dynamic Meshes to append */
	UPROPERTY(meta = (DataflowInput))
	TArray<TObjectPtr<UDynamicMesh>> Meshes;

	/** Index of parent bone for appended meshes. If invalid, meshes will be appended to a root node. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	int32 ParentIndex = -1;

	FAppendMeshesToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Converts a BoundingBox into a DynamicMesh
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FBoxToMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBoxToMeshDataflowNode, "BoxToMesh", "Mesh|Utilities", "")

public:
	/** BoundingBox input */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FBox Box = FBox(ForceInit);

	/** Mesh output */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

	/** Mesh triangle count */
	UPROPERTY(meta = (DataflowOutput))
	int32 TriangleCount = 0;

	FBoxToMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Box);
		RegisterOutputConnection(&Mesh);
		RegisterOutputConnection(&TriangleCount);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Collects information from the DynamicMesh and outputs it into a formatted string
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FMeshInfoDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMeshInfoDataflowNode, "MeshInfo", "Mesh|Utilities", "")

public:
	/** DynamicMesh for the information */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> Mesh;

	/** Formatted output string */
	UPROPERTY(meta = (DataflowOutput))
	FString InfoString = FString("");

	FMeshInfoDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Mesh);
		RegisterOutputConnection(&InfoString);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 * Converts a DynamicMesh to a Collection
 * DEPRECATED 5.8 - use FMeshToCollectionDataflowNode_v2 instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FMeshToCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMeshToCollectionDataflowNode, "MeshToCollection", "Mesh|Utilities", "")

public:
	/** DynamicMesh to convert */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> Mesh;

	/** Output Collection */
	UPROPERTY(meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	/** Whether to split the mesh into multiple bones based on the mesh connectivity */
	UPROPERTY(EditAnywhere, Category = "General|SplitIslands", meta = (DataflowInput));
	bool bSplitIslands = false;

	/** Whether to consider coincident vertices as connected even if the topology does not connect them */
	UPROPERTY(EditAnywhere, Category = "General|SplitIslands", meta = (EditCondition = "bSplitIslands"));
	bool bConnectIslandsByVertexOverlap = false;

	/** Vertices closer than this distance are considered to be overlapping */
	UPROPERTY(EditAnywhere, Category = "General|SplitIslands", meta = (EditCondition = "bSplitIslands", Units = "cm", ClampMin = 0.0));
	float ConnectVerticesThreshold = 0.001f;

	/** If > 0, bridge separate islands whose surfaces are within this vertex-to-triangle distance. 0 = disabled. */
	UPROPERTY(EditAnywhere, Category = "General|SplitIslands", meta = (EditCondition = "bSplitIslands", Units = "cm", ClampMin = 0.0));
	float VertexToSurfaceBridgeDistance = 0.f;

	/** Whether to add a root cluster for the single mesh case. Note if the mesh is split, the root cluster will always be added. */
	UPROPERTY(EditAnywhere, Category = "General", meta = (DataflowInput));
	bool bAddClusterRootForSingleMesh = false;

	FMeshToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Converts a DynamicMesh to a Collection
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FMeshToCollectionDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMeshToCollectionDataflowNode_v2, "MeshToCollection", "Mesh|Utilities", "")

private:
	/** DynamicMesh to convert */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDataflowMesh> Mesh;

	/** Output Collection */
	UPROPERTY(meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	/** Material array output */
	UPROPERTY(meta = (DataflowOutput))
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	/** Whether to split the mesh into multiple bones based on the mesh connectivity */
	UPROPERTY(EditAnywhere, Category = "General|SplitIslands", meta = (DataflowInput));
	bool bSplitIslands = false;

	/** Whether to consider coincident vertices as connected even if the topology does not connect them */
	UPROPERTY(EditAnywhere, Category = "General|SplitIslands", meta = (EditCondition = "bSplitIslands"));
	bool bConnectIslandsByVertexOverlap = false;

	/** Vertices closer than this distance are considered to be overlapping */
	UPROPERTY(EditAnywhere, Category = "General|SplitIslands", meta = (EditCondition = "bSplitIslands", Units = "cm", ClampMin = 0.0));
	float ConnectVerticesThreshold = 0.001f;

	/** If > 0, bridge separate islands whose surfaces are within this vertex-to-triangle distance. 0 = disabled. */
	UPROPERTY(EditAnywhere, Category = "General|SplitIslands", meta = (EditCondition = "bSplitIslands", Units = "cm", ClampMin = 0.0));
	float VertexToSurfaceBridgeDistance = 0.f;

	/** Whether to add a root cluster for the single mesh case. Note if the mesh is split, the root cluster will always be added. */
	UPROPERTY(EditAnywhere, Category = "General", meta = (DataflowInput));
	bool bAddClusterRootForSingleMesh = false;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FMeshToCollectionDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Converts a Collection to a DynamicMesh
 * DEPRECATED 5.8 - use FCollectionToMeshDataflowNode_v2 instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionToMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionToMeshDataflowNode, "CollectionToMesh", "Mesh|Utilities", "")

public:
	/** Collection to convert*/
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Whether to translate the mesh geometry to be centered around its bounding box. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bCenterPivot = false;

	/** Output DynamicMesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

	FCollectionToMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterOutputConnection(&Mesh);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	
private:
	/** Geometry on or under selected bones will be appended to the output mesh. If not connected, all geometry will be processed. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection TransformSelection;

	/** Whether the processed mesh will have edges at normal/UV/color seams welded so they are treated as one edge during processing. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bWeldVertices = true;

	/** Whether to preserve isolated vertices which aren't used by any triangles. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bPreserveIsolatedVertices = true;


};

/**
 * Converts a Collection to a DataflowMesh
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionToMeshDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionToMeshDataflowNode_v2, "CollectionToMesh", "Mesh|Utilities", "")

private:
	/** Collection to convert*/
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Material array from the static mesh */
	UPROPERTY(meta = (DataflowInput))
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	/** Whether to translate the mesh geometry to be centered around its bounding box. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bCenterPivot = false;

	/** Geometry on or under selected bones will be appended to the output mesh. If not connected, all geometry will be processed. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection TransformSelection;

	/** Whether the processed mesh will have edges at normal/UV/color seams welded so they are treated as one edge during processing. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bWeldVertices = true;

	/** Whether to preserve isolated vertices which aren't used by any triangles. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bPreserveIsolatedVertices = true;

	/** Output dataflow mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDataflowMesh> Mesh;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FCollectionToMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Converts a StaticMesh into a DynamicMesh
 * DEPRECATED 5.8 - use FStaticMeshToMeshDataflowNode_v2 instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FStaticMeshToMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FStaticMeshToMeshDataflowNode, "StaticMeshToMesh", "Mesh|Utilities", "")

public:
	/** StaticMesh to convert */
	UPROPERTY(EditAnywhere, Category = "StaticMesh", meta = (DataflowInput));
	TObjectPtr<UStaticMesh> StaticMesh;

	/** Output the HiRes representation, if set to true and HiRes doesn't exist it will output empty mesh */
	UPROPERTY(EditAnywhere, Category = "StaticMesh", meta = (DisplayName = "Use HiRes"));
	bool bUseHiRes = false;

	/** Specifies the LOD level to use */
	UPROPERTY(EditAnywhere, Category = "StaticMesh", meta = (DisplayName = "LOD Level"));
	int32 LODLevel = 0;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

	/** Output materials */
	UPROPERTY(meta = (DataflowOutput))
	TArray<TObjectPtr<UMaterialInterface>> MaterialArray;

	FStaticMeshToMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&StaticMesh);
		RegisterOutputConnection(&Mesh);
		RegisterOutputConnection(&MaterialArray);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 * Get the local geometric bounding box a dynamic mesh
 * DEPRECATED 5.8 - use FGetMeshBoundingBoxDataflowNode_v2 instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FGetMeshBoundingBoxDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetMeshBoundingBoxDataflowNode, "GetMeshBoundingBox", "Mesh", "Bounds Size Dimensions Extents Center")

private:
	/** dynamic mesh to compute the bouning box from */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<const UDynamicMesh> Mesh = nullptr;

	/** Geometric bounding box of the input dynamic mesh */
	UPROPERTY(meta = (DataflowOutput))
	FBox BoundingBox = FBox(ForceInit);

	/** Center of the resulting bounding box */
	UPROPERTY(meta = (DataflowOutput))
	FVector Center = FVector::ZeroVector;

	/** Dimensions of the resulting bounding box in centimers */
	UPROPERTY(meta = (DataflowOutput))
	FVector Dimensions = FVector::ZeroVector;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FGetMeshBoundingBoxDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

};

/**
 * Get the local geometric bounding box of a dataflow mesh
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGetMeshBoundingBoxDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetMeshBoundingBoxDataflowNode_v2, "GetMeshBoundingBox", "Mesh", "Bounds Size Dimensions Extents Center")

private:
	/** dynamic mesh to compute the bouning box from */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<const UDataflowMesh> Mesh = nullptr;

	/** Geometric bounding box of the input dynamic mesh */
	UPROPERTY(meta = (DataflowOutput))
	FBox BoundingBox = FBox(ForceInit);

	/** Center of the resulting bounding box */
	UPROPERTY(meta = (DataflowOutput))
	FVector Center = FVector::ZeroVector;

	/** Dimensions of the resulting bounding box in centimers */
	UPROPERTY(meta = (DataflowOutput))
	FVector Dimensions = FVector::ZeroVector;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FGetMeshBoundingBoxDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

};

/**
 *
 * Appends two meshes
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FMeshAppendDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMeshAppendDataflowNode, "MeshAppend", "Mesh|Utilities", "")

public:
	/** Mesh input */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> Mesh1;

	/** Mesh input */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> Mesh2;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

	FMeshAppendDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Mesh1);
		RegisterInputConnection(&Mesh2);
		RegisterOutputConnection(&Mesh);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Combine two Dataflow meshes
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FDataflowMeshAppendDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDataflowMeshAppendDataflowNode, "DataflowMeshAppend", "Mesh|Utilities", "")

public:

	FDataflowMeshAppendDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	/** Mesh input/output */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Mesh", DataflowIntrinsic))
	TObjectPtr<UDataflowMesh> Mesh;

	/** Mesh to append  */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<const UDataflowMesh> AppendMesh;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Create a UDataflow mesh from an input UDynamicMesh and material array
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FMakeDataflowMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeDataflowMeshDataflowNode, "MakeDataflowMesh", "Mesh|Utilities", "")

public:

	FMakeDataflowMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	/** DynamicMesh input */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> InMesh;

	/** Materials input */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<TObjectPtr<UMaterialInterface>> InMaterials;

	/** DataflowMesh output */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDataflowMesh> Mesh;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


/**
 * Create a new UV layer/channel in a UDataflowMesh
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FDuplicateMeshUVChannelNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FDuplicateMeshUVChannelNode, "DuplicateMeshUVChannelNode", "Mesh|Utilities", "Mesh UV DataflowMesh")

public:

	FDuplicateMeshUVChannelNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	/** DataflowMesh input/output */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Mesh", DataflowIntrinsic))
	TObjectPtr<UDataflowMesh> Mesh;

	/** Index of the source UV channel */
	UPROPERTY(EditAnywhere, Category = Options)
	int32 SourceUVChannel = -1;

	/** Index of the added UV channel */
	UPROPERTY(meta = (DataflowOutput, DataflowUVChannel))
	int32 NewUVChannel = -1;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


UENUM(BlueprintType)
enum class EDataflowMeshSplitIslandsMethod : uint8
{
	NoSplit,
	ByMeshTopology,
	ByVertexOverlap
};


/**
 *
 * Split a mesh into a connected islands
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FSplitMeshIslandsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSplitMeshIslandsDataflowNode, "SplitMeshIslands", "Mesh|Utilities", "")

public:

	FSplitMeshIslandsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	/** Mesh input */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> Mesh;

	/** Meshes output */
	UPROPERTY(meta = (DataflowOutput))
	TArray<TObjectPtr<UDynamicMesh>> Meshes;

	/** Whether to consider coincident vertices as connected even if the topology does not connect them */
	UPROPERTY(EditAnywhere, Category = "General");
	EDataflowMeshSplitIslandsMethod SplitMethod = EDataflowMeshSplitIslandsMethod::ByMeshTopology;

	/** Vertices closer than this distance are considered to be overlapping */
	UPROPERTY(EditAnywhere, Category = "General", meta = (ClampMin = 0.0, EditCondition = "SplitMethod == EDataflowMeshSplitIslandsMethod::ByVertexOverlap"));
	float ConnectVerticesThreshold = 0.001f;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Split a UDataflow mesh into a UDynamicMesh and a material array
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FSplitDataflowMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSplitDataflowMeshDataflowNode, "SplitDataflowMesh", "Mesh|Utilities", "")

public:

	FSplitDataflowMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	/** DataflowMesh input */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDataflowMesh> InMesh;

	/** DyanmicMesh output */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

	/** Materials output */
	UPROPERTY(meta = (DataflowOutput))
	TArray<TObjectPtr<UMaterialInterface>> MaterialArray;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	virtual void OnRenderOutput(UE::Dataflow::FContext& Context, const FName OutputName, const FName RenderGroup, const TArray<UPrimitiveComponent*>& RenderComponents) const override;
};




/**
 *
 * Copies the same mesh with scale onto points
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FMeshCopyToPointsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMeshCopyToPointsDataflowNode, "ScatterMeshes", "Mesh|Utilities", "")

public:
	/** Points to copy meshes onto */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<FVector> Points;

	/** Mesh to copy onto points */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic, DisplayName = "MeshToScatter"))
	TObjectPtr<UDynamicMesh> MeshToCopy;

	/** Scale applied to the mesh */
	UPROPERTY(EditAnywhere, Category = "Copy");
	float Scale = 1.f;

	/** merged result mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

	/** Result meshes as individual ones */
	UPROPERTY(meta = (DataflowOutput))
	TArray<TObjectPtr<UDynamicMesh>> Meshes;

	FMeshCopyToPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Points);
		RegisterInputConnection(&MeshToCopy);
		RegisterOutputConnection(&Mesh);
		RegisterOutputConnection(&Meshes);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Outputs Mesh data
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGetMeshDataDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetMeshDataDataflowNode, "GetMeshData", "Mesh|Utilities", "")

public:
	/** Mesh for the data */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> Mesh;

	/** Number of vertices */
	UPROPERTY(meta = (DataflowOutput))
	int32 VertexCount = 0;

	/** Number of edges */
	UPROPERTY(meta = (DataflowOutput))
	int32 EdgeCount = 0;

	/** Number of triangles */
	UPROPERTY(meta = (DataflowOutput))
	int32 TriangleCount = 0;

	FGetMeshDataDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Mesh);
		RegisterOutputConnection(&VertexCount);
		RegisterOutputConnection(&EdgeCount);
		RegisterOutputConnection(&TriangleCount);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 * Converts a StaticMesh into a DataflowMesh
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FStaticMeshToMeshDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FStaticMeshToMeshDataflowNode_v2, "StaticMeshToMesh", "Mesh|Utilities", "")

private:
	/** StaticMesh to convert */
	UPROPERTY(EditAnywhere, Category = "StaticMesh", meta = (DataflowInput));
	TObjectPtr<UStaticMesh> StaticMesh;

	/** Output the HiRes representation, if set to true and HiRes doesn't exist it will output empty mesh */
	UPROPERTY(EditAnywhere, Category = "StaticMesh", meta = (DisplayName = "Use HiRes"));
	bool bUseHiRes = false;

	/** Specifies the LOD level to use */
	UPROPERTY(EditAnywhere, Category = "StaticMesh", meta = (DisplayName = "LOD Level"));
	int32 LODLevel = 0;

	/** Output Dataflow mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDataflowMesh> Mesh;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FStaticMeshToMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Get the local geometric bounding sphere of a dataflow mesh
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGetMeshBoundingSphereDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetMeshBoundingSphereDataflowNode, "GetMeshBoundingSphere", "Mesh", "Bounds Size Dimensions Extents Center")

private:
	/** Dataflow mesh to compute the bounding sphere from */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<const UDataflowMesh> Mesh = nullptr;

	/** Geometric bounding sphere of the input dataflow mesh */
	UPROPERTY(meta = (DataflowOutput))
	FSphere BoundingSphere = FSphere(ForceInit);

	/** Center of the resulting bounding sphere */
	UPROPERTY(meta = (DataflowOutput))
	FVector Center = FVector::ZeroVector;

	/** Radius of the resulting bounding sphere in centimers */
	UPROPERTY(meta = (DataflowOutput))
	float Radius = 0.f;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FGetMeshBoundingSphereDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

};

namespace UE::Dataflow
{
	void GeometryCollectionMeshNodes();
}

