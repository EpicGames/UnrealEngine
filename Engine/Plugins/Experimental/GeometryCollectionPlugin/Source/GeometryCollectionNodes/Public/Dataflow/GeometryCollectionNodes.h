// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowCollectionAttributeKeyNodes.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "UDynamicMesh.h"
#include "Dataflow/DataflowSelection.h"
#include "Math/MathFwd.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Dataflow/DataflowDebugDraw.h"
#include "Curves/CurveFloat.h"
#include "Dataflow/DataflowMesh.h"

#include "GeometryCollectionNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class FGeometryCollection;
class UGeometryCollection;
class UStaticMesh;
class UMaterial;


/**
 * Get a collection from a geometry collecion asset
 * DEPRECATED 5.7 : use GeometryCollectionToCollection node instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.7"))
struct FGetCollectionFromAssetDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetCollectionFromAssetDataflowNode, "GetCollectionFromAsset", "GeometryCollection|Asset", "")

public:
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowInput, DisplayName = "Asset"))
	TObjectPtr<UGeometryCollection> CollectionAsset;

	UPROPERTY(meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	FGetCollectionFromAssetDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&CollectionAsset);
		RegisterOutputConnection(&Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


/**
 *
 * Description for this node
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FAppendCollectionAssetsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAppendCollectionAssetsDataflowNode, "AppendCollections", "GeometryCollection", "")

public:
	typedef FManagedArrayCollection DataType;

	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection", DataflowPassthrough = "Collection1"))
	FManagedArrayCollection Collection1;

	UPROPERTY(meta = (DataflowInput, DisplayName = "Collection"))
	FManagedArrayCollection Collection2;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "GeometryGroupIndicesOut1"))
	TArray<FString> GeometryGroupGuidsOut1;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "GeometryGroupIndicesOut2"))
	TArray<FString> GeometryGroupGuidsOut2;

	FAppendCollectionAssetsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection1);
		RegisterInputConnection(&Collection2);
		RegisterOutputConnection(&Collection1, &Collection1);
		RegisterOutputConnection(&GeometryGroupGuidsOut1);
		RegisterOutputConnection(&GeometryGroupGuidsOut2);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Description for this node
 * DEPRECATED 5.5 - use Print node ( core nodes ) 
 */
USTRUCT(meta = (Deprecated = "5.5"))
struct FPrintStringDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FPrintStringDataflowNode, "PrintString", "Development", "")

public:
	UPROPERTY(EditAnywhere, Category = "Print");
	bool bPrintToScreen = true;

	UPROPERTY(EditAnywhere, Category = "Print");
	bool bPrintToLog = true;

	UPROPERTY(EditAnywhere, Category = "Print");
	FColor Color = FColor::White;

	UPROPERTY(EditAnywhere, Category = "Print");
	float Duration = 2.f;

	UPROPERTY(EditAnywhere, Category = "Print", meta = (DataflowInput));
	FString String = FString("");

	FPrintStringDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&String);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Description for this node
 * DEPRECATED - use Print node ( core nodes ) 
  */
USTRUCT(meta=(Deprecated = "5.5"))
struct FLogStringDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FLogStringDataflowNode, "LogString", "Development", "")

public:
	UPROPERTY(EditAnywhere, Category = "Print");
	bool bPrintToLog = true;

	UPROPERTY(EditAnywhere, Category = "Print", meta = (DataflowInput));
	FString String = FString("");

	FLogStringDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&String);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 * Get the geometric bounding box a collection in collection space
 * DEPRECATED 5.8 - use CollectionBounds node instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FBoundingBoxDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBoundingBoxDataflowNode, "GetCollectionBoundingBox", "Utilities|Box", "Collection Bounds Size Dimensions Extents Center")

public:
	/** Collection to compute the bouning box from */
	UPROPERTY(meta = (DataflowInput))
	FManagedArrayCollection Collection;

	/** Geometric bounding box of the input collection */
	UPROPERTY(meta = (DataflowOutput))
	FBox BoundingBox = FBox(ForceInit);

	/** Center of the resulting bounding box */
	UPROPERTY(meta = (DataflowOutput))
	FVector Center = FVector::ZeroVector;

	/** Dimensions of the resulting bounding box in centimers */
	UPROPERTY(meta = (DataflowOutput))
	FVector Dimensions = FVector::ZeroVector;

	FBoundingBoxDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Get the bounding box a collection and compute its box ratio from a reference size
 * The reference size is the length of a perfect cube representing a reference volume
 * 
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FBoundingBoxRatioDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBoundingBoxRatioDataflowNode, "GetCollectionBoundingBoxRatio", "Utilities|Box", "Collection Reference Resolution Bounds Size Dimensions Extents Center")

public:
	FBoundingBoxRatioDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Collection to compute the bounding box and ratio from */
	UPROPERTY(meta = (DataflowInput))
	FManagedArrayCollection Collection;

	/** Reference box size to use to compute the ratio */
	UPROPERTY(EditAnywhere, Category = "Proximity", meta = (DataflowInput, ClampMin = "0", UIMin = "0", Units = cm))
	float ReferenceBoxSize = 100;

	/** Geometric bounding box of the input collection */
	UPROPERTY(meta = (DataflowOutput))
	FBox BoundingBox = FBox(ForceInit);

	/** Dimensions of the resulting bounding box in centimers */
	UPROPERTY(meta = (DataflowOutput))
	FVector Dimensions = FVector::ZeroVector;

	UPROPERTY(meta = (DataflowOutput))
	float BoxRatio = 1.0;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Compute the boudning sphere of the geometrry of a collection 
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FBoundingSphereDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBoundingSphereDataflowNode, "BoundingSphere", "Utilities|Sphere", "")

public:
	UPROPERTY(meta = (DataflowInput))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowOutput))
	FSphere BoundingSphere = FSphere(ForceInit);

	FBoundingSphereDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&BoundingSphere);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

UENUM()
enum class EBoxLengthMeasurementMethod : uint8
{
	XAxis,
	YAxis,
	ZAxis,
	ShortestAxis,
	LongestAxis,
	Diagonal
};

/**
 *
 * Create an array of lengths of bounding boxes (measured along an axis, diagonal, or the max/min axes) from an array of bounding boxes
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGetBoxLengthsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetBoxLengthsDataflowNode, "GetBoxLengths", "Utilities|Box", "")

public:
	
	UPROPERTY(meta = (DataflowInput))
	TArray<FBox> Boxes;

	UPROPERTY(meta = (DataflowOutput))
	TArray<float> Lengths;

	UPROPERTY(EditAnywhere, Category = Options)
	EBoxLengthMeasurementMethod MeasurementMethod = EBoxLengthMeasurementMethod::Diagonal;

	inline double BoxToMeasurement(const FBox& Box) const
	{
		FVector Size = Box.GetSize();
		switch (MeasurementMethod)
		{
		case EBoxLengthMeasurementMethod::XAxis:
			return Size.X;
		case EBoxLengthMeasurementMethod::YAxis:
			return Size.Y;
		case EBoxLengthMeasurementMethod::ZAxis:
			return Size.Z;
		case EBoxLengthMeasurementMethod::ShortestAxis:
			return Size.GetMin();
		case EBoxLengthMeasurementMethod::LongestAxis:
			return Size.GetMax();
		case EBoxLengthMeasurementMethod::Diagonal:
			return Size.Length();
		}
		checkNoEntry(); // switch above should handle all cases
		return Size.X;
	}

	FGetBoxLengthsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Boxes);
		RegisterOutputConnection(&Lengths);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 * Get the bounding box average extents information
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FBoundingBoxAverageExtentsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBoundingBoxAverageExtentsDataflowNode, "GetBoxAverageExtents", "Utilities|Box", "BoundingBox size extent ratio length min max")

public:
	FBoundingBoxAverageExtentsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Bounding box to get the information from */
	UPROPERTY(meta = (DataflowInput))
	FBox BoundingBox = FBox(ForceInit);;

	/** Average of the 3 axis extents */
	UPROPERTY(meta = (DataflowOutput))
	float AverageExtents = 0.f;

	/** Average of the 3 axis half extents */
	UPROPERTY(meta = (DataflowOutput))
	float AverageHalfExtents = 0.f;

	/** Minimum edge length of the box */
	UPROPERTY(meta = (DataflowOutput))
	float MinLength = 0.f;

	/** Maximum edge length of the box */
	UPROPERTY(meta = (DataflowOutput))
	float MaxLength = 0.f;

	/** Ratio of the min edge length over the max edge length */
	UPROPERTY(meta = (DataflowOutput))
	float LengthRatio = 1.f;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Break a bounding box into individual components ( Min/Max corners, Center, HalfExtents and Volume )
 */
USTRUCT()
struct FExpandBoundingBoxDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FExpandBoundingBoxDataflowNode, "BreakBox", "Utilities|Box", "Bounds Bounding Expand Min Max Center Half Extents Volume")

public:
	/** Bounding box to get the information from */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Box"))
	FBox BoundingBox = FBox(ForceInit);;

	/** Minimum corner ofthe bounding box */
	UPROPERTY(meta = (DataflowOutput))
	FVector Min = FVector(0.0);

	/** Maximum corner ofthe bounding box */
	UPROPERTY(meta = (DataflowOutput))
	FVector Max = FVector(0.0);

	/** Center point of the bounding box */
	UPROPERTY(meta = (DataflowOutput))
	FVector Center = FVector(0.0);

	/** Distances from the center to any corner along the three axis */
	UPROPERTY(meta = (DataflowOutput))
	FVector HalfExtents = FVector(0.0);

	/** Dimensions of the bounding box */
	UPROPERTY(meta = (DataflowOutput))
	FVector Dimensions = FVector::ZeroVector;

	/** Total volume of the bounding box */
	UPROPERTY(meta = (DataflowOutput))
	float Volume = 0.0;

	FExpandBoundingBoxDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Expands data of FSphere
 *
 */
USTRUCT()
struct FExpandBoundingSphereDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FExpandBoundingSphereDataflowNode, "BreakBoundingSphere", "Utilities|Sphere", "Expand Radius Center Volume")

public:
	UPROPERTY(meta = (DataflowInput))
	FSphere BoundingSphere = FSphere(ForceInit);;

	UPROPERTY(meta = (DataflowOutput))
	FVector Center = FVector(0.0);

	UPROPERTY(meta = (DataflowOutput))
	float Radius = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float Volume = 0.f;

	FExpandBoundingSphereDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&BoundingSphere);
		RegisterOutputConnection(&Center);
		RegisterOutputConnection(&Radius);
		RegisterOutputConnection(&Volume);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


/**
 *
 * Expands a Vector into X, Y, Z components
 *
 */
USTRUCT()
struct FExpandVectorDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FExpandVectorDataflowNode, "BreakVector", "Utilities|Vector", "Expand X Y Z")

public:
	UPROPERTY(meta = (DataflowInput, GizmoType = Translate))
	FVector Vector = FVector(0.0);

	UPROPERTY(meta = (DataflowOutput))
	float X = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float Y = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float Z = 0.f;


	FExpandVectorDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Vector);
		RegisterOutputConnection(&X);
		RegisterOutputConnection(&Y);
		RegisterOutputConnection(&Z);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Concatenates two strings together to make a new string
 * DEPRECATED - use new version of the same node
 *
 */
USTRUCT(meta = (Deprecated = "5.6"))
struct FStringAppendDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FStringAppendDataflowNode, "StringAppend", "Utilities|String", "")

public:
	UPROPERTY(EditAnywhere, Category = "String", meta = (DataflowInput))
	FString String1 = FString("");

	UPROPERTY(EditAnywhere, Category = "String", meta = (DataflowInput))
	FString String2 = FString("");

	UPROPERTY(meta = (DataflowOutput))
	FString String = FString("");

	FStringAppendDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&String1);
		RegisterInputConnection(&String2);
		RegisterOutputConnection(&String);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Concatenates strings together to make a new string
 *
 */
USTRUCT()
struct FStringAppendDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FStringAppendDataflowNode_v2, "StringAppend", "Utilities|String", "")

private:
	UPROPERTY()
	TArray<FDataflowStringConvertibleTypes> Inputs;

	UPROPERTY(meta = (DataflowOutput))
	FString String = FString("");

	//~ Begin FDataflowNode interface
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override;
	virtual bool CanRemovePin() const override;
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	UE::Dataflow::TConnectionReference<FDataflowStringConvertibleTypes> GetConnectionReference(int32 Index) const;

	static constexpr int32 NumOtherInputs = 0; // No other inputs
	static constexpr int32 NumInitialVariableInputs = 2;

public:
	FStringAppendDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Generates a hash value from a string
 *
 */
USTRUCT()
struct FHashStringDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FHashStringDataflowNode, "HashString", "Utilities|String", "")

public:
	/** String to hash */
	UPROPERTY(meta = (DataflowInput))
	FString String = FString("");

	/** Generated hash value */
	UPROPERTY(meta = (DataflowOutput))
	int32 Hash = 0;

	FHashStringDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&String);
		RegisterOutputConnection(&Hash);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Generates a hash value from a vector
 *
 */
USTRUCT()
struct FHashVectorDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FHashVectorDataflowNode, "HashVector", "Utilities|Vector", "")

public:
	/** Vector to hash */
	UPROPERTY(meta = (DataflowInput))
	FVector Vector = FVector(0.0);

	/** Generated hash value */
	UPROPERTY(meta = (DataflowOutput))
	int32 Hash = 0;

	FHashVectorDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Vector);
		RegisterOutputConnection(&Hash);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Gets BoundingBoxes of pieces from a Collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGetBoundingBoxesFromCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetBoundingBoxesFromCollectionDataflowNode, "GetBoundingBoxesFromCollection", "GeometryCollection|Utilities", "")

public:
	/** Input Collection */
	UPROPERTY(meta = (DataflowInput))
	FManagedArrayCollection Collection;

	/** The BoundingBoxes will be output for the bones selected in the TransformSelection  */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/** Output BoundingBoxes */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FBox> BoundingBoxes;

	FGetBoundingBoxesFromCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterOutputConnection(&BoundingBoxes);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Get the root node index
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGetRootIndexFromCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetRootIndexFromCollectionDataflowNode, "GetRootIndexFromCollection", "GeometryCollection|Utilities", "")

public:
	UPROPERTY(meta = (DataflowInput))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowOutput))
	int32 RootIndex = INDEX_NONE;

	FGetRootIndexFromCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&RootIndex);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};



/**
 *
 * Gets centroids of pieces from a Collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGetCentroidsFromCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetCentroidsFromCollectionDataflowNode, "GetCentroidsFromCollection", "GeometryCollection|Utilities", "")

private:
	/** Input Collection */
	UPROPERTY(meta = (DataflowInput))
	FManagedArrayCollection Collection;

	/** The centroids will be output for the bones selected in the TransformSelection  */
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection TransformSelection;

	/** Control point color by level or setting */
	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	bool bColorByLevel = true;

	/** Point color */
	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (EditCondition = "bColorByLevel == false", EditConditionHides))
	FLinearColor Color = FLinearColor::Blue;

	/** Control the point size by level or setting */
	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	bool bSizeByLevel = true;

	/** Point size */
	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (EditCondition = "bSizeByLevel == false", EditConditionHides))
	float Size = 8.f;

	/** Float curve to control point size by level */
	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	FRuntimeFloatCurve PointSize;

	/** Output centroids */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Points"))
	TArray<FVector> Centroids = TArray<FVector>();

	/** Hidden output to store computed level values for centroids */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> Levels;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

#if WITH_EDITOR
	virtual bool CanDebugDraw() const override;
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif
public:
	FGetCentroidsFromCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};


UENUM(BlueprintType)
enum class ERotationOrderEnum : uint8
{
	Dataflow_RotationOrder_XYZ UMETA(DisplayName = "XYZ"),
	Dataflow_RotationOrder_YZX UMETA(DisplayName = "YZX"),
	Dataflow_RotationOrder_ZXY UMETA(DisplayName = "ZXY"),
	Dataflow_RotationOrder_XZY UMETA(DisplayName = "XZY"),
	Dataflow_RotationOrder_YXZ UMETA(DisplayName = "YXZ"),
	Dataflow_RotationOrder_ZYX UMETA(DisplayName = "ZYX"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 * Transforms a Collection
 *
 */
USTRUCT()
struct FTransformCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FTransformCollectionDataflowNode, "TransformCollection", "Math|Transform", "")

public:
	/** Output mesh */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Transform selection for transforming */
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection TransformSelection;

	/** Translation */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DataflowInput, GizmoType = Translate));
	FVector Translate = FVector(0.0);

	/** Rotation Order */
	UPROPERTY(EditAnywhere, Category = "Transform");
	ERotationOrderEnum RotationOrder = ERotationOrderEnum::Dataflow_RotationOrder_XYZ;

	/** Rotation */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DataflowInput, GizmoType = Rotate, GizmoPosition = Translate));
	FVector Rotate = FVector(0.0);

	/** Scale */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (UIMin = 0.001f), meta = (DataflowInput, GizmoType = Scale, GizmoPosition = Translate));
	FVector Scale = FVector(1.0);

	/** Uniform scale */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (UIMin = 0.001f));
	float UniformScale = 1.f;

	/** Pivot for the rotation */
	UPROPERTY(EditAnywhere, Category = "Pivot");
	FVector RotatePivot = FVector(0.0);

	/** Pivot for the scale */
	UPROPERTY(EditAnywhere, Category = "Pivot");
	FVector ScalePivot = FVector(0.0);

	/** Invert the transformation */
	UPROPERTY(EditAnywhere, Category = "General");
	bool bInvertTransformation = false;

	FTransformCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Bake transforms in Collection
 *
 */
USTRUCT()
struct FBakeTransformsInCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBakeTransformsInCollectionDataflowNode, "BakeTransformsInCollection", "Math|Transform", "")

public:
	/** Collection to bake transforms in */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	FBakeTransformsInCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 * Transforms a mesh
 * DEPRECATED 5.8 - use FTransformMeshDataflowNode_v2 instead
 */
USTRUCT(meta = (Deprecated = "5.8"))
struct FTransformMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FTransformMeshDataflowNode, "TransformMesh", "Math|Transform", "")

public:
	/** Output mesh */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Mesh", DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> Mesh;

	/** Translation */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DataflowInput, GizmoType = Translate));
	FVector Translate = FVector(0.0);

	/** Rotation Order */
	UPROPERTY(EditAnywhere, Category = "Transform");
	ERotationOrderEnum RotationOrder = ERotationOrderEnum::Dataflow_RotationOrder_XYZ;

	/** Rotation */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DataflowInput, GizmoType = Rotate, GizmoPosition = Translate));
	FVector Rotate = FVector(0.0);

	/** Scale */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (UIMin = 0.001f, DataflowInput, GizmoType = Scale, GizmoPosition = Translate));
	FVector Scale = FVector(1.0);

	/** Shear */
//	UPROPERTY(EditAnywhere, Category = "Transform");
//	FVector Shear = FVector(0.0);

	/** Uniform scale */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (UIMin = 0.001f, DataflowInput));
	float UniformScale = 1.f;

	/** Pivot for the rotation */
	UPROPERTY(EditAnywhere, Category = "Pivot");
	FVector RotatePivot = FVector(0.0);

	/** Pivot for the scale */
	UPROPERTY(EditAnywhere, Category = "Pivot", meta = (DataflowInput));
	FVector ScalePivot = FVector(0.0);

	/** Invert the transformation */
	UPROPERTY(EditAnywhere, Category = "General", meta = (DataflowInput));
	bool bInvertTransformation = false;

	FTransformMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Mesh);
		RegisterInputConnection(&Translate);
		RegisterInputConnection(&Rotate);
		RegisterInputConnection(&Scale);
		RegisterInputConnection(&UniformScale);
		RegisterInputConnection(&RotatePivot).SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&ScalePivot).SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&bInvertTransformation).SetCanHidePin(true).SetPinIsHidden(true);
		RegisterOutputConnection(&Mesh, &Mesh);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

UENUM(BlueprintType)
enum class ECompareOperationEnum : uint8
{
	Dataflow_Compare_Equal UMETA(DisplayName = "=="),
	Dataflow_Compare_Smaller UMETA(DisplayName = "<"),
	Dataflow_Compare_SmallerOrEqual UMETA(DisplayName = "<="),
	Dataflow_Compare_Greater UMETA(DisplayName = ">"),
	Dataflow_Compare_GreaterOrEqual UMETA(DisplayName = ">="),
	Dataflow_Compare_NotEqual UMETA(DisplayName = "!="),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 * Comparison between integers
 *
 */
USTRUCT()
struct FCompareIntDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCompareIntDataflowNode, "CompareInt", "Math|Compare", "")

public:
	/** Comparison operation */
	UPROPERTY(EditAnywhere, Category = "Compare");
	ECompareOperationEnum Operation = ECompareOperationEnum::Dataflow_Compare_Equal;

	/** Int input */
	UPROPERTY(EditAnywhere, Category = "Compare");
	int32 IntA = 0;

	/** Int input */
	UPROPERTY(EditAnywhere, Category = "Compare");
	int32 IntB = 0;

	/** Boolean result of the comparison */
	UPROPERTY(meta = (DataflowOutput));
	bool Result = false;

	FCompareIntDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&IntA);
		RegisterInputConnection(&IntB);
		RegisterOutputConnection(&Result);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Comparison between floats
 *
 */
USTRUCT()
struct FCompareFloatDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCompareFloatDataflowNode, "CompareFloat", "Math|Compare", "")

public:
	/** Comparison operation */
	UPROPERTY(EditAnywhere, Category = "Compare");
	ECompareOperationEnum Operation = ECompareOperationEnum::Dataflow_Compare_Equal;

	/** Float input */
	UPROPERTY(EditAnywhere, Category = "Compare");
	float FloatA = 0;

	/** Float input */
	UPROPERTY(EditAnywhere, Category = "Compare");
	float FloatB = 0;

	/** Boolean result of the comparison */
	UPROPERTY(meta = (DataflowOutput));
	bool Result = false;

	FCompareFloatDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FloatA);
		RegisterInputConnection(&FloatB);
		RegisterOutputConnection(&Result);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


UENUM(BlueprintType)
enum class EBooleanOperationEnum : uint8
{
	Dataflow_And UMETA(DisplayName = "&&"),
	Dataflow_Or UMETA(DisplayName = "||"),
	Dataflow_Not UMETA(DisplayName = "!"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 * Boolean operations
 *
 */
USTRUCT()
struct FBooleanOperationDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBooleanOperationDataflowNode, "BooleanOperation", "Math|Boolean", "")

public:
	/** Boolean operation */
	UPROPERTY(EditAnywhere, Category = "Boolean");
	EBooleanOperationEnum Operation = EBooleanOperationEnum::Dataflow_And;

	/** Boolean input */
	UPROPERTY(EditAnywhere, Category = "Boolean", Meta = (DataflowInput));
	bool bBoolA = false;

	/** Boolean input */
	UPROPERTY(EditAnywhere, Category = "Boolean", Meta = (EditCondition = "Operation != EBooleanOperationEnum::Dataflow_Not", DataflowInput));
	bool bBoolB = false;

	/** Boolean result of the operator */
	UPROPERTY(meta = (DataflowOutput));
	bool bResult = false;

	FBooleanOperationDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&bBoolA);
		RegisterInputConnection(&bBoolB);
		RegisterOutputConnection(&bResult);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Branch between two mesh inputs based on boolean condition
 *
 */
USTRUCT()
struct FBranchMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBranchMeshDataflowNode, "BranchMesh", "Utilities|FlowControl", "")

public:
	/** Mesh input */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDynamicMesh> MeshA;

	/** Mesh input */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<UDynamicMesh> MeshB;

	/** If true, Output = MeshA, otherwise Output = MeshB */
	UPROPERTY(EditAnywhere, Category = "Branch");
	bool bCondition = false;

	/** Output mesh */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> Mesh;

	FBranchMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&MeshA);
		RegisterInputConnection(&MeshB);
		RegisterInputConnection(&bCondition);
		RegisterOutputConnection(&Mesh);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Branch between two Managed Array Collections based on Boolean condition
 *
 */
USTRUCT()
struct FBranchCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBranchCollectionDataflowNode, "BranchCollection", "Utilities|FlowControl", "")


public:
	/** Collection input for the 'true' case */
	UPROPERTY(meta = (DataflowInput))
	FManagedArrayCollection TrueCollection;

	/** Collection input for the 'false' case */
	UPROPERTY(meta = (DataflowInput))
	FManagedArrayCollection FalseCollection;

	/** Condition to select which Collection is chosen as ChosenCollection */
	UPROPERTY(EditAnywhere, Category = "Branch");
	bool bCondition = false;

	/** Output Collection */
	UPROPERTY(meta = (DataflowOutput))
	FManagedArrayCollection ChosenCollection;

	FBranchCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&TrueCollection);
		RegisterInputConnection(&FalseCollection);
		RegisterInputConnection(&bCondition);
		RegisterOutputConnection(&ChosenCollection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};



/**
 *
 * Collects group and attribute information from the Collection and outputs it into a formatted string
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGetSchemaDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetSchemaDataflowNode, "GetSchema", "GeometryCollection|Utilities", "")

public:
	/** GeometryCollection  for the information */
	UPROPERTY(meta = (DataflowInput))
	FManagedArrayCollection Collection;

	/** Formatted string containing the groups and attributes */
	UPROPERTY(meta = (DataflowOutput))
	FString String;

	FGetSchemaDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&String);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


USTRUCT(meta = (DataflowGeometryCollection))
struct FRemoveOnBreakDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRemoveOnBreakDataflowNode, "RemoveOnBreak", "GeometryCollection|Utilities", "")

public:
	/** Collection to set theremoval data on */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** selection to apply the data on ( if not specified the entire collection will be set ) */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/** Whether or not to enable the removal on the selection */
	UPROPERTY(EditAnywhere, Category = "Removal", meta = (DataflowInput))
	bool bEnabledRemoval = true;

	/** How long after the break the removal will start ( Min / Max ) */
	UPROPERTY(EditAnywhere, Category = "Removal", meta = (DataflowInput))
	FVector2f PostBreakTimer{0.0, 0.0};

	/** How long removal will last ( Min / Max ) */
	UPROPERTY(EditAnywhere, Category = "Removal", meta = (DataflowInput))
	FVector2f RemovalTimer{0.0, 1.0};

	/** If applied to a cluster this will cause the cluster to crumble upon removal, otherwise will have no effect */
	UPROPERTY(EditAnywhere, Category = "Removal", meta = (DataflowInput))
	bool bClusterCrumbling = false;

	FRemoveOnBreakDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterInputConnection(&bEnabledRemoval);
		RegisterInputConnection(&PostBreakTimer);
		RegisterInputConnection(&RemovalTimer);
		RegisterInputConnection(&bClusterCrumbling);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


UENUM(BlueprintType)
enum class EAnchorStateEnum : uint8
{
	Dataflow_AnchorState_Anchored UMETA(DisplayName = "Anchored"),
	Dataflow_AnchorState_NotAnchored UMETA(DisplayName = "Not Anchored"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 * Sets the anchored state on the selected bones in a Collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FSetAnchorStateDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetAnchorStateDataflowNode, "SetAnchorState", "GeometryCollection|Utilities", "")

private:
	/** What anchor state to set on selected bones */
	UPROPERTY(EditAnywhere, Category = "Anchoring");
	EAnchorStateEnum AnchorState = EAnchorStateEnum::Dataflow_AnchorState_Anchored;

	/** If true, sets the non selected bones to opposite anchor state */
	UPROPERTY(EditAnywhere, Category = "Anchoring")
	bool bSetNotSelectedBonesToOppositeState = false;

	/** GeometryCollection to set anchor state on */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Bone selection for setting the state on */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection", DataflowIntrinsic))
	FDataflowTransformSelection TransformSelection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

#if WITH_EDITOR
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif //WITH_EDITOR

public:
	FSetAnchorStateDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

UENUM(BlueprintType)
enum class EDataflowGeometryCollectionDynamicState : uint8
{
	None = 0		UMETA(DisplayName = "None"),
	Dynamic  = 1	UMETA(DisplayName = "Dynamic"),
	Kinematic =  2	UMETA(DisplayName = "Kinematic"),
	Static = 3		UMETA(DisplayName = "Static")
};

/**
 *
 * Sets the dynamic state on the selected bones in a Collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FSetDynamicStateDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetDynamicStateDataflowNode, "SetDynamicState", "GeometryCollection|Utilities", "")

private:
	/** Dynamic state to set on selected bones */
	UPROPERTY(EditAnywhere, Category = "State");
	EDataflowGeometryCollectionDynamicState DynamicState = EDataflowGeometryCollectionDynamicState::Kinematic;

	/** GeometryCollection to set anchor state on */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Bone selection for setting the state on */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection", DataflowIntrinsic))
	FDataflowTransformSelection TransformSelection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FSetDynamicStateDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

};


UENUM(BlueprintType)
enum class EProximityMethodEnum : uint8
{
	/** Precise proximity mode looks for geometry with touching vertices or touching, coplanar, opposite - facing triangles.This works well with geometry fractured using our fracture tools. */
	Dataflow_ProximityMethod_Precise UMETA(DisplayName = "Precise"),
	/** Convex Hull proximity mode looks for geometry with overlapping convex hulls(with an optional offset) */
	Dataflow_ProximityMethod_ConvexHull UMETA(DisplayName = "ConvexHull"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

UENUM(BlueprintType)
enum class EProximityContactFilteringMethodEnum : uint8
{
	/** Rejects proximity if the bounding boxes do not overlap by more than Contact Threshold centimeters in any major axis direction (or at least half the max possible). This can filter out corner connections of box-like shapes. */
	Dataflow_ProximityContactFilteringMethod_ProjectedBoundsOverlap UMETA(DisplayName = "Projected Bounds Overlap"),
	/** Rejects proximity if the intersection of convex hulls (allowing for optional offset) follows a sharp, thin region which is not wider than Contact Threshold centimeters (or at least half the max possible). */
	Dataflow_ProximityContactFilteringMethod_ConvexHullSharp UMETA(DisplayName = "Convex Hull Sharp Contact"),
	/** Rejects proximity if the surface area of the intersection of convex hulls (allowing for optional offset) is smaller than Contact Threshold squared (or at least half the max possible). */
	Dataflow_ProximityContactFilteringMethod_ConvexHullArea UMETA(DisplayName = "Convex Hull Area Contact"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

UENUM(BlueprintType)
enum class EConnectionContactAreaMethodEnum : uint8
{
	/** Do not compute contact areas */
	Dataflow_ConnectionContactAreaMethod_None UMETA(DisplayName = "None"),
	/** Compute approximate contact surface area via the intersection of convex hulls (allowing for optional offset) */
	Dataflow_ProximityContactFilteringMethod_ConvexHullArea UMETA(DisplayName = "Convex Hull Area Contact"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};


/**
 *
 * Update the proximity (contact) graph for the bones in a Collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, Icon = "FractureEditor.Proximity"))
struct FProximityDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FProximityDataflowNode, "Proximity", "GeometryCollection|Utilities", "")

private:
	/** Which method to use to decide whether a given piece of geometry is in proximity with another */
	UPROPERTY(EditAnywhere, Category = "Proximity");
	EProximityMethodEnum ProximityMethod = EProximityMethodEnum::Dataflow_ProximityMethod_Precise;

	/** If hull-based proximity detection is enabled, amount to expand hulls when searching for overlapping neighbors */
	UPROPERTY(EditAnywhere, Category = "Proximity", meta = (DataflowInput, ClampMin = "0", Units = cm,
		EditCondition = "ProximityMethod == EProximityMethodEnum::Dataflow_ProximityMethod_ConvexHull || FilterContactMethod == EProximityContactFilteringMethodEnum::Dataflow_ProximityContactFilteringMethod_ConvexHullSharp || FilterContactMethod == EProximityContactFilteringMethodEnum::Dataflow_ProximityContactFilteringMethod_ConvexHullArea || ContactAreaMethod == EConnectionContactAreaMethodEnum::Dataflow_ProximityContactFilteringMethod_ConvexHullArea"))
	float DistanceThreshold = 1;

	// If greater than zero, proximity will be additionally filtered by a 'contact' threshold, in cm, to exclude grazing / corner proximity
	UPROPERTY(EditAnywhere, Category = "Proximity", meta = (DataflowInput, ClampMin = "0", Units = cm))
	float ContactThreshold = 0;

	/** How to use the Contact Threshold (if > 0) to filter out unwanted small or corner contacts from the proximity graph. If contact threshold is zero, no filtering is applied. */
	UPROPERTY(EditAnywhere, Category = "Proximity");
	EProximityContactFilteringMethodEnum FilterContactMethod = EProximityContactFilteringMethodEnum::Dataflow_ProximityContactFilteringMethod_ProjectedBoundsOverlap;

	/** Whether to automatically transform the proximity graph into a connection graph to be used for simulation */
	UPROPERTY(EditAnywhere, Category = "Proximity")
	bool bUseAsConnectionGraph = false;

	/** The method used to compute contact areas for simulation purposes (only when 'Use As Connection Graph' is enabled) */
	UPROPERTY(EditAnywhere, Category = "Proximity")
	EConnectionContactAreaMethodEnum ContactAreaMethod = EConnectionContactAreaMethodEnum::Dataflow_ConnectionContactAreaMethod_None;

	/** Whether to compute new convex hulls for proximity, or use the pre-existing hulls on the Collection, when using convex hulls to determine proximity */
	UPROPERTY(EditAnywhere, Category = "Proximity", meta = (
		EditCondition = "ProximityMethod == EProximityMethodEnum::Dataflow_ProximityMethod_ConvexHull || FilterContactMethod == EProximityContactFilteringMethodEnum::Dataflow_ProximityContactFilteringMethod_ConvexHullSharp || FilterContactMethod == EProximityContactFilteringMethodEnum::Dataflow_ProximityContactFilteringMethod_ConvexHullArea || ContactAreaMethod == EConnectionContactAreaMethodEnum::Dataflow_ProximityContactFilteringMethod_ConvexHullArea"))
	bool bRecomputeConvexHulls = true;

	/** GeometryCollection to update the proximity graph on */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FProximityDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&DistanceThreshold);
		RegisterInputConnection(&ContactThreshold);
		RegisterOutputConnection(&Collection, &Collection);
	}
};


/**
 *
 * Sets pivot for Collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionSetPivotDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSetPivotDataflowNode, "SetPivot", "GeometryCollection|Utilities", "")

public:
	/** Collection for the pivot change */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Pivot transform */
	UPROPERTY(EditAnywhere, Category = "Pivot", meta = (DataflowInput))
	FTransform Transform;

	FCollectionSetPivotDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&Transform);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


UENUM(BlueprintType)
enum class EStandardGroupNameEnum : uint8
{
	Dataflow_EStandardGroupNameEnum_Transform	UMETA(DisplayName = "Transform", ToolTip = ""),
	Dataflow_EStandardGroupNameEnum_Geometry	UMETA(DisplayName = "Geometry", ToolTip = ""),
	Dataflow_EStandardGroupNameEnum_Faces		UMETA(DisplayName = "Faces", ToolTip = ""),
	Dataflow_EStandardGroupNameEnum_Vertices	UMETA(DisplayName = "Vertices", ToolTip = ""),
	Dataflow_EStandardGroupNameEnum_Material	UMETA(DisplayName = "Material", ToolTip = ""),
	Dataflow_EStandardGroupNameEnum_Breaking	UMETA(DisplayName = "Breaking", ToolTip = ""),
	Dataflow_EStandardGroupNameEnum_Custom		UMETA(DisplayName = "Custom", ToolTip = ""),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ECustomAttributeTypeEnum : uint8
{
	Dataflow_CustomAttributeType_UInt8				UMETA(DisplayName = "UInt8", ToolTip = "Data type: UInt8"),
	Dataflow_CustomAttributeType_Int32				UMETA(DisplayName = "Int32", ToolTip = "Data type: Int32"),
	Dataflow_CustomAttributeType_Float				UMETA(DisplayName = "Float", ToolTip = "Data type: Float"),
	Dataflow_CustomAttributeType_Double				UMETA(DisplayName = "Double", ToolTip = "Data type: Double"),
	Dataflow_CustomAttributeType_Bool				UMETA(DisplayName = "Bool", ToolTip = "Data type: Bool"),
	Dataflow_CustomAttributeType_String				UMETA(DisplayName = "String", ToolTip = "Data type: FString"),
	Dataflow_CustomAttributeType_Vector2f			UMETA(DisplayName = "Vector2D", ToolTip = "Data type: FVector2f"),
	Dataflow_CustomAttributeType_Vector3f			UMETA(DisplayName = "Vector", ToolTip = "Data type: FVector3f"),
	Dataflow_CustomAttributeType_Vector3d			UMETA(DisplayName = "Vector3d", ToolTip = "Data type: FVector3d"),
	Dataflow_CustomAttributeType_Vector4f			UMETA(DisplayName = "Vector4f", ToolTip = "Data type: FVector4f"),
	Dataflow_CustomAttributeType_LinearColor		UMETA(DisplayName = "LinearColor", ToolTip = "Data type: FLinearColor"),
	Dataflow_CustomAttributeType_Transform			UMETA(DisplayName = "Transform", ToolTip = "Data type: FTransform"),
	Dataflow_CustomAttributeType_Quat4f				UMETA(DisplayName = "Quat", ToolTip = "Data type: FQuat4f"),
	Dataflow_CustomAttributeType_Box				UMETA(DisplayName = "Box", ToolTip = "Data type: FBox"),
	Dataflow_CustomAttributeType_Guid				UMETA(DisplayName = "Guid", ToolTip = "Data type: FGuid"),
	Dataflow_CustomAttributeType_Int32Set			UMETA(DisplayName = "IntArray", ToolTip = "Data type: TSet<int32>"),
	Dataflow_CustomAttributeType_Int32Array			UMETA(DisplayName = "Int32Array", ToolTip = "Data type: TArray<int32>"),
	Dataflow_CustomAttributeType_IntVector			UMETA(DisplayName = "IntVector", ToolTip = "Data type: FIntVector"),
	Dataflow_CustomAttributeType_IntVector2			UMETA(DisplayName = "IntVector2", ToolTip = "Data type: FIntVector2"),
	Dataflow_CustomAttributeType_IntVector4			UMETA(DisplayName = "IntVector4", ToolTip = "Data type: FIntVector4"),
	Dataflow_CustomAttributeType_IntVector2Array	UMETA(DisplayName = "IntVector2Array", ToolTip = "Data type: TArray<FIntVector2>"),
	Dataflow_CustomAttributeType_FloatArray			UMETA(DisplayName = "FloatArray", ToolTip = "Data type: TArray<float>"),
	Dataflow_CustomAttributeType_Vector2fArray		UMETA(DisplayName = "Vector2DArray", ToolTip = "Data type: TArray<FVector2f>"),
	Dataflow_CustomAttributeType_FVector3fArray		UMETA(DisplayName = "FVectorArray", ToolTip = "Data type: TArray<FVector3f>"),

	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 * Adds custom attribute to Collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FAddCustomCollectionAttributeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAddCustomCollectionAttributeDataflowNode, "AddCustomCollectionAttribute", "GeometryCollection|Utilities", "")

public:
	/** Collection for the custom attribute */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Standard group names */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Group"))
	EStandardGroupNameEnum GroupName = EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Transform;

	/** User specified group name */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Custom Group", EditCondition = "GroupName == EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Custom"))
	FString CustomGroupName = FString("");

	/** Attribute name */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Attribute"))
	FString AttrName = FString("");

	/** Attribute type */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Attribute Type"));
	ECustomAttributeTypeEnum CustomAttributeType = ECustomAttributeTypeEnum::Dataflow_CustomAttributeType_Float;

	/** Number of elements for the attribute */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Number of Elements"));
	int32 NumElements = 0;

	FAddCustomCollectionAttributeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&NumElements);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Returns number of elements in a group in a Collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGetNumElementsInCollectionGroupDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetNumElementsInCollectionGroupDataflowNode, "GetNumElementsInCollectionGroup", "GeometryCollection|Utilities", "")

public:
	/** Collection for the custom attribute */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Standard group names */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Group"))
	EStandardGroupNameEnum GroupName = EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Transform;

	/** User specified group name */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Custom Group", EditCondition = "GroupName == EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Custom"))
	FString CustomGroupName = FString("");

	/** Number of elements for the attribute */
	UPROPERTY(meta = (DataflowOutput));
	int32 NumElements = 0;

	FGetNumElementsInCollectionGroupDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&NumElements);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Get attribute data from a Collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGetCollectionAttributeDataTypedDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetCollectionAttributeDataTypedDataflowNode, "GetCollectionAttributeDataTyped", "GeometryCollection|Utilities", "")

public:
	/** Collection for the custom attribute */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Input to drive the Attribute and Group name */
	UPROPERTY(meta = (DataflowInput, DisplayName = "AttributeKey"))
	FCollectionAttributeKey AttributeKey;

	/** Standard group names */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Group"))
	EStandardGroupNameEnum GroupName = EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Transform;

	/** User specified group name */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Custom Group", EditCondition = "GroupName == EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Custom"))
	FString CustomGroupName = FString("");

	/** Attribute name */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Attribute"))
	FString AttrName = FString("");

	/** Bool type attribute data */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Bool Array"));
	TArray<bool> BoolAttributeData;

	/** Float type attribute data */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Float Array"));
	TArray<float> FloatAttributeData;

	/** Float type attribute data */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Double Array"));
	TArray<double> DoubleAttributeData;

	/** Int type attribute data */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Int32 Array"));
	TArray<int32> Int32AttributeData;

	/** Int type attribute data */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "String Array"));
	TArray<FString> StringAttributeData;

	/** Vector3f type attribute data */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Vector3f Array"));
	TArray<FVector3f> Vector3fAttributeData;

	/** Vector3d type attribute data */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Vector3d Array"));
	TArray<FVector3d> Vector3dAttributeData;

	/** Vector3d type attribute data */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Linear Color Array"));
	TArray<FLinearColor> LinearColorAttributeData;

	FGetCollectionAttributeDataTypedDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&AttributeKey);
		RegisterOutputConnection(&BoolAttributeData);
		RegisterOutputConnection(&FloatAttributeData);
		RegisterOutputConnection(&DoubleAttributeData);
		RegisterOutputConnection(&Int32AttributeData);
		RegisterOutputConnection(&StringAttributeData);
		RegisterOutputConnection(&Vector3fAttributeData);
		RegisterOutputConnection(&Vector3dAttributeData);
		RegisterOutputConnection(&LinearColorAttributeData);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Get attribute data from a Collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGetCollectionAttributeDataTypedDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetCollectionAttributeDataTypedDataflowNode_v2, "GetCollectionAttributeDataTyped", "GeometryCollection|Utilities", "")

private:
	/** Collection for the custom attribute */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Input to drive the Attribute and Group name */
	UPROPERTY(meta = (DataflowInput, DisplayName = "AttributeKey"))
	FCollectionAttributeKey AttributeKey;

	/** Standard group names */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Group"))
	EStandardGroupNameEnum GroupName = EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Transform;

	/** User specified group name */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Custom Group", EditCondition = "GroupName == EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Custom"))
	FString CustomGroupName = FString("");

	/** Attribute name */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Attribute"))
	FString AttrName = FString("");

	/** Bool type attribute data */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Bool Array"));
	TArray<bool> BoolAttributeData;

	/** Numeric Array types */
	UPROPERTY(meta = (DataflowOutput));
	FDataflowNumericArrayTypes NumericArray;

	/** Vector Array types */
	UPROPERTY(meta = (DataflowOutput));
	FDataflowVectorArrayTypes VectorArray;

	/** String Array types */
	UPROPERTY(meta = (DataflowOutput));
	FDataflowStringArrayTypes StringArray;

public:
	FGetCollectionAttributeDataTypedDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&AttributeKey);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&BoolAttributeData);
		RegisterOutputConnection(&NumericArray);
		RegisterOutputConnection(&VectorArray);
		RegisterOutputConnection(&StringArray);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Set attribute data in a Collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FSetCollectionAttributeDataTypedDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetCollectionAttributeDataTypedDataflowNode, "SetCollectionAttributeDataTyped", "GeometryCollection|Utilities", "")

public:
	/** Collection for the custom attribute */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Input to drive the Attribute and Group name */
	UPROPERTY(meta = (DataflowInput, DisplayName = "AttributeKey"))
	FCollectionAttributeKey AttributeKey;

	/** Standard group names */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Group"))
	EStandardGroupNameEnum GroupName = EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Transform;

	/** User specified group name */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Custom Group", EditCondition = "GroupName == EStandardGroupNameEnum::Dataflow_EStandardGroupNameEnum_Custom"))
	FString CustomGroupName = FString("");

	/** Attribute name */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Attribute"))
	FString AttrName = FString("");

	/** Bool type attribute data */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Bool Array"));
	TArray<bool> BoolAttributeData;

	/** Float type attribute data */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Float Array"));
	TArray<float> FloatAttributeData;

	/** Float type attribute data */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Double Array"));
	TArray<double> DoubleAttributeData;

	/** Int type attribute data */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Int32 Array"));
	TArray<int32> Int32AttributeData;

	/** Int type attribute data */
	UPROPERTY(meta = (DataflowInput, DisplayName = "String Array"));
	TArray<FString> StringAttributeData;

	/** Vector3f type attribute data */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Vector3f Array"));
	TArray<FVector3f> Vector3fAttributeData;

	/** Vector3d type attribute data */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Vector3d Array"));
	TArray<FVector3d> Vector3dAttributeData;

	/** LinearColor type attribute data */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Linear Color Array"));
	TArray<FLinearColor> LinearColorAttributeData;

	FSetCollectionAttributeDataTypedDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&AttributeKey);
		RegisterInputConnection(&BoolAttributeData);
		RegisterInputConnection(&FloatAttributeData);
		RegisterInputConnection(&DoubleAttributeData);
		RegisterInputConnection(&Int32AttributeData);
		RegisterInputConnection(&StringAttributeData);
		RegisterInputConnection(&Vector3fAttributeData);
		RegisterInputConnection(&Vector3dAttributeData);
		RegisterInputConnection(&LinearColorAttributeData);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 *
 *
 */
USTRUCT()
struct FSelectionToVertexListDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
		DATAFLOW_NODE_DEFINE_INTERNAL(FSelectionToVertexListDataflowNode, "SelectionToVertexList", "Selection|Utility", "")

public:

	/**  */
	UPROPERTY(meta = (DataflowInput, DisplayName = "VertexSelection", DataflowIntrinsic))
	FDataflowVertexSelection VertexSelection;

	/**  */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "VertexList"))
	TArray<int32> VertexList;


	FSelectionToVertexListDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&VertexSelection);
		RegisterOutputConnection(&VertexList);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Description for this node
 *
 */
USTRUCT()
struct FMultiplyTransformDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMultiplyTransformDataflowNode, "MultiplyTransform", "Math|Transform", "")

public:
	UPROPERTY(meta = (DataflowInput, DisplayName = "Left Transform"));
	FTransform InLeftTransform = FTransform::Identity;

	UPROPERTY(meta = (DataflowInput, DisplayName = "Right Transform"));
	FTransform InRightTransform = FTransform::Identity;

	UPROPERTY(meta = ( DataflowOutput, DisplayName = "Out Transform"));
	FTransform OutTransform = FTransform::Identity;

	FMultiplyTransformDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&InLeftTransform);
		RegisterInputConnection(&InRightTransform);
		RegisterOutputConnection(&OutTransform);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Invert a transform.
 *
 */
USTRUCT()
struct FInvertTransformDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FInvertTransformDataflowNode, "InvertTransform", "Math|Transform", "")

public:
	UPROPERTY(meta = (DataflowInput, DisplayName = "InTransform"));
	FTransform InTransform = FTransform::Identity;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "OutTransform"));
	FTransform OutTransform = FTransform::Identity;

	FInvertTransformDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&InTransform);
		RegisterOutputConnection(&OutTransform);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Branch between two float inputs based on boolean condition
 *
 */
USTRUCT()
struct FBranchFloatDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBranchFloatDataflowNode, "BranchFloat", "Utilities|FlowControl", "")

public:
	/** Float input */
	UPROPERTY(EditAnywhere, Category = "Branch", meta = (DataflowInput));
	float A = 0.f;

	/** Float input */
	UPROPERTY(EditAnywhere, Category = "Branch", meta = (DataflowInput));
	float B = 0.f;

	/** If true, Output = A, otherwise Output = B */
	UPROPERTY(EditAnywhere, Category = "Branch");
	bool bCondition = false;

	/** Output */
	UPROPERTY(meta = (DataflowOutput))
	float ReturnValue = 0.f;

	FBranchFloatDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&A);
		RegisterInputConnection(&B);
		RegisterInputConnection(&bCondition);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Branch between two int inputs based on boolean condition
 *
 */
USTRUCT()
struct FBranchIntDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBranchIntDataflowNode, "BranchInt", "Utilities|FlowControl", "")

public:
	/** Int input */
	UPROPERTY(EditAnywhere, Category = "Branch", meta = (DataflowInput));
	int32 A = 0;

	/** Int input */
	UPROPERTY(EditAnywhere, Category = "Branch", meta = (DataflowInput));
	int32 B = 0;

	/** If true, Output = A, otherwise Output = B */
	UPROPERTY(EditAnywhere, Category = "Branch");
	bool bCondition = false;

	/** Output */
	UPROPERTY(meta = (DataflowOutput))
	int32 ReturnValue = 0;

	FBranchIntDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&A);
		RegisterInputConnection(&B);
		RegisterInputConnection(&bCondition);
		RegisterOutputConnection(&ReturnValue);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

// Example to deprecate a Dataflow node
 //USTRUCT(meta = (Deprecated = "5.1"))
 //struct FLogStringDataflowNode : public FDataflowNode

 // Example to version up a Dataflow node, the type name needs to be versioned up
 // Important: don't change the display name!
  //USTRUCT()
  //struct FLogStringDataflowNode_v3 : public FDataflowNode

// Example of experimental Dataflow node
 //USTRUCT(meta = (Experimental))
 //struct FLogStringDataflowNode_v2 : public FDataflowNode

/**
 *
 * Visualize tetrahedrons in a collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FVisualizeTetrahedronsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVisualizeTetrahedronsDataflowNode, "VisualizeTetrahedrons", "Flesh|Utilities", "")
	DATAFLOW_NODE_RENDER_TYPE_START()
	DATAFLOW_NODE_RENDER_TYPE_ADD("TetrahedronRender", FGeometryCollection::StaticType(), "Collection")
	DATAFLOW_NODE_RENDER_TYPE_END()

private:
	/** Collection for the custom attribute */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> Vertices;

public:
	FVisualizeTetrahedronsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&Vertices);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Add point cloud to a collection as vertices
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FPointsToCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FPointsToCollectionDataflowNode, "PointsToCollection", "GeometryCollection|Utilities", "")

private:
	/** Collection to add the points to */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Points to add to the collection */
	UPROPERTY(EditAnywhere, Category = "Points", meta = (DataflowInput))
	TArray<FVector> Points;

public:
	FPointsToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&Points);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Get vertices from a collection as a point cloud
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionToPointsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionToPointsDataflowNode, "CollectionToPoints", "GeometryCollection|Utilities", "")

private:
	/** Collection storing the points */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Points from the collection */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> Points;

public:
	FCollectionToPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&Points);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Outputs Spheres as Points and radius values
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FSpheresToPointsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSpheresToPointsDataflowNode, "SpheresToPoints", "GeometryCollection|Utilities", "")

private:
	/** Input Spheres  */
	UPROPERTY(meta = (DataflowInputDataflowIntrinsic))
	TArray<FSphere> Spheres;

	/** Centers of the spheres */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> Points;

	/** Radius values */
	UPROPERTY(meta = (DataflowOutput))
	TArray<float> Radii;

public:
	FSpheresToPointsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Spheres);
		RegisterOutputConnection(&Points);
		RegisterOutputConnection(&Radii);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Outputs Collection info
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionInfoDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionInfoDataflowNode, "CollectionInfo", "GeometryCollection|Utilities", "")

private:
	/** Collection storing the points */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** MaxLevel */
	UPROPERTY(meta = (DataflowOutput))
	int32 MaxLevel = 0;

public:
	FCollectionInfoDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&MaxLevel);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

// -----------------------------------------------------------------------------------------------------------------------------------

/**
 * Transforms a Dataflow mesh
 */
USTRUCT()
struct FTransformMeshDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FTransformMeshDataflowNode_v2, "TransformMesh", "Math|Transform", "")

private:
	/** Output mesh */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Mesh", DataflowIntrinsic))
	TObjectPtr<UDataflowMesh> Mesh;

	/** Translation */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DataflowInput, GizmoType = Translate));
	FVector Translate = FVector(0.0);

	/** Rotation Order */
	UPROPERTY(EditAnywhere, Category = "Transform");
	ERotationOrderEnum RotationOrder = ERotationOrderEnum::Dataflow_RotationOrder_XYZ;

	/** Rotation */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (DataflowInput, GizmoType = Rotate, GizmoPosition = Translate));
	FVector Rotate = FVector(0.0);

	/** Scale */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (UIMin = 0.001f, DataflowInput, GizmoType = Scale, GizmoPosition = Translate));
	FVector Scale = FVector(1.0);

	/** Shear */
//	UPROPERTY(EditAnywhere, Category = "Transform");
//	FVector Shear = FVector(0.0);

	/** Uniform scale */
	UPROPERTY(EditAnywhere, Category = "Transform", meta = (UIMin = 0.001f, DataflowInput));
	float UniformScale = 1.f;

	/** Pivot for the rotation */
	UPROPERTY(EditAnywhere, Category = "Pivot", meta = (DataflowInput));
	FVector RotatePivot = FVector(0.0);

	/** Pivot for the scale */
	UPROPERTY(EditAnywhere, Category = "Pivot", meta = (DataflowInput));
	FVector ScalePivot = FVector(0.0);

	/** Invert the transformation */
	UPROPERTY(EditAnywhere, Category = "General", meta = (DataflowInput));
	bool bInvertTransformation = false;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FTransformMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

namespace UE::Dataflow
{
	void GeometryCollectionEngineNodes();
}
