// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowOverrideNode.h"

#include "GeometryCollectionOverrideNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/**
 *
 * 
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, DataflowOverride))
struct FGetFloatOverrideFromAssetDataflowNode : public FDataflowOverrideNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetFloatOverrideFromAssetDataflowNode, "GetFloatOverrideFromAsset", "GeometryCollection|Overrides", "")

public:
	UPROPERTY(meta = (DataflowOutput))
	float Float = 0.f;

	UPROPERTY(meta = (DataflowOutput))
	float FloatDefault = 0.f;

	FGetFloatOverrideFromAssetDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowOverrideNode(InParam, InGuid)
	{
		RegisterOutputConnection(&Float);
		RegisterOutputConnection(&FloatDefault);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 *
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, DataflowOverride))
struct FGetIntOverrideFromAssetDataflowNode : public FDataflowOverrideNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetIntOverrideFromAssetDataflowNode, "GetIntOverrideFromAsset", "GeometryCollection|Overrides", "")

public:
	UPROPERTY(meta = (DataflowOutput))
	int32 Int = 0;

	UPROPERTY(meta = (DataflowOutput))
	int32 IntDefault = 0;

	FGetIntOverrideFromAssetDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowOverrideNode(InParam, InGuid)
	{
		RegisterOutputConnection(&Int);
		RegisterOutputConnection(&IntDefault);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 *
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, DataflowOverride))
struct FGetBoolOverrideFromAssetDataflowNode : public FDataflowOverrideNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetBoolOverrideFromAssetDataflowNode, "GetBoolOverrideFromAsset", "GeometryCollection|Overrides", "")

public:
	UPROPERTY(meta = (DataflowOutput))
	bool Bool = false;

	UPROPERTY(meta = (DataflowOutput))
	bool BoolDefault = false;

	FGetBoolOverrideFromAssetDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowOverrideNode(InParam, InGuid)
	{
		RegisterOutputConnection(&Bool);
		RegisterOutputConnection(&BoolDefault);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 *
 *
 */
USTRUCT(meta = (DataflowGeometryCollection, DataflowOverride))
struct FGetStringOverrideFromAssetDataflowNode : public FDataflowOverrideNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetStringOverrideFromAssetDataflowNode, "GetStringOverrideFromAsset", "GeometryCollection|Overrides", "")

public:
	UPROPERTY(meta = (DataflowOutput))
	FString String;

	UPROPERTY(meta = (DataflowOutput))
	FString StringDefault;

	FGetStringOverrideFromAssetDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowOverrideNode(InParam, InGuid)
	{
		RegisterOutputConnection(&String);
		RegisterOutputConnection(&StringDefault);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void GeometryCollectionOverrideNodes();
}
