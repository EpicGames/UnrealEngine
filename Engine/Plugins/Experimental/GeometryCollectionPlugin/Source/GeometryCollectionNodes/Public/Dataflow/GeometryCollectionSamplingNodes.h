// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowDebugDraw.h"
#include "FractureEngineSampling.h"

#include "Skeletonization/MeshMedialAxisSampling.h"
#include "Dataflow/DataflowMedialSkeleton.h"
#include "Dataflow/DataflowMesh.h"
#include "Dataflow/DataflowPoints.h"

#include "GeometryCollectionSamplingNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class FGeometryCollection;
class UDynamicMesh;

/** Flags to control which mesh point filtering method(s) are applied */
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EFilterPointSetWithMeshDataflowMethodFlags : uint8
{
	None = 0 UMETA(Hidden),
	// Use the winding number to filter inside or outside of the mesh
	Winding = 1 << 0,
	// Filter away points below a minimum mesh distance
	MinDistance = 1 << 1,
	// Filter away points above a maximum mesh distance
	MaxDistance = 1 << 2
};
ENUM_CLASS_FLAGS(EFilterPointSetWithMeshDataflowMethodFlags)

/**
 * Filter a point set to only the points inside or outside of a given mesh
 * DEPRECATED 5.8 - use FFilterPointSetWithMeshDataflowNode_v2 instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FFilterPointSetWithMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFilterPointSetWithMeshDataflowNode, "FilterPointsWithMesh", "PointSampling", "")

public:
	FFilterPointSetWithMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Mesh to use to filter point set */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> TargetMesh;

	UPROPERTY(EditAnywhere, Category=Options, meta=(Bitmask, BitmaskEnum="/Script/GeometryCollectionNodes.EFilterPointSetWithMeshDataflowMethodFlags"))
	uint8 FilterMethod = (uint8)EFilterPointSetWithMeshDataflowMethodFlags::Winding;

	/** Whether to keep the points inside or (if false) outside the mesh, when filtering by Winding Number. */
	UPROPERTY(EditAnywhere, Category = WindingNumber, meta = (DataflowInput, EditCondition = "FilterMethod & \"/Script/GeometryCollectionNodes.EFilterPointSetWithMeshDataflowMethodFlags::Winding\""))
	bool bKeepInside = true;

	/** The winding number threshold to use for determining whether a point is inside or outside of the mesh, if corresponding Filter Method is set  */
	UPROPERTY(EditAnywhere, Category = WindingNumber, meta = (DataflowInput))
	float WindingThreshold = .5f;

	/** The min distance to surface to keep, if corresponding Filter Method is set  */
	UPROPERTY(EditAnywhere, Category = Distance, meta = (DataflowInput, EditCondition = "FilterMethod & \"/Script/GeometryCollectionNodes.EFilterPointSetWithMeshDataflowMethodFlags::MinDistance\""))
	float MinDistance = 0.f;

	/** The max distance to surface to keep, if corresponding Filter Method is set */
	UPROPERTY(EditAnywhere, Category = Distance, meta = (DataflowInput, EditCondition = "FilterMethod & \"/Script/GeometryCollectionNodes.EFilterPointSetWithMeshDataflowMethodFlags::MaxDistance\""))
	float MaxDistance = 1000.f;

	/**
	 * Whether to use signed distances for the Min and Max Distance thresholds. Otherwise, unsigned distance is used.
	 * Note: Signs are computed via the Winding Number. The sign is negative if the point's Winding Number is below the Winding Threshold. 
	 */
	UPROPERTY(EditAnywhere, Category = Distance, meta = (DataflowInput))
	bool bUseSignedDistance = false;

	/** Points to filter */
	UPROPERTY(meta = (DataflowInput, DataflowOutput))
	TArray<FVector> SamplePoints;


	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Filter a point set to only the points inside or outside of a given Dataflow mesh
 */
 USTRUCT(meta = (DataflowGeometryCollection))
 struct FFilterPointSetWithMeshDataflowNode_v2 : public FDataflowNode
 {
	 GENERATED_USTRUCT_BODY()
	 DATAFLOW_NODE_DEFINE_INTERNAL(FFilterPointSetWithMeshDataflowNode_v2, "FilterPointsWithMesh", "PointSampling", "")

 public:
	 FFilterPointSetWithMeshDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

 private:
	 /** Dataflow mesh to use to filter point set */
	 UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	 TObjectPtr<UDataflowMesh> TargetMesh;

	 UPROPERTY(EditAnywhere, Category = Options, meta = (Bitmask, BitmaskEnum = "/Script/GeometryCollectionNodes.EFilterPointSetWithMeshDataflowMethodFlags"))
	 uint8 FilterMethod = (uint8)EFilterPointSetWithMeshDataflowMethodFlags::Winding;

	 /** Whether to keep the points inside or (if false) outside the mesh, when filtering by Winding Number. */
	 UPROPERTY(EditAnywhere, Category = WindingNumber, meta = (DataflowInput, EditCondition = "FilterMethod & \"/Script/GeometryCollectionNodes.EFilterPointSetWithMeshDataflowMethodFlags::Winding\""))
	 bool bKeepInside = true;

	 /** The winding number threshold to use for determining whether a point is inside or outside of the mesh, if corresponding Filter Method is set  */
	 UPROPERTY(EditAnywhere, Category = WindingNumber, meta = (DataflowInput))
	 float WindingThreshold = .5f;

	 /** The min distance to surface to keep, if corresponding Filter Method is set  */
	 UPROPERTY(EditAnywhere, Category = Distance, meta = (DataflowInput, EditCondition = "FilterMethod & \"/Script/GeometryCollectionNodes.EFilterPointSetWithMeshDataflowMethodFlags::MinDistance\""))
	 float MinDistance = 0.f;

	 /** The max distance to surface to keep, if corresponding Filter Method is set */
	 UPROPERTY(EditAnywhere, Category = Distance, meta = (DataflowInput, EditCondition = "FilterMethod & \"/Script/GeometryCollectionNodes.EFilterPointSetWithMeshDataflowMethodFlags::MaxDistance\""))
	 float MaxDistance = 1000.f;

	 /**
	  * Whether to use signed distances for the Min and Max Distance thresholds. Otherwise, unsigned distance is used.
	  * Note: Signs are computed via the Winding Number. The sign is negative if the point's Winding Number is below the Winding Threshold.
	  */
	 UPROPERTY(EditAnywhere, Category = Distance, meta = (DataflowInput))
	 bool bUseSignedDistance = false;

	 /** Points to filter */
	 UPROPERTY(meta = (DataflowInput, DataflowOutput))
	 FDataflowPoints SamplePoints;

	 virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Uniform Sampling on a DynamicMesh
 * DEPRECATED 5.8 - use FUniformPointSamplingDataflowNode_v2 instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FUniformPointSamplingDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FUniformPointSamplingDataflowNode, "UniformPointSampling", "PointSampling", "")

public:
	/** Mesh to sample points on */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> TargetMesh;

	/** Desired "radius" of sample points. Spacing between samples is at least 2x this value. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float SamplingRadius = 10.f;

	/** Maximum number of samples requested. If 0 or default value, mesh will be maximally sampled */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0))
	int32 MaxNumSamples = 0;

	/** Density of subsampling used in Poisson strategy. Larger numbers mean "more accurate" (but slower) results. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float SubSampleDensity = 10.f;

	/** Random Seed used to initialize sampling strategies */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	int32 RandomSeed = 0;

	/** Sampled positions on the mesh */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> SamplePoints;

	/** Sampled triangleID */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> SampleTriangleIDs;

	/** Barycentric Coordinates of each Sample Point in it's respective Triangle. */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> SampleBarycentricCoords;

	/** Number of Sampled positions on the mesh */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePoints = 0;

	FUniformPointSamplingDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&TargetMesh);
		RegisterInputConnection(&SamplingRadius) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&MaxNumSamples) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&SubSampleDensity) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&RandomSeed) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterOutputConnection(&SamplePoints);
		RegisterOutputConnection(&SampleTriangleIDs);
		RegisterOutputConnection(&SampleBarycentricCoords);
		RegisterOutputConnection(&NumSamplePoints);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Uniform Sampling on a Dataflow Mesh
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FUniformPointSamplingDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FUniformPointSamplingDataflowNode_v2, "UniformPointSampling", "PointSampling", "")

private:
	/** Mesh to sample points on */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDataflowMesh> TargetMesh;

	/** Desired "radius" of sample points. Spacing between samples is at least 2x this value. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float SamplingRadius = 10.f;

	/** Maximum number of samples requested. If 0 or default value, mesh will be maximally sampled */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0))
	int32 MaxNumSamples = 0;

	/** Density of subsampling used in Poisson strategy. Larger numbers mean "more accurate" (but slower) results. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float SubSampleDensity = 10.f;

	/** Random Seed used to initialize sampling strategies */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	int32 RandomSeed = 0;

	/** Sampled positions on the mesh */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowPoints SamplePoints;

	/** Sampled triangleID */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> SampleTriangleIDs;

	/** Barycentric Coordinates of each Sample Point in it's respective Triangle. */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> SampleBarycentricCoords;

	/** Number of Sampled positions on the mesh */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePoints = 0;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FUniformPointSamplingDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * NonUniform Sampling on a DynamicMesh
 * DEPRECATED 5.8 - use FNonUniformPointSamplingDataflowNode_v2 instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FNonUniformPointSamplingDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FNonUniformPointSamplingDataflowNode, "NonUniformPointSampling", "PointSampling", "")

public:
	/** Mesh to sample points on */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> TargetMesh;

	/** Desired "radius" of sample points. Spacing between samples is at least 2x this value. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float SamplingRadius = 10.f;

	/** Maximum number of samples requested. If 0 or default value, mesh will be maximally sampled */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0))
	int32 MaxNumSamples = 0;

	/** Density of subsampling used in Poisson strategy. Larger numbers mean "more accurate" (but slower) results. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float SubSampleDensity = 10.f;

	/** Random Seed used to initialize sampling strategies */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	int32 RandomSeed = 0;

	/** If MaxSampleRadius > SampleRadius, then output sample radius will be in range [SampleRadius, MaxSampleRadius] */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float MaxSamplingRadius = 10.f;

	/** SizeDistribution setting controls the distribution of sample radii */
	UPROPERTY(EditAnywhere, Category = Distribution)
	ENonUniformSamplingDistributionMode SizeDistribution = ENonUniformSamplingDistributionMode::ENonUniformSamplingDistributionMode_Uniform;

	/** SizeDistributionPower is used to control how extreme the Size Distribution shift is. Valid range is [1,10] */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float SizeDistributionPower = 2.f;

	/** Sampled positions on the mesh */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> SamplePoints;

	/** Sampled radii */
	UPROPERTY(meta = (DataflowOutput))
	TArray<float> SampleRadii;

	/** Sampled triangleID */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> SampleTriangleIDs;

	/** Barycentric Coordinates of each Sample Point in it's respective Triangle. */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> SampleBarycentricCoords;

	/** Number of Sampled positions on the mesh */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePoints = 0;

	FNonUniformPointSamplingDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&TargetMesh);
		RegisterInputConnection(&SamplingRadius) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&MaxNumSamples) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&SubSampleDensity) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&RandomSeed) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&MaxSamplingRadius) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&SizeDistributionPower) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterOutputConnection(&SamplePoints);
		RegisterOutputConnection(&SampleRadii);
		RegisterOutputConnection(&SampleTriangleIDs);
		RegisterOutputConnection(&SampleBarycentricCoords);
		RegisterOutputConnection(&NumSamplePoints);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * NonUniform Sampling on a Dataflow Mesh
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FNonUniformPointSamplingDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FNonUniformPointSamplingDataflowNode_v2, "NonUniformPointSampling", "PointSampling", "")

private:
	/** Mesh to sample points on */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDataflowMesh> TargetMesh;

	/** Desired "radius" of sample points. Spacing between samples is at least 2x this value. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float SamplingRadius = 10.f;

	/** Maximum number of samples requested. If 0 or default value, mesh will be maximally sampled */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0))
	int32 MaxNumSamples = 0;

	/** Density of subsampling used in Poisson strategy. Larger numbers mean "more accurate" (but slower) results. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float SubSampleDensity = 10.f;

	/** Random Seed used to initialize sampling strategies */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	int32 RandomSeed = 0;

	/** If MaxSampleRadius > SampleRadius, then output sample radius will be in range [SampleRadius, MaxSampleRadius] */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float MaxSamplingRadius = 10.f;

	/** SizeDistribution setting controls the distribution of sample radii */
	UPROPERTY(EditAnywhere, Category = Distribution)
	ENonUniformSamplingDistributionMode SizeDistribution = ENonUniformSamplingDistributionMode::ENonUniformSamplingDistributionMode_Uniform;

	/** SizeDistributionPower is used to control how extreme the Size Distribution shift is. Valid range is [1,10] */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float SizeDistributionPower = 2.f;

	/** Sampled positions on the mesh */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowPoints SamplePoints;

	/** Sampled radii */
	UPROPERTY(meta = (DataflowOutput))
	TArray<float> SampleRadii;

	/** Sampled triangleID */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> SampleTriangleIDs;

	/** Barycentric Coordinates of each Sample Point in it's respective Triangle. */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> SampleBarycentricCoords;

	/** Number of Sampled positions on the mesh */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePoints = 0;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FNonUniformPointSamplingDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * VertexWeighted Sampling on a DynamicMesh
 * DEPRECATED 5.8 - use FVertexWeightedPointSamplingDataflowNode_v2 instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FVertexWeightedPointSamplingDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVertexWeightedPointSamplingDataflowNode, "VertexWeightedPointSampling", "PointSampling", "")

public:
	/** Mesh to sample points on */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> TargetMesh;

	/** Weight array */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<float> VertexWeights;

	/** Desired "radius" of sample points. Spacing between samples is at least 2x this value. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float SamplingRadius = 10.f;

	/** Maximum number of samples requested. If 0 or default value, mesh will be maximally sampled */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0))
	int32 MaxNumSamples = 0;

	/** Density of subsampling used in Poisson strategy. Larger numbers mean "more accurate" (but slower) results. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float SubSampleDensity = 10.f;

	/** Random Seed used to initialize sampling strategies */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	int32 RandomSeed = 0;

	/** If MaxSampleRadius > SampleRadius, then output sample radius will be in range [SampleRadius, MaxSampleRadius] */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float MaxSamplingRadius = 10.f;

	/** SizeDistribution setting controls the distribution of sample radii */
	UPROPERTY(EditAnywhere, Category = Distribution)
	ENonUniformSamplingDistributionMode SizeDistribution = ENonUniformSamplingDistributionMode::ENonUniformSamplingDistributionMode_Uniform;

	/** SizeDistributionPower is used to control how extreme the Size Distribution shift is. Valid range is [1,10] */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float SizeDistributionPower = 2.f;

	UPROPERTY(EditAnywhere, Category = Distribution)
	ENonUniformSamplingWeightMode WeightMode = ENonUniformSamplingWeightMode::ENonUniformSamplingWeightMode_WeightedRandom;

	UPROPERTY(EditAnywhere, Category = Options)
	bool bInvertWeights = false;

	/** Sampled positions on the mesh */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> SamplePoints;

	/** Sampled radii */
	UPROPERTY(meta = (DataflowOutput))
	TArray<float> SampleRadii;

	/** Sampled triangleID */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> SampleTriangleIDs;

	/** Barycentric Coordinates of each Sample Point in it's respective Triangle. */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> SampleBarycentricCoords;

	/** Number of Sampled positions on the mesh */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePoints = 0;

	FVertexWeightedPointSamplingDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&TargetMesh);
		RegisterInputConnection(&VertexWeights);
		RegisterInputConnection(&SamplingRadius) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&MaxNumSamples) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&SubSampleDensity) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&RandomSeed) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&MaxSamplingRadius) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterInputConnection(&SizeDistributionPower) .SetCanHidePin(true).SetPinIsHidden(true);
		RegisterOutputConnection(&SamplePoints);
		RegisterOutputConnection(&SampleRadii);
		RegisterOutputConnection(&SampleTriangleIDs);
		RegisterOutputConnection(&SampleBarycentricCoords);
		RegisterOutputConnection(&NumSamplePoints);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * VertexWeighted Sampling on a DynamicMesh
 * If VertexWeights input is not connected it uses max weight for every vertex
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FVertexWeightedPointSamplingDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FVertexWeightedPointSamplingDataflowNode_v2, "VertexWeightedPointSampling", "PointSampling", "")

private:
	/** Dataflow mesh to sample points on */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDataflowMesh> TargetMesh;

	/** Weight array, number of elements in the array has to match number of vertices of the TargetMesh  */
	UPROPERTY(meta = (DataflowInput))
	TArray<float> VertexWeights;

	/** Desired "radius" of sample points. Spacing between samples is at least 2x this value. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float SamplingRadius = 10.f;

	/** Maximum number of samples requested. If 0 or default value, mesh will be maximally sampled */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0))
	int32 MaxNumSamples = 0;

	/** Density of subsampling used in Poisson strategy. Larger numbers mean "more accurate" (but slower) results. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float SubSampleDensity = 10.f;

	/** Random Seed used to initialize sampling strategies */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	int32 RandomSeed = 0;

	/** If MaxSampleRadius > SampleRadius, then output sample radius will be in range [SampleRadius, MaxSampleRadius] */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float MaxSamplingRadius = 10.f;

	/** SizeDistribution setting controls the distribution of sample radii */
	UPROPERTY(EditAnywhere, Category = Distribution)
	ENonUniformSamplingDistributionMode SizeDistribution = ENonUniformSamplingDistributionMode::ENonUniformSamplingDistributionMode_Uniform;

	/** SizeDistributionPower is used to control how extreme the Size Distribution shift is. Valid range is [1,10] */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = 0.f))
	float SizeDistributionPower = 2.f;

	UPROPERTY(EditAnywhere, Category = Distribution)
	ENonUniformSamplingWeightMode WeightMode = ENonUniformSamplingWeightMode::ENonUniformSamplingWeightMode_WeightedRandom;

	UPROPERTY(EditAnywhere, Category = Options)
	bool bInvertWeights = false;

	/** Sampled positions on the mesh */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowPoints SamplePoints;

	/** Sampled radii */
	UPROPERTY(meta = (DataflowOutput))
	TArray<float> SampleRadii;

	/** Sampled triangleID */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> SampleTriangleIDs;

	/** Barycentric Coordinates of each Sample Point in it's respective Triangle. */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FVector> SampleBarycentricCoords;

	/** Number of Sampled positions on the mesh */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSamplePoints = 0;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FVertexWeightedPointSamplingDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Sample the medial skeleton of a target mesh
 * DEPRECATED 5.8 - use FMeshMedialSkeletonSamplingDataflowNode_v2 instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FMeshMedialSkeletonSamplingDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMeshMedialSkeletonSamplingDataflowNode, "MeshMedialSkeletonSampling", "MedialSkeletonSampling", "")

private:
	/** Mesh to sample skeleton on */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> TargetMesh;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowMedialSkeleton MedialSkeleton;

	UPROPERTY(meta = (DataflowOutput, DataflowRenderGroups = "None"))
	TArray<FSphere> MedialSpheres;

	// Stop at this sphere count
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 1))
	int32 MaxSpheres = 1000;

	// Do not split a cluster if its max error is below this threshold
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = .1, ClampMin = 0.))
	float MinClusterErrorToSplit = 10.f;

	// Clusters with fewer vertices than this will be discarded
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 1, UIMax = 10))
	int32 MinClusterSizeToKeep = 4;

	// Whether to test medial skeleton edges for intersection w/ the input surface, and try to refine where these intersections are found
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	bool bSplitClustersIfEdgesIntersectSurface = true;

	// Factor to scale vertex error near skeleton-edge/input-surface intersections
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (DataflowInput, ClampMin = 1, UIMax = 100))
	float ErrorScaleNearEdgeSurfaceIntersection = 10.f;

	// Weight for the position error term. Relative to plane error term, so normal and position error weights sum to 1. Should be in the (0,1] range.
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (DisplayName = "Position Error Weight", DataflowInput, UIMin = .1, ClampMin = .001, ClampMax = 1))
	float PosErrorWt = .2f;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FMeshMedialSkeletonSamplingDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Sample the medial skeleton of a target Dataflow mesh
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FMeshMedialSkeletonSamplingDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMeshMedialSkeletonSamplingDataflowNode_v2, "MeshMedialSkeletonSampling", "MedialSkeletonSampling", "")

private:
	/** Dataflow mesh to sample skeleton on */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDataflowMesh> TargetMesh;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowMedialSkeleton MedialSkeleton;

	UPROPERTY(meta = (DataflowOutput, DataflowRenderGroups = "None"))
	TArray<FSphere> MedialSpheres;

	// Stop at this sphere count
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 1))
	int32 MaxSpheres = 1000;

	// Do not split a cluster if its max error is below this threshold
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, UIMin = .1, ClampMin = 0.))
	float MinClusterErrorToSplit = 10.f;

	// Clusters with fewer vertices than this will be discarded
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 1, UIMax = 10))
	int32 MinClusterSizeToKeep = 4;

	// Whether to test medial skeleton edges for intersection w/ the input surface, and try to refine where these intersections are found
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	bool bSplitClustersIfEdgesIntersectSurface = true;

	// Factor to scale vertex error near skeleton-edge/input-surface intersections
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (DataflowInput, ClampMin = 1, UIMax = 100))
	float ErrorScaleNearEdgeSurfaceIntersection = 10.f;

	// Weight for the position error term. Relative to plane error term, so normal and position error weights sum to 1. Should be in the (0,1] range.
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (DisplayName = "Position Error Weight", DataflowInput, UIMin = .1, ClampMin = .001, ClampMax = 1))
	float PosErrorWt = .2f;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FMeshMedialSkeletonSamplingDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Simplify a medial skeleton using the quadric error metric
 * DEPRECATED 5.8 - use FSimplifyMedialSkeletonDataflowNode_v2 instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FSimplifyMedialSkeletonDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSimplifyMedialSkeletonDataflowNode, "SimplifyMedialSkeleton", "MedialSkeletonSampling", "Reduce")

private:
	/** Medial skeleton to simplify */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowIntrinsic))
	FDataflowMedialSkeleton MedialSkeleton;

	/** Mesh the skeleton was sampled from, used to compute the error metric */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> TargetMesh;

	UPROPERTY(meta = (DataflowOutput, DataflowRenderGroups = "None"))
	TArray<FSphere> MedialSpheres;

	// Stop simplifying at this sphere count
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 1))
	int32 MinSpheres = 1;

	// If positive, will not collapse edges if it would introduce more quadric error than the square of this threshold.
	// (Note: Not a distance error threshold.)
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (DataflowInput, ClampMin = 0, DisplayName="Quadric Error Threshold (Sqrt)"))
	float QEMErrorThresholdSqrt = 0.;

	// If positive, the maximum edge length to consider for collapse
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 0, Units = "cm"))
	float EdgeLengthThreshold = 0.;
	
	// If positive, the maximum sphere radius to consider for collapse.  If either cluster's medial sphere on an edge has smaller radii, it is allowed to collapse.
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 0, Units = "cm"))
	float SphereRadiusThreshold = 0.;

	// If positive, the minimum medial sphere overlap fraction to consider for collapse -- so e.g. at 1.0 we will only collapse an edge if its smaller medial sphere is contained in the larger.
	// (A degenerate zero-radius sphere is considered to fully overlap any sphere it touches.)
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 0, ClampMax = 1))
	float SphereOverlapThreshold = 0.;

	// If true, only simplify edges of skeleton surfaces (edges that have triangles associated)
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	bool bOnlySimplifySurfaces = true;

	// How much the simplifier should attempt to remain close to the initial skeleton, vs keeping medial spheres close to the mesh surface.
	// In the [0,1] range. At 0 the original skeleton is not considered; at 1 the mesh surface is not considered.
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 0, ClampMax = 1, UIMax = .9999))
	float ClusterSkeletonDistanceWt = .5f;

	// If true, prevent collapses that would introduce intersections between skeleton edges and the mesh
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	bool bPreventEdgeSurfaceIntersections = true;

	// Regularization weight encouraging skeleton clusters to keep their original positions and radii
	//~ Not exposed to users currently
	float ClusterRegularizeWeight = 1e-6f;

	// Whether to split edges attached to skinny triangles before simplifying, which can add useful degrees of freedom to the simplifier
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (DataflowInput))
	bool bSplitThinTriEdges = true;

	// Minimum angle (degrees) at the opposite vertex for an edge to be a split candidate
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (EditCondition="bSplitThinTriEdges", DataflowInput, ClampMin = 90, ClampMax = 180))
	float SplitThinTriEdgeAngleThresholdDeg = 120.f;

	// Position error weight used when re-clustering vertices after thin-triangle edge splits
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (EditCondition="bSplitThinTriEdges", DataflowInput, ClampMin = 0, ClampMax = 1))
	float SplitThinTriEdgePosErrorWt = .2f;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FSimplifyMedialSkeletonDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Simplify a medial skeleton using the quadric error metric
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FSimplifyMedialSkeletonDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSimplifyMedialSkeletonDataflowNode_v2, "SimplifyMedialSkeleton", "MedialSkeletonSampling", "Reduce")

private:
	/** Medial skeleton to simplify */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowIntrinsic))
	FDataflowMedialSkeleton MedialSkeleton;

	/** Mesh the skeleton was sampled from, used to compute the error metric */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDataflowMesh> TargetMesh;

	UPROPERTY(meta = (DataflowOutput, DataflowRenderGroups = "None"))
	TArray<FSphere> MedialSpheres;

	// Stop simplifying at this sphere count
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 1))
	int32 MinSpheres = 1;

	// If positive, will not collapse edges if it would introduce more quadric error than the square of this threshold.
	// (Note: Not a distance error threshold.)
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (DataflowInput, ClampMin = 0, DisplayName = "Quadric Error Threshold (Sqrt)"))
	float QEMErrorThresholdSqrt = 0.;

	// If positive, the maximum edge length to consider for collapse
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 0, Units = "cm"))
	float EdgeLengthThreshold = 0.;

	// If positive, the maximum sphere radius to consider for collapse.  If either cluster's medial sphere on an edge has smaller radii, it is allowed to collapse.
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 0, Units = "cm"))
	float SphereRadiusThreshold = 0.;

	// If positive, the minimum medial sphere overlap fraction to consider for collapse -- so e.g. at 1.0 we will only collapse an edge if its smaller medial sphere is contained in the larger.
	// (A degenerate zero-radius sphere is considered to fully overlap any sphere it touches.)
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 0, ClampMax = 1))
	float SphereOverlapThreshold = 0.;

	// If true, only simplify edges of skeleton surfaces (edges that have triangles associated)
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	bool bOnlySimplifySurfaces = true;

	// How much the simplifier should attempt to remain close to the initial skeleton, vs keeping medial spheres close to the mesh surface.
	// In the [0,1] range. At 0 the original skeleton is not considered; at 1 the mesh surface is not considered.
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 0, ClampMax = 1, UIMax = .9999))
	float ClusterSkeletonDistanceWt = .5f;

	// If true, prevent collapses that would introduce intersections between skeleton edges and the mesh
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	bool bPreventEdgeSurfaceIntersections = true;

	// Regularization weight encouraging skeleton clusters to keep their original positions and radii
	//~ Not exposed to users currently
	float ClusterRegularizeWeight = 1e-6f;

	// Whether to split edges attached to skinny triangles before simplifying, which can add useful degrees of freedom to the simplifier
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (DataflowInput))
	bool bSplitThinTriEdges = true;

	// Minimum angle (degrees) at the opposite vertex for an edge to be a split candidate
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (EditCondition = "bSplitThinTriEdges", DataflowInput, ClampMin = 90, ClampMax = 180))
	float SplitThinTriEdgeAngleThresholdDeg = 120.f;

	// Position error weight used when re-clustering vertices after thin-triangle edge splits
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (EditCondition = "bSplitThinTriEdges", DataflowInput, ClampMin = 0, ClampMax = 1))
	float SplitThinTriEdgePosErrorWt = .2f;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FSimplifyMedialSkeletonDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/** Which sphere radius to use when computing radius-based target edge lengths */
UENUM()
enum class ESubdivideEdgeRadiusReference : uint8
{
	// Use the larger of the two sphere radii on the edge
	Larger,
	// Use the smaller of the two sphere radii on the edge
	Smaller,
	// Use the average of the two sphere radii on the edge
	Average
};

/**
 * Subdivide a medial skeleton by splitting edges that exceed a target length
 * DEPRECATED 5.8 - use FSubdivideMedialSkeletonDataflowNode_v2 instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FSubdivideMedialSkeletonDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSubdivideMedialSkeletonDataflowNode, "SubdivideMedialSkeleton", "MedialSkeletonSampling", "Subdivide Refine Split")

private:
	/** Medial skeleton to subdivide */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowIntrinsic))
	FDataflowMedialSkeleton MedialSkeleton;

	/** Mesh the skeleton was sampled from, used to re-cluster vertices after subdivision */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> TargetMesh;

	UPROPERTY(meta = (DataflowOutput, DataflowRenderGroups = "None"))
	TArray<FSphere> MedialSpheres;

	// If positive, target edge length as a fraction of a medial sphere radius on the edge (see Radius Reference)
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 0))
	float TargetEdgeLengthRadiusFraction = 0.f;

	// Which sphere radius to reference when computing radius-based target edge lengths
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, EditCondition = "TargetEdgeLengthRadiusFraction > 0"))
	ESubdivideEdgeRadiusReference RadiusReference = ESubdivideEdgeRadiusReference::Average;

	// If positive, minimum target edge length
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 0, Units = "cm"))
	float MinTargetEdgeLength = 0.f;

	// If positive, maximum target edge length. Also used as the target directly when Radius Fraction and Min are both 0
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 0, Units = "cm"))
	float MaxTargetEdgeLength = 0.f;

	// If true, also subdivide edges that have associated triangles (splitting those triangles)
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	bool bSubdivideOnSurfaces = false;

	// Position error weight for re-clustering vertices after edge splits
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (DataflowInput, ClampMin = 0, ClampMax = 1))
	float ReassignClusterPosErrorWt = .2f;

	// If true, project newly-added spheres to the medial axis. Otherwise, new spheres will linearly interpolate their source edge spheres.
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	bool bProjectNewMedialSpheres = false;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FSubdivideMedialSkeletonDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Subdivide a medial skeleton by splitting edges that exceed a target length
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FSubdivideMedialSkeletonDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSubdivideMedialSkeletonDataflowNode_v2, "SubdivideMedialSkeleton", "MedialSkeletonSampling", "Subdivide Refine Split")

private:
	/** Medial skeleton to subdivide */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowIntrinsic))
	FDataflowMedialSkeleton MedialSkeleton;

	/** Mesh the skeleton was sampled from, used to re-cluster vertices after subdivision */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDataflowMesh> TargetMesh;

	UPROPERTY(meta = (DataflowOutput, DataflowRenderGroups = "None"))
	TArray<FSphere> MedialSpheres;

	// If positive, target edge length as a fraction of a medial sphere radius on the edge (see Radius Reference)
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 0))
	float TargetEdgeLengthRadiusFraction = 0.f;

	// Which sphere radius to reference when computing radius-based target edge lengths
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, EditCondition = "TargetEdgeLengthRadiusFraction > 0"))
	ESubdivideEdgeRadiusReference RadiusReference = ESubdivideEdgeRadiusReference::Average;

	// If positive, minimum target edge length
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 0, Units = "cm"))
	float MinTargetEdgeLength = 0.f;

	// If positive, maximum target edge length. Also used as the target directly when Radius Fraction and Min are both 0
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 0, Units = "cm"))
	float MaxTargetEdgeLength = 0.f;

	// If true, also subdivide edges that have associated triangles (splitting those triangles)
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	bool bSubdivideOnSurfaces = false;

	// Position error weight for re-clustering vertices after edge splits
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (DataflowInput, ClampMin = 0, ClampMax = 1))
	float ReassignClusterPosErrorWt = .2f;

	// If true, project newly-added spheres to the medial axis. Otherwise, new spheres will linearly interpolate their source edge spheres.
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	bool bProjectNewMedialSpheres = false;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FSubdivideMedialSkeletonDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

namespace UE::Dataflow
{
	void GeometryCollectionSamplingNodes();
}

