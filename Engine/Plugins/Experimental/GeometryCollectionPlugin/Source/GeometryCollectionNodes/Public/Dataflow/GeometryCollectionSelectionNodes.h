// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowSelection.h"
#include "Dataflow/DataflowAnyType.h"
#include "Math/MathFwd.h"
#include "Math/Sphere.h"
#include "Dataflow/DataflowConnectionTypes.h"
#include "Dataflow/DataflowPoints.h"
#include "Dataflow/DataflowPrimitiveTypes.h"

#include "GeometryCollectionSelectionNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class FGeometryCollection;


/**
 *
 * Selects all the bones for the Collection
 * DEPRECATED 5.8 - use FCollectionSelectionAllDataflowNode instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionTransformSelectionAllDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionAllDataflowNode, "CollectionTransformSelectAll", "GeometryCollection|Selection|Transform", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionAllDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


UENUM(BlueprintType)
enum class ESetOperationEnum : uint8
{
	/** Select elements that are selected in both incoming selections (Bitwise AND) */
	Dataflow_SetOperation_AND UMETA(DisplayName = "Intersect"),
	/** Select elements that are selected in either incoming selections (Bitwise OR) */
	Dataflow_SetOperation_OR UMETA(DisplayName = "Union"),
	/** Select elements that are selected in exactly one incoming selection (Bitwise XOR) */
	Dataflow_SetOperation_XOR UMETA(DisplayName = "Symmetric Difference (XOR)"),
	/** Select elements that are selected in only the first of the incoming selections (Bitwise A AND (NOT B)) */
	Dataflow_SetOperation_Subtract UMETA(DisplayName = "Difference"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 * Runs boolean operation on TransformSelections
 * Deprecated (5.6) : use the generic CollectionSelectionSetOperation node instead
 */

USTRUCT(meta = (Deprecated = "5.6"))
struct FCollectionTransformSelectionSetOperationDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionSetOperationDataflowNode, "CollectionTransformSelectionSetOperation", "GeometryCollection|Selection|Transform", "")

public:
	/** Boolean operation */
	UPROPERTY(EditAnywhere, Category = "Compare");
	ESetOperationEnum Operation = ESetOperationEnum::Dataflow_SetOperation_AND;

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelectionA", DataflowIntrinsic))
	FDataflowTransformSelection TransformSelectionA;

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelectionB", DataflowIntrinsic))
	FDataflowTransformSelection TransformSelectionB;

	/** Array of the selected bone indices after operation*/
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection", DataflowPassthrough = "TransformSelectionA"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionSetOperationDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&TransformSelectionA);
		RegisterInputConnection(&TransformSelectionB);
		RegisterOutputConnection(&TransformSelection, &TransformSelectionA);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Generates a formatted string of the bones and the selection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionTransformSelectionInfoDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionInfoDataflowNode, "CollectionTransformSelectionInfo", "GeometryCollection|Selection|Transform", "")

public:
	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection", DataflowIntrinsic))
	FDataflowTransformSelection TransformSelection;

	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput))
	FManagedArrayCollection Collection;

	/** Formatted string of the bones and selection */
	UPROPERTY(meta = (DataflowOutput))
	FString String;

	FCollectionTransformSelectionInfoDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&TransformSelection);
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&String);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Generates an empty bone selection for the Collection
 * DEPRECATED 5.8 - use FCollectionSelectionNoneDataflowNode instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionTransformSelectionNoneDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionNoneDataflowNode, "CollectionTransformSelectNone", "GeometryCollection|Selection|Transform", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionNoneDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Inverts selection of bones
 *
 */
USTRUCT(meta = (Deprecated = "5.6"))
struct FCollectionTransformSelectionInvertDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionInvertDataflowNode, "CollectionTransformSelectInvert", "GeometryCollection|Selection|Transform", "")

public:
	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "TransformSelection", DataflowPassthrough = "TransformSelection", DataflowIntrinsic))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionInvertDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&TransformSelection);
		RegisterOutputConnection(&TransformSelection, &TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Selects bones randomly in the Collection
 * DEPRECATED 5.8 - use FCollectionSelectionRandomDataflowNode instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionTransformSelectionRandomDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionRandomDataflowNode, "CollectionTransformSelectRandom", "GeometryCollection|Selection|Transform", "")

public:
	/** If true, it always generates the same result for the same RandomSeed */
	UPROPERTY(EditAnywhere, Category = "Random")
	bool bDeterministic = false;

	/** Seed for the random generation, only used if Deterministic is on */
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput, EditCondition = "bDeterministic"))
	float RandomSeed = 0.f;

	/** Bones get selected if RandomValue < RandomThreshold */
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput, UIMin = 0.f, UIMax = 1.f))
	float RandomThreshold = 0.5f;

	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionRandomDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RandomSeed = FMath::FRandRange(-1e5, 1e5);
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&RandomSeed);
		RegisterInputConnection(&RandomThreshold);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Selects the root bones in the Collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionTransformSelectionRootDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionRootDataflowNode, "CollectionTransformSelectRoot", "GeometryCollection|Selection|Transform", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionRootDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 * 
 * Selects specified bones in the GeometryCollection by using a 
 * space separated list, e.g. "0 1 2 12 23"
 * DEPRECATED 5.6 - use FCollectionTransformSelectionCustomDataflowNode_v2 instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.6"))
struct FCollectionTransformSelectionCustomDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionCustomDataflowNode, "CollectionTransformSelectCustom", "GeometryCollection|Selection|Transform", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Space separated list of bone indices to specify the selection, e.g. "0 1 2 3 23 34" */
	UPROPERTY(EditAnywhere, Category = "Selection", meta=(DisplayName="Bone Indices"))
	FString BoneIndicies = FString(); //Fix typo for v2

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionCustomDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&BoneIndicies);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Selects specified bones in the GeometryCollection by using a
 * comma separated list, e.g. "0, 2, 5-10, 12-15"
 * DEPRECATED 5.8 - use FCollectionSelectionCustomDataflowNode instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionTransformSelectionCustomDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionCustomDataflowNode_v2, "CollectionTransformSelectCustom", "GeometryCollection|Selection|Transform", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Comma separated list of single or a range of bone indices to specify the selection, e.g. "0, 2, 5-10, 12-15" */
	UPROPERTY(EditAnywhere, Category = "Selection", meta = (DisplayName = "Bone Indices"))
	FString BoneIndices;

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionCustomDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Convert index array to a transform selection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionTransformSelectionFromIndexArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionFromIndexArrayDataflowNode, "CollectionTransformSelectionFromIndexArray", "GeometryCollection|Selection|Array", "")

public:

	/** Collection to use for the selection. Note only valid bone indices for the collection will be included in the output selection. */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Array of bone indices to convert to a trannsform selection */
	UPROPERTY(EditAnywhere, Category = "Selection", meta = (DataflowInput))
	TArray<int32> BoneIndices;

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionFromIndexArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&BoneIndices);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Selects the parents of the currently selected bones
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionTransformSelectionParentDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionParentDataflowNode, "CollectionTransformSelectParent", "GeometryCollection|Selection|Transform", "")

public:
	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "TransformSelection", DataflowPassthrough = "TransformSelection", DataflowIntrinsic))
	FDataflowTransformSelection TransformSelection;

	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	FCollectionTransformSelectionParentDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection, &TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Outputs the specified percentage of the selected bones
 * DEPRECATED 5.8 - use FCollectionSelectionByPercentageDataflowNode instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionTransformSelectionByPercentageDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionByPercentageDataflowNode, "CollectionTransformSelectByPercentage", "GeometryCollection|Selection|Transform", "")

public:
	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "TransformSelection", DataflowPassthrough = "TransformSelection", DataflowIntrinsic))
	FDataflowTransformSelection TransformSelection;

	/** Percentage to keep from the original selection */
	UPROPERTY(EditAnywhere, Category = "Selection", meta = (UIMin = 0, UIMax = 100))
	int32 Percentage = 100;

	/** Sets the random generation to deterministic */
	UPROPERTY(EditAnywhere, Category = "Random")
	bool bDeterministic = false;

	/** Seed value for the random generation */
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput, EditCondition = "bDeterministic"))
	float RandomSeed = 0.f;

	FCollectionTransformSelectionByPercentageDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RandomSeed = FMath::RandRange(-100000, 100000);
		RegisterInputConnection(&TransformSelection);
		RegisterInputConnection(&Percentage);
		RegisterInputConnection(&RandomSeed);
		RegisterOutputConnection(&TransformSelection, &TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Selects the children of the selected bones
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionTransformSelectionChildrenDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionChildrenDataflowNode, "CollectionTransformSelectChildren", "GeometryCollection|Selection|Transform", "")

public:
	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "TransformSelection", DataflowPassthrough = "TransformSelection", DataflowIntrinsic))
	FDataflowTransformSelection TransformSelection;

	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	FCollectionTransformSelectionChildrenDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection, &TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Selects the siblings of the selected bones
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionTransformSelectionSiblingsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionSiblingsDataflowNode, "CollectionTransformSelectSiblings", "GeometryCollection|Selection|Transform", "")

public:
	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "TransformSelection", DataflowPassthrough = "TransformSelection", DataflowIntrinsic))
	FDataflowTransformSelection TransformSelection;

	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	FCollectionTransformSelectionSiblingsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection, &TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Expand the selection to include all nodes with the same level as the selected nodes
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionTransformSelectionLevelDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionLevelDataflowNode, "CollectionTransformSelectSameLevel", "GeometryCollection|Selection|Transform", "")

public:
	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "TransformSelection", DataflowPassthrough = "TransformSelection", DataflowIntrinsic))
	FDataflowTransformSelection TransformSelection;

	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	FCollectionTransformSelectionLevelDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection, &TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Selects the root bones in the Collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionTransformSelectionTargetLevelDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionTargetLevelDataflowNode, "CollectionTransformSelectTargetLevel", "GeometryCollection|Selection|Transform", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Level to select */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 0))
	int32 TargetLevel = 1;

	/** Whether to avoid embedded geometry in the selection (i.e., only select rigid and cluster nodes) */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bSkipEmbedded = false;

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionTargetLevelDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TargetLevel);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};



/**
 *
 * Selects the contact(s) of the selected bones
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionTransformSelectionContactDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionContactDataflowNode, "CollectionTransformSelectContact", "GeometryCollection|Selection|Transform", "")

public:
	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "TransformSelection", DataflowPassthrough = "TransformSelection", DataflowIntrinsic))
	FDataflowTransformSelection TransformSelection;

	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Whether to allow contact with bones that are in a parent level */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bAllowContactInParentLevels = true;

	FCollectionTransformSelectionContactDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection, &TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Selects the leaves in the Collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionTransformSelectionLeafDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionLeafDataflowNode, "CollectionTransformSelectLeaf", "GeometryCollection|Selection|Transform", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionLeafDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Selects the clusters in the Collection
 * Deprecated : this node had the wrong behavior and select the leaves instead
 *				Replace it by CollectionTransformSelectLeaf or use the second version of CollectionTransformSelectCluster
 *
 */
USTRUCT(meta = (Deprecated = "5.5"))
struct FCollectionTransformSelectionClusterDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionClusterDataflowNode, "CollectionTransformSelectCluster", "GeometryCollection|Selection|Transform", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionClusterDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Selects the clusters in the Collection
 * this version works properly and address the issues found in the deprecated version 1
 */
USTRUCT()
struct FCollectionTransformSelectionClusterDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionClusterDataflowNode_v2, "CollectionTransformSelectCluster", "GeometryCollection|Selection|Transform", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionClusterDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


UENUM(BlueprintType)
enum class ERangeSettingEnum : uint8
{
	/** Values for selection must be inside of the specified range */
	Dataflow_RangeSetting_InsideRange UMETA(DisplayName = "Inside Range"),
	/** Values for selection must be outside of the specified range */
	Dataflow_RangeSetting_OutsideRange UMETA(DisplayName = "Outside Range"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};


/**
 *
 * Selects indices of a float array by range
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FSelectFloatArrayIndicesInRangeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSelectFloatArrayIndicesInRangeDataflowNode, "SelectFloatArrayIndicesInRange", "GeometryCollection|Selection|Array", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput))
	TArray<float> Values;

	/** Minimum value for the selection */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (UIMin = 0.f, UIMax = 1000000000.f))
	float Min = 0.f;

	/** Maximum value for the selection */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (UIMin = 0.f, UIMax = 1000000000.f))
	float Max = 1000.f;

	/** Values for the selection has to be inside or outside [Min, Max] range */
	UPROPERTY(EditAnywhere, Category = "Attribute")
	ERangeSettingEnum RangeSetting = ERangeSettingEnum::Dataflow_RangeSetting_InsideRange;

	/** If true then range includes Min and Max values */
	UPROPERTY(EditAnywhere, Category = "Attribute")
	bool bInclusive = true;

	/** Indices of float Values matching the specified range */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int> Indices;

	FSelectFloatArrayIndicesInRangeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Values);
		RegisterInputConnection(&Min);
		RegisterInputConnection(&Max);
		RegisterOutputConnection(&Indices);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Selects pieces based on their size
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionTransformSelectionBySizeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionBySizeDataflowNode, "CollectionTransformSelectBySize", "GeometryCollection|Selection|Transform", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Minimum size for the selection */
	UPROPERTY(EditAnywhere, Category = "Size", meta = (UIMin = 0.f, UIMax = 1000000000.f))
	float SizeMin = 0.f;

	/** Maximum size for the selection */
	UPROPERTY(EditAnywhere, Category = "Size", meta = (UIMin = 0.f, UIMax = 1000000000.f))
	float SizeMax = 1000.f;

	/** Values for the selection has to be inside or outside [Min, Max] range */
	UPROPERTY(EditAnywhere, Category = "Size")
	ERangeSettingEnum RangeSetting = ERangeSettingEnum::Dataflow_RangeSetting_InsideRange;

	/** If true then range includes Min and Max values */
	UPROPERTY(EditAnywhere, Category = "Size")
	bool bInclusive = true;

	/** Whether to use the 'Relative Size' -- i.e., the Size / Largest Bone Size. Otherwise, Size is the cube root of Volume. */
	UPROPERTY(EditAnywhere, Category = "Size")
	bool bUseRelativeSize = true;

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionBySizeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&SizeMin);
		RegisterInputConnection(&SizeMax);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Selects pieces based on their volume
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionTransformSelectionByVolumeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionByVolumeDataflowNode, "CollectionTransformSelectByVolume", "GeometryCollection|Selection|Transform", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Minimum volume for the selection */
	UPROPERTY(EditAnywhere, Category = "Volume", meta = (UIMin = 0.f, UIMax = 1000000000.f))
	float VolumeMin = 0.f;

	/** Maximum volume for the selection */
	UPROPERTY(EditAnywhere, Category = "Volume", meta = (UIMin = 0.f, UIMax = 1000000000.f))
	float VolumeMax = 1000.f;

	/** Values for the selection has to be inside or outside [Min, Max] range */
	UPROPERTY(EditAnywhere, Category = "Volume")
	ERangeSettingEnum RangeSetting = ERangeSettingEnum::Dataflow_RangeSetting_InsideRange;

	/** If true then range includes Min and Max values */
	UPROPERTY(EditAnywhere, Category = "Volume")
	bool bInclusive = true;

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionByVolumeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&VolumeMin);
		RegisterInputConnection(&VolumeMax);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


UENUM(BlueprintType)
enum class ESelectSubjectTypeEnum : uint8
{
	/** InBox must contain the vertices of the bone */
	Dataflow_SelectSubjectType_Vertices UMETA(DisplayName = "Vertices"),
	/** InBox must contain the BoundingBox of the bone */
	Dataflow_SelectSubjectType_BoundingBox UMETA(DisplayName = "BoundingBox"),
	/** InBox must contain the centroid of the bone */
	Dataflow_SelectSubjectType_Centroid UMETA(DisplayName = "Centroid"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

/**
 *
 * Selects bones if their Vertices/BoundingBox/Centroid in a box
 * DEPRECATED 5.8 - use FCollectionSelectionByPrimitiveDataflowNode instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionTransformSelectionInBoxDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionInBoxDataflowNode, "CollectionTransformSelectInBox", "GeometryCollection|Selection|Transform", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Box to contain Vertices/BoundingBox/Centroid */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FBox Box = FBox(ForceInit);

	/** Transform for the box */
	UPROPERTY(EditAnywhere, Category = "Select")
	FTransform Transform;

	/** Subject (Vertices/BoundingBox/Centroid) to check against box */
	UPROPERTY(EditAnywhere, Category = "Select", DisplayName = "Type to Check in Box")
	ESelectSubjectTypeEnum Type = ESelectSubjectTypeEnum::Dataflow_SelectSubjectType_Centroid;

	/** If true all the vertices of the piece must be inside of box */
	UPROPERTY(EditAnywhere, Category = "Select", meta = (EditCondition = "Type == ESelectSubjectTypeEnum::Dataflow_SelectSubjectType_Vertices"))
	bool bAllVerticesMustContainedInBox = true;

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionInBoxDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&Box);
		RegisterInputConnection(&Transform);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

#if WITH_EDITOR
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif //WITH_EDITOR
};

/**
 *
 * Selects bones if their Vertices/BoundingBox/Centroid in a sphere
 * DEPRECATED 5.8 - use FCollectionSelectionByPrimitiveDataflowNode instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionTransformSelectionInSphereDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionInSphereDataflowNode, "CollectionTransformSelectInSphere", "GeometryCollection|Selection|Transform", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Sphere to contain Vertices/BoundingBox/Centroid */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FSphere Sphere = FSphere(ForceInit);

	/** Transform for the sphere */
	UPROPERTY(EditAnywhere, Category = "Select")
	FTransform Transform;

	/** Subject (Vertices/BoundingBox/Centroid) to check against box */
	UPROPERTY(EditAnywhere, Category = "Select", DisplayName = "Type to Check in Sphere")
	ESelectSubjectTypeEnum Type = ESelectSubjectTypeEnum::Dataflow_SelectSubjectType_Centroid;

	/** If true all the vertices of the piece must be inside of box */
	UPROPERTY(EditAnywhere, Category = "Select", meta = (EditCondition = "Type == ESelectSubjectTypeEnum::Dataflow_SelectSubjectType_Vertices"))
	bool bAllVerticesMustContainedInSphere = true;

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionInSphereDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&Sphere);
		RegisterInputConnection(&Transform);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

#if WITH_EDITOR
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif //WITH_EDITOR
};


/**
 *
 * Selects vertices they are contained in a transformed box
 * DEPRECATED 5.8 - use FCollectionSelectionByPrimitiveDataflowNode instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionVertexSelectionByBoxDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionVertexSelectionByBoxDataflowNode, "CollectionVertexSelectByBox", "GeometryCollection|Selection|Vertex", "In BoundingBox Points Transformed")

public:
	FCollectionVertexSelectionByBoxDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Box to contain Vertices */
	UPROPERTY(EditAnywhere, Category = "Select", meta = (DataflowInput))
	FBox Box = FBox(FVector(-50), FVector(+50));

	/** Transform for the box */
	UPROPERTY(EditAnywhere, Category = "Select", meta = (DataflowInput))
	FTransform BoxTransform;

	/** selected vertices */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVertexSelection VertexSelection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

#if WITH_EDITOR
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif //WITH_EDITOR
};


/**
 *
 * Selects vertices they are contained in a sphere
 * DEPRECATED 5.8 - use FCollectionSelectionByPrimitiveDataflowNode instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionVertexSelectionBySphereDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionVertexSelectionBySphereDataflowNode, "CollectionVertexSelectBySphere", "GeometryCollection|Selection|Vertex", "In Points Radius Center")

public:
	FCollectionVertexSelectionBySphereDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** sphere to select vertices */
	UPROPERTY(EditAnywhere, Category = "Select", meta = (DataflowInput))
	FSphere Sphere = FSphere(FVector(0, 0, 0), 50.0);

	/** selected vertices */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVertexSelection VertexSelection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

#if WITH_EDITOR
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif //WITH_EDITOR
};

/**
 *
 * Selects vertices they are contained on a place side 
 * DEPRECATED 5.8 - use FCollectionSelectionByPrimitiveDataflowNode instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionVertexSelectionByPlaneDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionVertexSelectionByPlaneDataflowNode, "CollectionVertexSelectByPlane", "GeometryCollection|Selection|Vertex", "Side Points Normal")

public:
	FCollectionVertexSelectionByPlaneDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** plane to select vertices */
	UPROPERTY(EditAnywhere, Category = "Select", meta = (DataflowInput))
	FTransform PlaneTransform = FTransform(FRotator(0, 90, 0));

	/** true to use the positive side of the plane */
	UPROPERTY(EditAnywhere, Category = "Select")
	bool bPositiveSide = true;

	/** selected vertices */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowVertexSelection VertexSelection;

	/** Size of the debug draw plane in cm */
	UPROPERTY(EditAnywhere, Category = "DebugDraw", meta=(Units=cm, ClampMin = "0.001", UIMin = "0.001"))
	float PlaneSize = 200;

	/** number of vertices along an edge of the debug draw plane */
	UPROPERTY(EditAnywhere, Category = "DebugDraw", meta = (ClampMin = "2", UIMin = "2"))
	int32 PlaneResolution = 20;


	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

#if WITH_EDITOR
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif //WITH_EDITOR
};

/**
 *
 * Selects bones by a float attribute
 * DEPRECATED 5.8 - use FCollectionSelectionByAttributeDataflowNode instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionTransformSelectionByFloatAttrDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionByFloatAttrDataflowNode, "CollectionTransformSelectByFloatAttribute", "GeometryCollection|Selection|Transform", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Group name for the attr */
	UPROPERTY(VisibleAnywhere, Category = "Attribute", meta = (DisplayName = "Group"))
	FString GroupName = FString("Transform");

	/** Attribute name */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Attribute"))
	FString AttrName = FString("");

	/** Minimum value for the selection */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (UIMin = 0.f, UIMax = 1000000000.f))
	float Min = 0.f;

	/** Maximum value for the selection */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (UIMin = 0.f, UIMax = 1000000000.f))
	float Max = 1000.f;

	/** Values for the selection has to be inside or outside [Min, Max] range */
	UPROPERTY(EditAnywhere, Category = "Attribute")
	ERangeSettingEnum RangeSetting = ERangeSettingEnum::Dataflow_RangeSetting_InsideRange;

	/** If true then range includes Min and Max values */
	UPROPERTY(EditAnywhere, Category = "Attribute")
	bool bInclusive = true;

	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionByFloatAttrDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&Min);
		RegisterInputConnection(&Max);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Selects bones by an int attribute
 * DEPRECATED 5.8 - use FCollectionSelectionByPrimitiveDataflowNode instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionTransformSelectionByIntAttrDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionTransformSelectionByIntAttrDataflowNode, "CollectionTransformSelectByIntAttribute", "GeometryCollection|Selection|Transform", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Group name for the attr */
	UPROPERTY(VisibleAnywhere, Category = "Attribute", meta = (DisplayName = "Group"))
	FString GroupName = FString("Transform");

	/** Attribute name */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Attribute"))
	FString AttrName = FString("");

	/** Minimum value for the selection */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (UIMin = 0, UIMax = 1000000000))
	int32 Min = 0;

	/** Maximum value for the selection */
	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (UIMin = 0, UIMax = 1000000000))
	int32 Max = 1000;

	/** Values for the selection has to be inside or outside [Min, Max] range */
	UPROPERTY(EditAnywhere, Category = "Attribute")
	ERangeSettingEnum RangeSetting = ERangeSettingEnum::Dataflow_RangeSetting_InsideRange;

	/** If true then range includes Min and Max values */
	UPROPERTY(EditAnywhere, Category = "Attribute")
	bool bInclusive = true;

	/** Transform selection including the new indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	FCollectionTransformSelectionByIntAttrDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&Min);
		RegisterInputConnection(&Max);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Selects specified vertices in the GeometryCollection by using a
 * space separated list
 * DEPRECATED 5.8 - use FCollectionSelectionCustomDataflowNode instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionVertexSelectionCustomDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionVertexSelectionCustomDataflowNode, "CollectionVertexSelectCustom", "GeometryCollection|Selection|Vertex", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Space separated list of vertex indices to specify the selection */
	UPROPERTY(EditAnywhere, Category = "Selection")
	FString VertexIndicies = FString(); //Fix typo for v2

	/** Vertex selection including the new indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "VertexSelection"))
	FDataflowVertexSelection VertexSelection;

	FCollectionVertexSelectionCustomDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&VertexIndicies);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&VertexSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Selects specified faces in the GeometryCollection by using a
 * space separated list
 * DEPRECATED 5.8 - use FCollectionSelectionCustomDataflowNode instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionFaceSelectionCustomDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionFaceSelectionCustomDataflowNode, "CollectionFaceSelectCustom", "GeometryCollection|Selection|Face", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Space separated list of face indices to specify the selection */
	UPROPERTY(EditAnywhere, Category = "Selection")
	FString FaceIndicies = FString(); //Fix typo for v2

	/** Face selection including the new indices */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "FaceSelection"))
	FDataflowFaceSelection FaceSelection;

	FCollectionFaceSelectionCustomDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&FaceIndicies);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&FaceSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Converts Vertex/Face/Transform selection into Vertex/Face/Transform selection
 * DEPRECATED 5.8 - use FCollectionSelectionConvertDataflowNode_v2 instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionSelectionConvertDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSelectionConvertDataflowNode, "CollectionSelectionConvert", "GeometryCollection|Selection", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Transform selection including the new indices */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "TransformSelection", DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/** Face selection including the new indices */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "FaceSelection", DisplayName = "FaceSelection"))
	FDataflowFaceSelection FaceSelection;

	/** Vertex selection including the new indices */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "VertexSelection", DisplayName = "VertexSelection"))
	FDataflowVertexSelection VertexSelection;
	
	/** If true then for converting vertex/face selection to transform selection all vertex/face must be selected for selecting the associated transform */
	UPROPERTY(EditAnywhere, Category = "Selection")
	bool bAllElementsMustBeSelected = false;

	FCollectionSelectionConvertDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&VertexSelection);
		RegisterInputConnection(&FaceSelection);
		RegisterInputConnection(&TransformSelection);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&TransformSelection, &TransformSelection);
		RegisterOutputConnection(&FaceSelection, &FaceSelection);
		RegisterOutputConnection(&VertexSelection, &VertexSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Inverts selection of faces
 *
 */
USTRUCT(meta = (Deprecated = "5.6"))
struct FCollectionFaceSelectionInvertDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionFaceSelectionInvertDataflowNode, "CollectionFaceSelectInvert", "GeometryCollection|Selection|Face", "")

public:
	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "FaceSelection", DataflowPassthrough = "FaceSelection", DataflowIntrinsic))
	FDataflowFaceSelection FaceSelection;

	FCollectionFaceSelectionInvertDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&FaceSelection);
		RegisterOutputConnection(&FaceSelection, &FaceSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};


/**
 *
 * Outputs the specified percentage of the selected vertices
 * DEPRECATED 5.8 - use FCollectionSelectionByPercentageDataflowNode instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionVertexSelectionByPercentageDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionVertexSelectionByPercentageDataflowNode, "CollectionVertexSelectByPercentage", "GeometryCollection|Selection|Vertex", "")

public:
	/** Array of the selected bone indices */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "VertexSelection", DataflowPassthrough = "VertexSelection", DataflowIntrinsic))
	FDataflowVertexSelection VertexSelection;

	/** Percentage to keep from the original selection */
	UPROPERTY(EditAnywhere, Category = "Selection", meta = (UIMin = 0, UIMax = 100))
	int32 Percentage = 100;

	/** Sets the random generation to deterministic */
	UPROPERTY(EditAnywhere, Category = "Random")
	bool bDeterministic = false;

	/** Seed value for the random generation */
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput, EditCondition = "bDeterministic"))
	float RandomSeed = 0.f;

	FCollectionVertexSelectionByPercentageDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RandomSeed = FMath::RandRange(-100000, 100000);
		RegisterInputConnection(&VertexSelection);
		RegisterInputConnection(&Percentage);
		RegisterInputConnection(&RandomSeed);
		RegisterOutputConnection(&VertexSelection, &VertexSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/**
 *
 * Runs boolean operation on VertexSelections
 * Deprecated (5.6) : use the generic CollectionSelectionSetOperation node instead
 *
 */
USTRUCT(meta = (Deprecated = "5.6"))
struct FCollectionVertexSelectionSetOperationDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionVertexSelectionSetOperationDataflowNode, "CollectionVertexSelectionSetOperation", "GeometryCollection|Selection|Vertex", "")

public:
	/** Boolean operation */
	UPROPERTY(EditAnywhere, Category = "Compare");
	ESetOperationEnum Operation = ESetOperationEnum::Dataflow_SetOperation_AND;

	/** Array of the selected vertex indices */
	UPROPERTY(meta = (DataflowInput, DisplayName = "VertexSelectionA", DataflowIntrinsic))
	FDataflowVertexSelection VertexSelectionA;

	/** Array of the selected vertex indices */
	UPROPERTY(meta = (DataflowInput, DisplayName = "VertexSelectionB", DataflowIntrinsic))
	FDataflowVertexSelection VertexSelectionB;

	/** Array of the selected vertex indices after operation */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "VertexSelection", DataflowPassthrough = "VertexSelectionA"))
	FDataflowVertexSelection VertexSelection;

	FCollectionVertexSelectionSetOperationDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&VertexSelectionA);
		RegisterInputConnection(&VertexSelectionB);
		RegisterOutputConnection(&VertexSelection, &VertexSelectionA);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

UENUM(BlueprintType)
enum class ESelectionByAttrGroup : uint8
{
	Vertices UMETA(DisplayName = "Vertices"),
	Faces UMETA(DisplayName = "Faces"),
	Transform UMETA(DisplayName = "Transform"),
	Geometry UMETA(DisplayName = "Geometry"),
	Material UMETA(DisplayName = "Material"),
	Curves UMETA(DisplayName = "Curves")
};

namespace UE::Dataflow::Private
{
	inline FName GetAttributeFromEnumAsName(const ESelectionByAttrGroup Value)
	{
		static const UEnum* SelectionByAttrGroupEnum = StaticEnum<ESelectionByAttrGroup>();
		return *SelectionByAttrGroupEnum->GetNameStringByValue((int64)Value);
	}
}

UENUM(BlueprintType)
enum class ESelectionByAttrOperation : uint8
{
	/** Select faces which attribute value equal with specified value */
	Equal UMETA(DisplayName = "=="),
	/** Select faces which attribute value not equal with specified value */
	NotEqual UMETA(DisplayName = "!="),
	/** Select faces which attribute value greater than specified value */
	Greater UMETA(DisplayName = ">"),
	/** Select faces which attribute value greater or equal than specified value */
	GreaterOrEqual UMETA(DisplayName = ">="),
	/** Select faces which attribute value smaller than specified value */
	Smaller UMETA(DisplayName = "<"),
	/** Select faces which attribute value greater than specified value */
	SmallerOrEqual UMETA(DisplayName = "<="),
	/** Select faces which attribute value greater than specified value */
	Maximum UMETA(DisplayName = "Max"),
	/** Select faces which attribute value greater than specified value */
	Minimum UMETA(DisplayName = "Min")
};

/**
 *
 * Selects specified Vertices/Faces/Transforms in the GeometryCollection by using an attribute value
 * Currently supported attribute types: float, int32, String, bool
 * DEPRECATED 5.8 - use FCollectionSelectionByAttributeDataflowNode instead
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FCollectionSelectionByAttrDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSelectionByAttrDataflowNode, "CollectionSelectByAttr", "GeometryCollection|Selection|All", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** AttributeKey input */
	UPROPERTY(meta = (DataflowInput))
	FCollectionAttributeKey AttributeKey;

	/** Group */
	UPROPERTY(EditAnywhere, Category = "Selection")
	ESelectionByAttrGroup Group = ESelectionByAttrGroup::Faces;

	/** Attribute for the selection */
	UPROPERTY(EditAnywhere, Category = "Selection")
	FString Attribute = FString("Internal");

	/** Operation */
	UPROPERTY(EditAnywhere, Category = "Selection")
	ESelectionByAttrOperation Operation = ESelectionByAttrOperation::Equal;

	/** Attribute value for the operation */
	UPROPERTY(EditAnywhere, Category = "Selection")
	FString Value = FString("true");

	/** Vertex selection output */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "VertexSelection"))
	FDataflowVertexSelection VertexSelection;

	/** Face selection output */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "FaceSelection"))
	FDataflowFaceSelection FaceSelection;

	/** Transform selection output */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/** Geometry selection output */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "GeometrySelection"))
	FDataflowGeometrySelection GeometrySelection;

	/** Material selection output */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "MaterialSelection"))
	FDataflowMaterialSelection MaterialSelection;
	
	/** Curve selection output */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "CurveSelection"))
	FDataflowCurveSelection CurveSelection;

	FCollectionSelectionByAttrDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&AttributeKey);
		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&VertexSelection);
		RegisterOutputConnection(&FaceSelection);
		RegisterOutputConnection(&TransformSelection);
		RegisterOutputConnection(&GeometrySelection);
		RegisterOutputConnection(&MaterialSelection);
		RegisterOutputConnection(&CurveSelection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Converts GeometrySelection to VertexSelection
 * DEPRECATED 5.8 - use FCollectionSelectionConvertDataflowNode_v2 instead
 * 
 */
USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.8"))
struct FGeometrySelectionToVertexSelectionDataflowNode final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGeometrySelectionToVertexSelectionDataflowNode, "GeometrySelectionToVertexSelection", "GeometryCollection|Selection|All", "")

public:
	/** GeometryCollection */
	UPROPERTY(meta = (DataflowInput))
	FManagedArrayCollection Collection;

	/** Space separated list of geometry indices to specify the selection when GeometrySelection is not connected*/
	UPROPERTY(EditAnywhere, Category = "Selection")
	FString GeometryIndices = FString();

	/** Input geometry selection */
	UPROPERTY(meta = (DataflowInput, DisplayName = "GeometrySelection"))
	FDataflowGeometrySelection GeometrySelection;

	/** Vertex selection output */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "VertexSelection"))
	FDataflowVertexSelection VertexSelection;

	FGeometrySelectionToVertexSelectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&GeometrySelection);
		RegisterOutputConnection(&VertexSelection);
	}

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Runs boolean operation on selection ( support all selection types )
 *
 */
USTRUCT()
struct FCollectionSelectionSetOperationDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSelectionSetOperationDataflowNode, "CollectionSelectionSetOperation", "GeometryCollection|Selection", "")

private:
	/** Boolean operation */
	UPROPERTY(EditAnywhere, Category = "Operation");
	ESetOperationEnum Operation = ESetOperationEnum::Dataflow_SetOperation_AND;

	/** First Selection object */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowSelectionTypes SelectionA;

	/** Second Selection object */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowSelectionTypes SelectionB;

	/** Array of the selected bone indices after operation*/
	UPROPERTY(meta = (DataflowOutput, DataflowPassthrough = "SelectionA"))
	FDataflowSelectionTypes Selection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FCollectionSelectionSetOperationDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};


/**
 *
 * Inverts selection ( support all selection types )
 *
 */
USTRUCT()
struct FCollectionSelectionInvertDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSelectionInvertDataflowNode, "CollectionSelectionInvert", "GeometryCollection|Selection", "")

private:
	/** selection to invert */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Selection", DataflowIntrinsic))
	FDataflowSelectionTypes Selection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FCollectionSelectionInvertDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Select internal faces
 *
 */
USTRUCT()
struct FCollectionSelectInternalFacesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSelectInternalFacesDataflowNode, "CollectionSelectInternalFaces", "GeometryCollection|Selection", "")

private:
	/** Collection to select the internal faces from */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/**
	* Transform selection to get the internal faces from
	* if this input is not connected, then all internal faces from the collection will be returned
	*/
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection TransformSelection;

	/** selection containing Internal faces */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowFaceSelection FaceSelection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FCollectionSelectInternalFacesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

UENUM()
enum class EDataflowCollectionSelectionByNameMethod: uint8
{
	/** name must match exactly the input text */
	Exact,
	/** name must start with the input text */
	StartsWith, 
	/** name must end with the input text */
	EndsWith,
	/** name must contain the input text */
	Contains,
};

/**
 * Selects transforms by name using a the BoneName attributeor other Transform group string typed attributes 
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionSelectTransformStringDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSelectTransformStringDataflowNode, "CollectionSelectTransformString", "GeometryCollection|Selection|Transform", "Name, Bone, Attribute")

private:
	/** Collection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Text to serach in the name */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	FString Attribute = "BoneName";

	/** Text to serach in the name */
	UPROPERTY(EditAnywhere, Category = Options, meta=(DataflowInput))
	FString SearchText;

	/** Method to use to match the name */
	UPROPERTY(EditAnywhere, Category = "Volume")
	EDataflowCollectionSelectionByNameMethod Method = EDataflowCollectionSelectionByNameMethod::Contains;

	/** output selection of the matching transforms */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowTransformSelection TransformSelection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FCollectionSelectTransformStringDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Set selected transform string value
 * the string format can use the following predefined value : 
 * - {Current} current value of the attribute
 * - {Index} index in the selection passed as input
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionSetTransformStringValueDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSetTransformStringValueDataflowNode, "CollectionSetTransformString", "GeometryCollection|Selection|Transform", "Bone Name Attribute")

private:
	/** Collection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** input selection of the transforms to set - if not connected use all */
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection TransformSelection;

	/** Text to serach in the name */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	FString Attribute = "BoneName";

	/** Text to set  */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	FString TextToSet;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FCollectionSetTransformStringValueDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};


/**
 *
 * Returns number of elements in a Selection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGetNumElementsInSelectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetNumElementsInSelectionDataflowNode, "GetNumElementsInSelection", "GeometryCollection|Utilities", "")

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Selection", DataflowIntrinsic))
	FDataflowSelectionTypes Selection;

	/** Number of all (selected and non-selected) elements in the Selection */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumElements = 0;

	/** Number of selected elements in the Selection */
	UPROPERTY(meta = (DataflowOutput))
	int32 NumSelectedElements = 0;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FGetNumElementsInSelectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/**
 *
 * Creates any type of selection from a collection by using a comma separated list
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionSelectionCustomDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSelectionCustomDataflowNode, "CollectionSelectionCustom", "GeometryCollection|Selection", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Comma separated list of indices (example: "0, 2, 5-10, 12-15") to specify the selection */
	UPROPERTY(EditAnywhere, Category = "Selection", meta = (DataflowInput))
	FString Indices = FString();

	/** Selection from the indices. To set the selection type rt mouse click on the Selection output */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowSelectionTypes Selection;

	FCollectionSelectionCustomDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/* ----------------------------------------------------------------------------------------------------------------------- */

/**
 *
 * Converts Vertex/Face/Transform selection into Vertex/Face/Transform selection
 * Supported groups: Transform/Vertices/Faces/Geometry/Curves
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionSelectionConvertDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSelectionConvertDataflowNode_v2, "CollectionSelectionConvert", "GeometryCollection|Selection", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Vertex/Face/Transform selection to convert. To set the selection type rt mouse click on the Selection output */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Selection", DataflowIntrinsic))
	FDataflowSelectionTypes Selection;

	/** If true then for converting vertex/face selection to transform selection all vertex/face must be selected for selecting the associated transform */
	UPROPERTY(EditAnywhere, Category = "Selection")
	bool bAllElementsMustBeSelected = false;

	FCollectionSelectionConvertDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/* ----------------------------------------------------------------------------------------------------------------------- */

/**
 *
 * Selects all the elements in the selected group from the collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionSelectionAllDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSelectionAllDataflowNode, "CollectionSelectionAll", "GeometryCollection|Selection", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Output selection. To set the selection type rt mouse click on the Selection output */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowSelectionTypes Selection;

	FCollectionSelectionAllDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/* ----------------------------------------------------------------------------------------------------------------------- */

/**
 *
 * Selects none of the elements in the selected group from the collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionSelectionNoneDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSelectionNoneDataflowNode, "CollectionSelectionNone", "GeometryCollection|Selection", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Output selection. To set the selection type rt mouse click on the Selection output */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowSelectionTypes Selection;

	FCollectionSelectionNoneDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/* ----------------------------------------------------------------------------------------------------------------------- */

/**
 *
 * Selects randomly in the collection from the specified group
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionSelectionRandomDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSelectionRandomDataflowNode, "CollectionSelectionRandom", "GeometryCollection|Selection", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** If true, it always generates the same result for the same RandomSeed */
	UPROPERTY(EditAnywhere, Category = "Random")
	bool bDeterministic = false;

	/** Seed for the random generation, only used if Deterministic is on */
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput, EditCondition = "bDeterministic"))
	int32 RandomSeed = 0;

	/** Get selected if RandomValue < RandomThreshold */
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput, UIMin = 0.f, UIMax = 1.f))
	float RandomThreshold = 0.5f;

	/** Output selection. To set the selection type rt mouse click on the Selection output */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowSelectionTypes Selection;

	FCollectionSelectionRandomDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/* ----------------------------------------------------------------------------------------------------------------------- */

/**
 *
 * Selects specified percentage of elements in the collection from the specified group
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionSelectionByPercentageDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSelectionByPercentageDataflowNode, "CollectionSelectionByPercentage", "GeometryCollection|Selection", "")

public:
	/** Selection to selectt from by percentage */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Selection", DataflowIntrinsic))
	FDataflowSelectionTypes Selection;

	/** Percentage to keep from the elements */
	UPROPERTY(EditAnywhere, Category = "Selection", meta = (DataflowInput, UIMin = 0, ClampMin = 0, UIMax = 100, ClampMax = 100))
	int32 Percentage = 100;

	/** If true, it always generates the same result for the same RandomSeed */
	UPROPERTY(EditAnywhere, Category = "Random")
	bool bDeterministic = false;

	/** Seed for the random generation, only used if Deterministic is on */
	UPROPERTY(EditAnywhere, Category = "Random", meta = (DataflowInput, EditCondition = "bDeterministic"))
	int32 RandomSeed = 0;

	FCollectionSelectionByPercentageDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};

/* ----------------------------------------------------------------------------------------------------------------------- */

/**
 *
 * Selects specified Vertices/Faces/Transforms/.... in the collection by using an attribute value
 * Currently supported attribute types: float, int32, String, bool
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionSelectionByAttributeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSelectionByAttributeDataflowNode, "CollectionSelectionByAttribute", "GeometryCollection|Selection", "")

public:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Attribute for the selection */
	UPROPERTY(EditAnywhere, Category = "Selection")
	FString Attribute = FString("Internal");

	/** Operation */
	UPROPERTY(EditAnywhere, Category = "Selection")
	ESelectionByAttrOperation Operation = ESelectionByAttrOperation::Equal;

	/** Attribute value for the operation */
	UPROPERTY(EditAnywhere, Category = "Selection")
	FString Value = FString("true");

	/** Selection output. To set the selection type rt mouse click on the Selection output */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowSelectionTypes Selection;

	FCollectionSelectionByAttributeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/* ----------------------------------------------------------------------------------------------------------------------- */

/**
 *
 * Selects elements which are contained in a transformed box
 * Supported groups: Transform/Vertices/Faces/Geometry/Curves
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionSelectionByPrimitiveDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSelectionByPrimitiveDataflowNode, "CollectionSelectionByPrimitive", "GeometryCollection|Selection", "In BoundingBox Points Transformed")

private:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Primitive used for the selection (Box, Sphere or Plane) */
	UPROPERTY(EditAnywhere, Category = "Selection", meta = (DataflowInput))
	FDataflowPrimitiveTypes Primitive;

	/** Transform for the primitive */
	UPROPERTY(EditAnywhere, Category = "Selection", meta = (DataflowInput, GizmoType = "Transform"))
	FTransform Transform;

	/** If true then all elements must contained by the primitive (all vertices of a face, all vertices of a curve, etc.) */
	UPROPERTY(EditAnywhere, Category = "Selection")
	bool bAllElementsMustBeSelected = false;

	/** Select elements on positive side of the plane. Only used for selection by plane. */
	UPROPERTY(EditAnywhere, Category = "Select")
	bool bPositiveSide = true;

	/** DebugDraw / Plane size */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Plane", meta = (UIMin = 1, ClampMin = 1))
	float PlaneSize = 250.f;

	/** DebugDraw / Vertex per edge */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Plane", meta = (UIMin = 2, ClampMin = 2))
	int32 VertexPerEdge = 5;

	/** Output selection */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowSelectionTypes Selection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

#if WITH_EDITOR
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif //WITH_EDITOR

public:
	FCollectionSelectionByPrimitiveDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/* ----------------------------------------------------------------------------------------------------------------------- */

class UDynamicMesh;

/** Flags to control which mesh point filtering method(s) are applied */
UENUM(BlueprintType)
enum class ESelectionByMeshMethodFlags : uint8
{
	None UMETA(DisplayName = "None"),
	Winding UMETA(DisplayName = "Winding"),
	MinDistance UMETA(DisplayName = "MinDistance"),
	MaxDistance UMETA(DisplayName = "MaxDistance")
};
/**
 *
 * Collection selection bassed on if the element is inside or outside of a given mesh
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionSelectionByMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSelectionByMeshDataflowNode, "CollectionSelectionByMesh", "PointSampling", "")

public:
	FCollectionSelectionByMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** GeometryCollection for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Mesh to use for the selection */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TObjectPtr<UDynamicMesh> Mesh;

	UPROPERTY(EditAnywhere, Category = Options)
	ESelectionByMeshMethodFlags FilterMethod = ESelectionByMeshMethodFlags::Winding;

	/** Whether to keep the points inside or (if false) outside the mesh, when filtering by Winding Number. */
	UPROPERTY(EditAnywhere, Category = WindingNumber, meta = (DataflowInput, EditCondition = "FilterMethod == ESelectionByMeshMethodFlags::Winding"))
	bool bKeepInside = true;

	/** The winding number threshold to use for determining whether a point is inside or outside of the mesh, if corresponding Filter Method is set  */
	UPROPERTY(EditAnywhere, Category = WindingNumber, meta = (DataflowInput))
	float WindingThreshold = .5f;

	/** The min distance to surface to keep, if corresponding Filter Method is set  */
	UPROPERTY(EditAnywhere, Category = Distance, meta = (DataflowInput, EditCondition = "FilterMethod == ESelectionByMeshMethodFlags::MinDistance"))
	float MinDistance = 0.f;

	/** The max distance to surface to keep, if corresponding Filter Method is set */
	UPROPERTY(EditAnywhere, Category = Distance, meta = (DataflowInput, EditCondition = "FilterMethod == ESelectionByMeshMethodFlags::MaxDistance"))
	float MaxDistance = 1000.f;

	/**
	 * Whether to use signed distances for the Min and Max Distance thresholds. Otherwise, unsigned distance is used.
	 * Note: Signs are computed via the Winding Number. The sign is negative if the point's Winding Number is below the Winding Threshold.
	 */
	UPROPERTY(EditAnywhere, Category = Distance, meta = (DataflowInput))
	bool bUseSignedDistance = false;

	/** If true then for converting vertex/face selection to transform selection all vertex/face must be selected for selecting the associated transform */
	UPROPERTY(EditAnywhere, Category = "Selection")
	bool bAllElementsMustBeSelected = false;

	/** Output selection */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowSelectionTypes Selection;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/* ----------------------------------------------------------------------------------------------------------------------- */

/** Attribute type for SelectionToAttribute */
UENUM(BlueprintType)
enum class ESelectionToAttributeTypeFlags : uint8
{
	Int UMETA(DisplayName = "Int"),
	Float UMETA(DisplayName = "Float"),
	Bool UMETA(DisplayName = "Bool"),
	Double UMETA(DisplayName = "Double"),
	String UMETA(DisplayName = "String"),
	Name UMETA(DisplayName = "Name"),
	Vec2f UMETA(DisplayName = "Vec2f"),
	Vec3f UMETA(DisplayName = "Vec3f"),
	Vec4f UMETA(DisplayName = "Vec4f"),
	Vec2i UMETA(DisplayName = "Vec2i"),
	Vec3i UMETA(DisplayName = "Vec3i"),
	Vec4i UMETA(DisplayName = "Vec4i"),
	Color UMETA(DisplayName = "Color"),
	Transform UMETA(DisplayName = "Transform"),
	Matrix UMETA(DisplayName = "Matrix"),
	Quat UMETA(DisplayName = "Quat"),
	Box UMETA(DisplayName = "Box"),
};

/**
 *
 * Creates specified attribute on the incoming selection's group and sets specified value to selected elements,
 * the nonselected elements will be set to default valu. 
 * If the attribute already exists then sets specified value to selected elements.
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionSelectionToAttributeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionSelectionToAttributeDataflowNode, "CollectionSelectionToAttribute", "GeometryCollection|Selection", "")

public:
	/** GeometryCollection input */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Selection input. The type of this selection defines the group where the data will be stored */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowSelectionTypes Selection;

	/** Attribute type for storing the selection if not already in the collection */
	UPROPERTY(EditAnywhere, Category = "Selection")
	ESelectionToAttributeTypeFlags AttributeType = ESelectionToAttributeTypeFlags::Int;

	/** Attribute for storing the selection */
	UPROPERTY(EditAnywhere, Category = "Selection", meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Attribute", DisplayName = "Attribute Name"))
	FString Attribute = FString("Dummy");

	/** Attribute default value */
	UPROPERTY(EditAnywhere, Category = "Selection", meta = (DataflowInput))
	FString DefaultValue = FString("0");

	/** Attribute value */
	UPROPERTY(EditAnywhere, Category = "Selection", meta = (DataflowInput, DisplayName = "Selection Value"))
	FString Value = FString("1");

	FCollectionSelectionToAttributeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/* ----------------------------------------------------------------------------------------------------------------------- */

namespace UE::Dataflow
{
	void GeometryCollectionSelectionNodes();
}
