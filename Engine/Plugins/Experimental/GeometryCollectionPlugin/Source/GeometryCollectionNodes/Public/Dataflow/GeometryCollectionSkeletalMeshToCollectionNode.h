// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "GeometryCollectionSkeletalMeshToCollectionNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class USkeletalMesh;
class UMaterialInterface;


USTRUCT(meta = (DataflowGeometryCollection))
struct FSkeletalMeshToCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSkeletalMeshToCollectionDataflowNode, "SkeletalMeshToCollection", "GeometryCollection", "")

public:
	typedef FManagedArrayCollection DataType;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowInput, DisplayName = "SkeletalMesh"))
	TObjectPtr<const USkeletalMesh> SkeletalMesh = nullptr;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DisplayName = "Transforms Only"))
	bool bImportTransformOnly = false;

	FSkeletalMeshToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&SkeletalMesh);
		RegisterOutputConnection(&Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


USTRUCT(meta = (DataflowGeometryCollection))
struct FCollectionToSkeletalMeshDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCollectionToSkeletalMeshDataflowNode, "CollectionToSkeletalMesh", "GeometryCollection", "")

private:
	UPROPERTY(meta = (DataflowInput))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput))
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<const USkeletalMesh> SkeletalMesh = nullptr;

	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<const USkeleton> Skeleton = nullptr;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

#if WITH_EDITOR
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
#endif 

public:
	FCollectionToSkeletalMeshDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

