// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "GeometryCollectionSkeletonToCollectionNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class USkeletalMesh;

USTRUCT(meta = (DataflowGeometryCollection, Deprecated = "5.5"))
struct UE_DEPRECATED(5.5, "Use SkeletalMeshToCollection instead") FSkeletonToCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSkeletonToCollectionDataflowNode, "SkeletonToCollection", "GeometryCollection", "")

public:
	typedef FManagedArrayCollection DataType;

	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowInput, DisplayName = "Skeleton"))
	TObjectPtr<const USkeleton> Skeleton = nullptr;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "Collection"))
	FManagedArrayCollection Collection;

	FSkeletonToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Skeleton);
		RegisterOutputConnection(&Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


