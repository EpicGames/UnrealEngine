// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/GeometryCollectionAssetNodes.h"
#include "GeometryCollection/GeometryCollectionObject.h"

#include "GeometryCollectionStaticMeshToCollectionNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class UGeometryCollection;
class UStaticMesh;
class UMaterial;

/**
 * Create a geometry collection from a UStaticMesh
 */
USTRUCT()
struct FStaticMeshToCollectionDataflowNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FStaticMeshToCollectionDataflowNode_v2, "StaticMeshToCollection", "GeometryCollection|Asset", "")

private:
	/** Asset input */
	UPROPERTY(EditAnywhere, Category = "Asset", meta = (DataflowInput));
	TObjectPtr<UStaticMesh> StaticMesh;

	/** Transform to apply to the mesh before converting it to a collection */
	UPROPERTY(EditAnywhere, Category = "Asset", meta = (DataflowInput, GizmoType = "Transform"));
	FTransform MeshTransform = FTransform::Identity;

	/** Geometry collection newly created */
	UPROPERTY(meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	/** Material array from the static mesh */
	UPROPERTY(meta = (DataflowOutput))
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	/** Array of instanced meshes*/
	UPROPERTY(meta = (DataflowOutput))
	TArray<FGeometryCollectionAutoInstanceMesh> InstancedMeshes;

	/** corresponding source proxies */
	UPROPERTY(meta = (DataflowOutput))
	TArray<FDataflowRootProxyMesh> RootProxyMeshes;

	/** Set the internal faces from material index */
	UPROPERTY(EditAnywhere, Category = "Asset");
	bool bSetInternalFromMaterialIndex = true;

	/** Split components - when enabled, each island of the mesh will be converted to an individual transform in the collection */
	UPROPERTY(EditAnywhere, Category = "Asset");
	bool bSplitComponents = false;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FStaticMeshToCollectionDataflowNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Create a geometry collection from a UStaticMesh
 * Deprecated (5.6) - use the second version of StaticMeshToCollection node with only one materials ouput
 */
USTRUCT(meta = (Deprecated = "5.6"))
struct FStaticMeshToCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FStaticMeshToCollectionDataflowNode, "StaticMeshToCollection", "GeometryCollection|Asset", "")

public:
	/** Asset input */
	UPROPERTY(EditAnywhere, Category = "Asset", meta = (DataflowInput, DisplayName = "StaticMesh"));
	TObjectPtr<UStaticMesh> StaticMesh;

	/** Transform to apply to the mesh before converting it to a collection */
	UPROPERTY(EditAnywhere, Category = "Asset", meta = (DataflowInput, GizmoType = "Transform"));
	FTransform MeshTransform = FTransform::Identity;

	/** Set the internal faces from material index */
	UPROPERTY(EditAnywhere, Category = "Asset");
	bool bSetInternalFromMaterialIndex = true;

	/** Split components */
	UPROPERTY(EditAnywhere, Category = "Asset");
	bool bSplitComponents = false;

	/** Geometry collection newly created */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Collection"))
	FManagedArrayCollection Collection;

	/** Materials array to use for this asset */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "Materials"))
	TArray<TObjectPtr<UMaterial>> Materials;

	/** Material instances array from the static mesh */
	UPROPERTY(meta = (DataflowOutput, DisplayName = "MaterialInstances"))
	TArray<TObjectPtr<UMaterialInterface>> MaterialInstances;

	/** Array of instanced meshes*/
	UPROPERTY(meta = (DataflowOutput, DisplayName = "InstancedMeshes"))
	TArray<FGeometryCollectionAutoInstanceMesh> InstancedMeshes;

	FStaticMeshToCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void RegisterGeometryStaticMeshToCollectionNodes();
}
