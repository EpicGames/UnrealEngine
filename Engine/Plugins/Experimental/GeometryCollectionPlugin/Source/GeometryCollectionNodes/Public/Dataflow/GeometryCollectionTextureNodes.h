// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowSelection.h"
#include "Dataflow/DataflowImage.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "FractureAutoUV.h"
#include "Image/ImageBuilder.h"
#include "Engine/Texture2D.h"

#include "GeometryCollectionTextureNodes.generated.h"

UENUM()
enum class ECollectionBakeTextureAttribute: int32
{
	// no attribute selected
	None = (int32)UE::PlanarCut::EBakeAttributes::None,

	// Phi value : distance to the closest face 
	DistanceToExternal = (int32)UE::PlanarCut::EBakeAttributes::DistanceToExternal,

	// Ambient occlusion
	AmbientOcclusion = (int32)UE::PlanarCut::EBakeAttributes::AmbientOcclusion,

	// surface smoothed curvature
	Curvature = (int32)UE::PlanarCut::EBakeAttributes::Curvature,
	
	// Normal X coordinate ( object space ) 
	NormalX = (int32)UE::PlanarCut::EBakeAttributes::NormalX,
	
	// Normal Y coordinate ( object space ) 
	NormalY = (int32)UE::PlanarCut::EBakeAttributes::NormalY,
	
	// Normal Z coordinate ( object space ) 
	NormalZ = (int32)UE::PlanarCut::EBakeAttributes::NormalZ,

	// Position X coordinate ( relative to object local bounds ) 
	PositionX = (int32)UE::PlanarCut::EBakeAttributes::PositionX,
	
	// Position Y coordinate ( relative to object local bounds ) 
	PositionY = (int32)UE::PlanarCut::EBakeAttributes::PositionY,
	
	// Position Z coordinate ( relative to object local bounds ) 
	PositionZ = (int32)UE::PlanarCut::EBakeAttributes::PositionZ,
};

/*
* Bake a texture from a geometry collection 
* Output to a 4 channels Image object  ( RGBA )
*/
USTRUCT()
struct FBakeTextureFromCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBakeTextureFromCollectionDataflowNode, "BakeTextureFromCollection", "GeometryCollection|Texture", "")

private:
	/** Target Collection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Output image with the bake attributes */
	UPROPERTY(meta = (DataflowOutput))
	FDataflowImage Image;

	/** selection of faces to bake : if not connected, all faces will be used */
	UPROPERTY(meta = (DataflowInput))
	FDataflowFaceSelection FaceSelection;

	/** resolution of the image to bake */
	UPROPERTY(EditAnywhere, Category = ImageOptions, meta = (DataflowInput))
	EDataflowImageResolution Resolution = EDataflowImageResolution::Resolution512;

	/** Whether to pad out gutter size so that all empty texels are filled */
	UPROPERTY(EditAnywhere, Category = ChannelOptions, meta = (DataflowInput))
	bool bFullyPadGutter = false;

	/** Approximate space to fill in between UV islands, measured in texels. Ignored if Fully Pad Gutter is enabled. */
	UPROPERTY(EditAnywhere, Category = ChannelOptions, meta = (DataflowInput, EditCondition="!bFullyPadGutter", ClampMin = 1, UIMin = 1))
	int32 GutterSize = 1;

	// If true, will clear the alpha channel for gutter pixels, so they are transparent and the UV boundaries are clearer. If false, gutter pixel values will be copied as-is from the nearest interior pixel.
	UPROPERTY(EditAnywhere, Category = ChannelOptions, meta = (DataflowInput))
	bool bClearGutterAlpha = true;

	/** Index of the added UV channel */
	UPROPERTY(EditAnywhere, Category = ChannelOptions, meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "UVChannel", DataflowUVChannel, ClampMin = 0, ClampMax = 7, UIMin = 0, UIMax = 7))
	int32 UVChannel = 0;

	/** Attribute to bake in the red channel */
	UPROPERTY(EditAnywhere, Category = ChannelOptions)
	ECollectionBakeTextureAttribute RedChannel = ECollectionBakeTextureAttribute::DistanceToExternal;

	/** Attribute to bake in the green channel */
	UPROPERTY(EditAnywhere, Category = ChannelOptions)
	ECollectionBakeTextureAttribute GreenChannel = ECollectionBakeTextureAttribute::AmbientOcclusion;

	/** Attribute to bake in the blue channel */
	UPROPERTY(EditAnywhere, Category = ChannelOptions)
	ECollectionBakeTextureAttribute BlueChannel = ECollectionBakeTextureAttribute::Curvature;

	/** Attribute to bake in the alpha channel */
	UPROPERTY(EditAnywhere, Category = ChannelOptions)
	ECollectionBakeTextureAttribute AlphaChannel = ECollectionBakeTextureAttribute::None;

	/** Max distance to search for the outer mesh surface */
	UPROPERTY(EditAnywhere, Category = DistToOuterSettings, meta = (
		DataflowInput,
		EditCondition = "RedChannel == ECollectionBakeTextureAttribute::DistanceToExternal || GreenChannel == ECollectionBakeTextureAttribute::DistanceToExternal || BlueChannel == ECollectionBakeTextureAttribute::DistanceToExternal || AlphaChannel == ECollectionBakeTextureAttribute::DistanceToExternal", 
		UIMin = "1", UIMax = "100", ClampMin = ".01", ClampMax = "1000"))
	float MaxDistance = 100;

	/** Number of occlusion rays */
	UPROPERTY(EditAnywhere, Category = AmbientOcclusionSettings, meta = (
		DataflowInput,
		EditCondition = "RedChannel == ECollectionBakeTextureAttribute::AmbientOcclusion || GreenChannel == ECollectionBakeTextureAttribute::AmbientOcclusion || BlueChannel == ECollectionBakeTextureAttribute::AmbientOcclusion || AlphaChannel == ECollectionBakeTextureAttribute::AmbientOcclusion",
		UIMin = "1", UIMax = "1024", ClampMin = "0", ClampMax = "50000"))
	int32 OcclusionRays = 16;

	/** Pixel Radius of Gaussian Blur Kernel applied to AO map (0 will apply no blur) */
	UPROPERTY(EditAnywhere, Category = AmbientOcclusionSettings, meta = (
		DataflowInput,
		EditCondition = "RedChannel == ECollectionBakeTextureAttribute::AmbientOcclusion || GreenChannel == ECollectionBakeTextureAttribute::AmbientOcclusion || BlueChannel == ECollectionBakeTextureAttribute::AmbientOcclusion || AlphaChannel == ECollectionBakeTextureAttribute::AmbientOcclusion",
		UIMin = "0", UIMax = "10.0", ClampMin = "0", ClampMax = "100.0"))
	float OcclusionBlurRadius = 2.25;

	/** Pixel Radius of Gaussian Blur Kernel applied to Curvature map (0 will apply no blur) */
	UPROPERTY(EditAnywhere, Category = SmoothedCurvatureSettings, meta = (
		DataflowInput,
		EditCondition = "RedChannel == ECollectionBakeTextureAttribute::Curvature || GreenChannel == ECollectionBakeTextureAttribute::Curvature || BlueChannel == ECollectionBakeTextureAttribute::Curvature || AlphaChannel == ECollectionBakeTextureAttribute::Curvature",
		UIMin = "0", UIMax = "10.0", ClampMin = "0", ClampMax = "100.0"))
	float CurvatureBlurRadius = 2.25;

	/** Voxel resolution of smoothed shape representation */
	UPROPERTY(EditAnywhere, Category = SmoothedCurvatureSettings, meta = (
		DataflowInput,
		EditCondition = "RedChannel == ECollectionBakeTextureAttribute::Curvature || GreenChannel == ECollectionBakeTextureAttribute::Curvature || BlueChannel == ECollectionBakeTextureAttribute::Curvature || AlphaChannel == ECollectionBakeTextureAttribute::Curvature",
		UIMin = "8", UIMax = "512", ClampMin = "4", ClampMax = "1024"))
	int32 VoxelResolution = 128;

	/** Amount of smoothing iterations to apply before computing curvature */
	UPROPERTY(EditAnywhere, Category = SmoothedCurvatureSettings, meta = (
		DataflowInput,
		EditCondition = "RedChannel == ECollectionBakeTextureAttribute::Curvature || GreenChannel == ECollectionBakeTextureAttribute::Curvature || BlueChannel == ECollectionBakeTextureAttribute::Curvature || AlphaChannel == ECollectionBakeTextureAttribute::Curvature",
		UIMin = "2", UIMax = "100", ClampMin = "2", ClampMax = "1000"))
	int32 SmoothingIterations = 10;

	/** Distance to search for correspondence between fractured shape and smoothed shape, as factor of voxel size */
	UPROPERTY(EditAnywhere, Category = SmoothedCurvatureSettings, meta = (
		DataflowInput,
		EditCondition = "RedChannel == ECollectionBakeTextureAttribute::Curvature || GreenChannel == ECollectionBakeTextureAttribute::Curvature || BlueChannel == ECollectionBakeTextureAttribute::Curvature || AlphaChannel == ECollectionBakeTextureAttribute::Curvature",
		UIMin = "2", UIMax = "10.0", ClampMin = "1", ClampMax = "100.0"))
	float ThicknessFactor = 4;

	/** Curvatures in the range [-MaxCurvature, MaxCurvature] will be mapped from [0,1]. Values outside that range will be clamped */
	UPROPERTY(EditAnywhere, Category = SmoothedCurvatureSettings, meta = (
		DataflowInput,
		EditCondition = "RedChannel == ECollectionBakeTextureAttribute::Curvature || GreenChannel == ECollectionBakeTextureAttribute::Curvature || BlueChannel == ECollectionBakeTextureAttribute::Curvature || AlphaChannel == ECollectionBakeTextureAttribute::Curvature",
		UIMin = ".01", UIMax = "1", ClampMin = ".0001", ClampMax = "10"))
	float MaxCurvature = .1f;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FBakeTextureFromCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

namespace UE::Dataflow
{
	void RegisterGeometryCollectionTextureNodes();
}

