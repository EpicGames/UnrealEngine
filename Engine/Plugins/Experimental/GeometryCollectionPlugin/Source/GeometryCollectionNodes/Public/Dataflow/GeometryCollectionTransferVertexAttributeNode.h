// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Chaos/BoundingVolumeHierarchy.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowConnectionTypes.h"
#include "GeometryCollection/GeometryCollection.h"
#include "Dataflow/DataflowSelection.h"

#include "GeometryCollectionTransferVertexAttributeNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

UENUM(BlueprintType)
enum class EDataflowTransferVertexAttributeNodeFalloff : uint8
{
	/** Squared falloff based on distance from triangle*/
	Squared  UMETA(DisplayName = "Squared"),

	/** Linear falloff based on distance from triangle*/
	Linear UMETA(DisplayName = "Linear"),

	/** No distance falloff */
	None UMETA(DisplayName = "None"),
	//~~~
	//256th entry
	Dataflow_Max UMETA(Hidden)
};

UENUM(BlueprintType)
enum class EDataflowTransferVertexAttributeNodeSourceScale : uint8
{
	/** Bounding volume hierarchy cell size based on max edge length of each geometry group. 
	Only works if there is 1-1 correspondence between source and target geometries, otherwise default to Asset Max Edge*/
	Component_Edge  UMETA(DisplayName = "Component Max Edge"),

	/** Bounding volume hierarchy cell size based on max edge length of the whole asset*/
	Asset_Edge UMETA(DisplayName = "Asset Max Edge"),

	/** Bounding volume hierarchy cell size based on max length of the bounding box of the whole asset*/
	Asset_Bound UMETA(DisplayName = "Asset Max Bound"),

	//~~~
	//256th entry
	Dataflow_Max UMETA(Hidden)
};

UENUM(BlueprintType)
enum class EDataflowTransferVertexAttributeNodeBoundingVolume : uint8
{
	/** Bounding volume on vertices of the source mesh*/
	Vertex UMETA(DisplayName = "Vertex"),

	/** Bounding volume on triangles of the source triangle mesh*/
	Triangle UMETA(DisplayName = "Triangle"),

	//~~~
	//256th entry
	Dataflow_Max UMETA(Hidden)
};

UENUM(BlueprintType)
enum class EDataflowTransferVertexAttributeNodeTransferMethod : uint8
{
	/** Transfers vertex attribute between matched geometries.*/
	Component  UMETA(DisplayName = "Paired Geometry Transfer"),

	/** Transfers vertex attribute globally. Warning: attributes on one geometry might bleed over to neighbor geometries.*/
	Global UMETA(DisplayName = "Global Transfer"),

	/** No transfer */
	None UMETA(DisplayName = "None"),
	//~~~
	//256th entry
	Dataflow_Max UMETA(Hidden)
};

/**
 * Transfer float properties from a source collection to a target collection.
 * Component Transfer is used when all geometries from the source collection have matched names with the target collection.
 * Otherwise, Global Transfer is used.
 * Geometries are matched when the geometry's BoneName can be found as the start of the BoneName of a geometry in the target collection.
 * Use TransformNameSuffix to add extra string to the source geometry's BoneName to avoid multiple matched names.
 * For example, source geometry has name SK_10 and target geometry has name SK_10_tet1
 * For all names, Check BoneName attribute in Transform group in the collection.
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGeometryCollectionTransferVertexAttributeNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGeometryCollectionTransferVertexAttributeNode, "TransferVertexAttribute", "GeometryCollection", "Transfer a named vertex attribute from the Source Collection to the Target Collection")
	DATAFLOW_NODE_RENDER_TYPE("SurfaceWeightsRender", FGeometryCollection::StaticType(), "Collection", "AttributeKey")

public:

	/* Target collection to transfer vertex attribute to. */
	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/* Source collection to transfer vertex attribute from. */
	UPROPERTY(Meta = (DataflowInput, DisplayName = "FromCollection"))
	FManagedArrayCollection FromCollection;

	/* The name of the vertex attribute to generate indices from. */
	UPROPERTY(EditAnywhere, Category = "Dataflow", Meta = (DataflowInput, DataflowOutput, DisplayName = "AttributeKey", DataflowPassthrough = "AttributeKey"))
	FCollectionAttributeKey AttributeKey = FCollectionAttributeKey(FString(""), FString("Vertices"));
	
	/* Transfer method [default: Paired Geometry Transfer] */
	UPROPERTY(EditAnywhere, Category = "Method")
	EDataflowTransferVertexAttributeNodeTransferMethod TransferMethod = EDataflowTransferVertexAttributeNodeTransferMethod::Global;

	/* Bounding volume type for source assets[default: Triangle] */
	UPROPERTY(EditAnywhere, Category = "Thresholds")
	EDataflowTransferVertexAttributeNodeBoundingVolume BoundingVolumeType = EDataflowTransferVertexAttributeNodeBoundingVolume::Triangle;

	/* Bounding volume hierarchy cell size for neighboring vertices to transfer into[default: Asset] */
	UPROPERTY(EditAnywhere, Category = "Thresholds")
	EDataflowTransferVertexAttributeNodeSourceScale SourceScale = EDataflowTransferVertexAttributeNodeSourceScale::Asset_Bound;

	/* Falloff of source value based on distance from source triangle[default: Squared] */
	UPROPERTY(EditAnywhere, Category = "Thresholds", meta = (EditCondition = "BoundingVolumeType == EDataflowTransferVertexAttributeNodeBoundingVolume::Triangle", EditConditionHides))
	EDataflowTransferVertexAttributeNodeFalloff Falloff = EDataflowTransferVertexAttributeNodeFalloff::None;

	/* Threshold based on distance from source triangle.Values past the threshold will falloff.[Defaults to 1 percent of triangle size(0.01)] */
	UPROPERTY(EditAnywhere, Category = "Thresholds", meta = (EditCondition = 
		"Falloff != EDataflowTransferVertexAttributeNodeFalloff::None"
		, EditConditionHides))
	float FalloffThreshold = 0.01f;

	/* Edge multiplier for the Bounding Volume Hierarchy(BVH) target's particle search radius. */
	UPROPERTY(EditAnywhere, Category = "Thresholds", meta = (EditCondition = "SourceScale == EDataflowTransferVertexAttributeNodeSourceScale::Asset_Edge || SourceScale == EDataflowTransferVertexAttributeNodeSourceScale::Component_Edge", EditConditionHides))
	float EdgeMultiplier = 0.5f;

	/* Max bound multiplier for the Bounding Volume Hierarchy(BVH) target's particle search radius. */
	UPROPERTY(EditAnywhere, Category = "Thresholds", meta = (
		EditCondition = "SourceScale == EDataflowTransferVertexAttributeNodeSourceScale::Asset_Bound",
		EditConditionHides))
	float BoundMultiplier = 0.01f;

	/* Suffix of transform names added to the source geometry's BoneName for geometry matching during transfer[default: _Tet]. In CreateTetrahedron node we add _Tet to tetrahedral geometries.*/
	UPROPERTY(EditAnywhere, Category = "Paired Geometry Transfer", meta = (
		EditCondition = "TransferMethod == EDataflowTransferVertexAttributeNodeTransferMethod::Component",
		EditConditionHides))
	FString TransformNameSuffix = "_Tet";

	FGeometryCollectionTransferVertexAttributeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&FromCollection);
		RegisterInputConnection(&AttributeKey);

		RegisterOutputConnection(&Collection, &Collection);
		RegisterOutputConnection(&AttributeKey, &AttributeKey);
	}

private:

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Transfer skin weights from a source collection to a target collection.
 * Component Transfer is used when all geometries from the source collection have matched names with the target collection.
 * Otherwise, Global Transfer is used.
 * Geometries are matched when the geometry's BoneName can be found as the start of the BoneName of a geometry in the target collection.
 * Use TransformNameSuffix to add extra string to the source geometry's BoneName to avoid multiple matched names.
 * For example, source geometry has name SK_10 and target geometry has name SK_10_tet1
 * For all names, Check BoneName attribute in Transform group in the collection.
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGeometryCollectionTransferVertexSkinWeightsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGeometryCollectionTransferVertexSkinWeightsNode, "TransferVertexSkinWeights", "GeometryCollection", "Transfer vertex skin weights from the Source Collection to the Target Collection")

public:

	/* Target collection to transfer vertex attribute to. */
	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/* Source collection to transfer vertex attribute from. */
	UPROPERTY(Meta = (DataflowInput, DisplayName = "FromCollection"))
	FManagedArrayCollection FromCollection;

	/* Transfer method [default: Paired Geometry Transfer] */
	UPROPERTY(EditAnywhere, Category = "Method")
	EDataflowTransferVertexAttributeNodeTransferMethod TransferMethod = EDataflowTransferVertexAttributeNodeTransferMethod::Global;

	/* Bounding volume type for source assets[default: Triangle] */
	UPROPERTY(EditAnywhere, Category = "Thresholds")
	EDataflowTransferVertexAttributeNodeBoundingVolume BoundingVolumeType = EDataflowTransferVertexAttributeNodeBoundingVolume::Triangle;

	/* Bounding volume hierarchy cell size for neighboring vertices to transfer into[default: Asset] */
	UPROPERTY(EditAnywhere, Category = "Thresholds")
	EDataflowTransferVertexAttributeNodeSourceScale SourceScale = EDataflowTransferVertexAttributeNodeSourceScale::Asset_Bound;

	/* Falloff of source value based on distance from source triangle[default: Squared] */
	UPROPERTY(EditAnywhere, Category = "Thresholds", meta = (EditCondition = "BoundingVolumeType == EDataflowTransferVertexAttributeNodeBoundingVolume::Triangle", EditConditionHides))
	EDataflowTransferVertexAttributeNodeFalloff Falloff = EDataflowTransferVertexAttributeNodeFalloff::None;

	/* Threshold based on distance from source triangle.Values past the threshold will falloff.[Defaults to 1 percent of triangle size(0.01)] */
	UPROPERTY(EditAnywhere, Category = "Thresholds", meta = (EditCondition = "Falloff != EDataflowTransferVertexAttributeNodeFalloff::None", EditConditionHides))
	float FalloffThreshold = 0.01f;

	/* Edge multiplier for the Bounding Volume Hierarchy(BVH) target's particle search radius. */
	UPROPERTY(EditAnywhere, Category = "Thresholds", meta = (EditCondition = "SourceScale == EDataflowTransferVertexAttributeNodeSourceScale::Asset_Edge || SourceScale == EDataflowTransferVertexAttributeNodeSourceScale::Component_Edge", EditConditionHides))
	float EdgeMultiplier = 0.5f;

	/* Max bound multiplier for the Bounding Volume Hierarchy(BVH) target's particle search radius. */
	UPROPERTY(EditAnywhere, Category = "Thresholds", meta = (
		EditCondition = "SourceScale == EDataflowTransferVertexAttributeNodeSourceScale::Asset_Bound",
		EditConditionHides))
	float BoundMultiplier = 0.01f;

	/* Suffix of transform names for geometry matching during transfer[default: _Tet]. In CreateTetrahedron node we add _Tet to tetrahedral geometries.*/
	UPROPERTY(EditAnywhere, Category = "Paired Geometry Transfer", meta = (
		EditCondition = "TransferMethod == EDataflowTransferVertexAttributeNodeTransferMethod::Component",
		EditConditionHides))
	FString TransformNameSuffix = "_Tet";

	FGeometryCollectionTransferVertexSkinWeightsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&FromCollection);
		RegisterOutputConnection(&Collection, &Collection);
	}

private:

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

UENUM(BlueprintType)
enum class ESetKinematicVertexSelectionKinematicValue : uint8
{
	/** Set vertices to be kinematic */
	SetKinematic     UMETA(DisplayName = "Kinematic"),

	/** Set vertices to be dynamic */
	SetNonKinematic  UMETA(DisplayName = "Non-Kinematic")
};

/**
 * Set VertexSelection to be kinematic. Note that kinematic particles need skin weights.
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGeometryCollectionSetKinematicVertexSelectionNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGeometryCollectionSetKinematicVertexSelectionNode, "SetKinematicVertexSelection", "GeometryCollection", "Set Vertex Collection to be kinematic")

public:

	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Vertex Selection set to be kinematic */
	UPROPERTY(meta = (DataflowInput, DisplayName = "VertexSelection"))
	FDataflowVertexSelection VertexSelection;

	UPROPERTY(EditAnywhere, Category = "Method", meta = (DisplayName = "VertexSelection"))
	ESetKinematicVertexSelectionKinematicValue KinematicValue = ESetKinematicVertexSelectionKinematicValue::SetKinematic;

	FGeometryCollectionSetKinematicVertexSelectionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&VertexSelection);
		RegisterOutputConnection(&Collection, &Collection);
	}

private:

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};