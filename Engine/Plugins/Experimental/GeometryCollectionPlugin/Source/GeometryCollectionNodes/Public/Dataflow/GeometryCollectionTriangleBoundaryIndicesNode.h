// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowCore.h"
#include "Dataflow/DataflowEngine.h"

#include "GeometryCollectionTriangleBoundaryIndicesNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

//Outputs boundary nodes of a triangle mesh
USTRUCT(meta = (DataflowFlesh, Experimental))
struct FTriangleBoundaryIndicesNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FTriangleBoundaryIndicesNode, "TriangleBoundaryIndices", "Flesh|Experimental", "")

private:
	typedef FManagedArrayCollection DataType;

	UPROPERTY(meta = (DataflowInput, DisplayName = "Collection", DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowOutput, DisplayName = "BoundaryIndices"))
	TArray<int32> BoundaryIndicesOut;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FTriangleBoundaryIndicesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};