// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowSelection.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "GeometryCollectionUVNodes.generated.h"

/*
* Add a new UV channel to the collection
* note that there's a maximum of 8 channels that can be handled by a collection
*/
USTRUCT()
struct FAddUVChannelDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAddUVChannelDataflowNode, "AddUVChannel", "GeometryCollection|UV", "")

private:
	/** Target Collection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Index of the added UV channel */
	UPROPERTY(meta = (DataflowOutput, DataflowUVChannel))
	int32 UVChannel = 0;

	/** Value to initialize the UV with */
	UPROPERTY(EditAnywhere, Category = Options)
	FVector2f DefaultValue = { 0, 0 };

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FAddUVChannelDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Auto unwrap UVs for a specific UV channel
 */
USTRUCT()
struct FAutoUnwrapUVDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FAutoUnwrapUVDataflowNode, "AutoUnwrapUV", "GeometryCollection|UV", "")

private:
	/** Target Collection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Faces to auto unwrap, no selection means all faces */
	UPROPERTY(meta = (DataflowInput))
	FDataflowFaceSelection FaceSelection;

	/** UV channel to unwrap into ( 0 by default ) */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "UVChannel", DataflowUVChannel, ClampMin = 0, ClampMax = 7, UIMin = 0, UIMax = 7))
	int32 UVChannel = 0;

	/** Approximate space to leave between UV islands, measured in texels for 512x512 texture */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 1, UIMin = 1))
	int32 GutterSize = 1;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FAutoUnwrapUVDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Merge adjacent UV Islands with similar normals for a specific UV channel
 */
USTRUCT()
struct FMergeUVIslandsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMergeUVIslandsDataflowNode, "MergeUVIslands", "GeometryCollection|UV", "")

private:
	/** Target Collection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Faces to auto unwrap, no selection means all faces */
	UPROPERTY(meta = (DataflowInput))
	FDataflowFaceSelection FaceSelection;

	/** UV channel to unwrap into ( 0 by default ) */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "UVChannel", DataflowUVChannel, ClampMin = 0, ClampMax = 7, UIMin = 0, UIMax = 7))
	int32 UVChannel = 0;

	// Threshold for allowed area distortion from merging islands (when we use ExpMap to compute new UVs for the merged island)
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 0, ClampMax = 10, UIMin = 0, UIMax = 10))
	double AreaDistortionThreshold = 1.5;
	// Threshold for allowed normal deviation between merge-able islands
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 0, ClampMax = 180, UIMin = 0, UIMax = 90))
	double MaxNormalDeviationDeg = 45.0;

	// Amount of normal smoothing to apply when computing new UVs for merged islands. More smoothing will result in UV maps that are less sensitive to local surface shape.
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (DataflowInput, ClampMin = 0, ClampMax = 500, UIMin = 0, UIMax = 100))
	int32 NormalSmoothingRounds = 0;
	// Strength of normal smoothing to apply when computing new UVs for merged islands. Stronger smoothing will result in UV maps that are less sensitive to local surface shape.
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (DataflowInput, ClampMin = 0, ClampMax = 1, UIMin = 0, UIMax = 1))
	double NormalSmoothingAlpha = 0.25;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FMergeUVIslandsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * Generates UVs using a box projection
 */
USTRUCT()
struct FBoxProjectUVDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FBoxProjectUVDataflowNode, "BoxProjectUV", "GeometryCollection|UV", "")

private:
	/** Target Collection */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Faces to auto unwrap, no selection means all faces */
	//UPROPERTY(meta = (DataflowInput))
	//FDataflowFaceSelection FaceSelection;

	/** UV channel to unwrap into ( 0 by default ) */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "UVChannel", DataflowUVChannel, ClampMin = 0, ClampMax = 7, UIMin = 0, UIMax = 7))
	int32 UVChannel = 0;

	/** Approximate space to leave between UV islands, measured in texels for 512x512 texture */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput, ClampMin = 1, UIMin = 1))
	int32 GutterSize = 1;

	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	FVector ProjectionScale = { 100, 100, 100 };

	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	FVector2f UVOffset = { 0.5, 0.5 };

	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	bool bAutoFitToBounds = false;

	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	bool bCenterBoxAtPivot = false;

	UPROPERTY(EditAnywhere, Category = Options, meta = (DataflowInput))
	bool bUniformProjectionScale = false;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

public:
	FBoxProjectUVDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

namespace UE::Dataflow
{
	void RegisterGeometryCollectionUVNodes();
}

