// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowSelection.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "CompGeom/ConvexDecomposition3.h"
#include "Dataflow/DataflowDebugDraw.h"

#include "FractureEngineConvex.h"
#include "FractureEngineUtility.h"

#include "GeometryCollectionUtilityNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

UENUM(BlueprintType)
enum class EConvexOverlapRemovalMethodEnum : uint8
{
	Dataflow_EConvexOverlapRemovalMethod_None UMETA(DisplayName = "None"),
	Dataflow_EConvexOverlapRemovalMethod_All UMETA(DisplayName = "All"),
	Dataflow_EConvexOverlapRemovalMethod_OnlyClusters UMETA(DisplayName = "Only Clusters"),
	Dataflow_EConvexOverlapRemovalMethod_OnlyClustersVsClusters UMETA(DisplayName = "Only Clusters vs Clusters"),
	//~~~
	//256th entry
	Dataflow_Max                UMETA(Hidden)
};

//~ Simple wrapper class to make the sphere covering data possible to pass via dataflow
// A set of spheres generated to represent empty space when creating a minimal set of convex hulls, e.g. in one of the Generate Cluster Convex Hulls nodes
USTRUCT()
struct FDataflowSphereCovering
{
	GENERATED_USTRUCT_BODY()

public:
	UE::Geometry::FSphereCovering Spheres;
};


USTRUCT()
struct FDataflowConvexDecompositionSettings
{
	GENERATED_USTRUCT_BODY()

public:
	// If greater than zero, the minimum geometry size (cube root of volume) to consider for convex decomposition
	UPROPERTY(EditAnywhere, Category = Filter, meta = (ClampMin = 0.0))
	float MinSizeToDecompose = 0.f;

	// If the geo volume / hull volume ratio is greater than this, do not consider convex decomposition
	UPROPERTY(EditAnywhere, Category = Filter, meta = (ClampMin = 0.0, ClampMax = 1.0))
	float MaxGeoToHullVolumeRatioToDecompose = 1.f;

	// Stop splitting when hulls have error less than this (expressed in cm; will be cubed for volumetric error).
	// Note: Decomposition will only be performed if: bProtectNegativeSpace is true, ErrorTolerance is > 0, or MaxHullsPerGeometry > 1
	UPROPERTY(EditAnywhere, Category = Decomposition, meta = (ClampMin = 0.0, Units = cm, EditCondition = "!bProtectNegativeSpace"))
	float ErrorTolerance = 0.f;

	// If greater than zero, maximum number of convex hulls to use in each convex decomposition.
	// Note: Decomposition will only be performed if: bProtectNegativeSpace is true, ErrorTolerance is > 0, or MaxHullsPerGeometry > 1
	UPROPERTY(EditAnywhere, Category = Decomposition, meta = (ClampMin = -1))
	int32 MaxHullsPerGeometry = -1;

	// Optionally specify a minimum thickness (in cm) for convex parts; parts below this thickness will always be merged away. Overrides NumOutputHulls and ErrorTolerance when needed.
	UPROPERTY(EditAnywhere, Category = Decomposition, meta = (ClampMin = 0.0))
	float MinThicknessTolerance = 0.f;

	// Control the search effort spent per convex decomposition: larger values will require more computation but may find better convex decompositions
	UPROPERTY(EditAnywhere, Category = Decomposition, meta = (ClampMin = 0, EditCondition = "!bProtectNegativeSpace"))
	int32 NumAdditionalSplits = 4;

	/** Whether to drive decomposition by finding a negative space that should not be covered by convex hulls. If enabled, ErrorTolerance and NumAdditionalSplits will not be used. */
	UPROPERTY(EditAnywhere, Category = NegativeSpace)
	bool bProtectNegativeSpace = false;

	/** When protecting negative space, only look for space that is connected out to the convex hull. This removes inaccessable internal negative space from consideration. */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (EditCondition = "bProtectNegativeSpace"))
	bool bOnlyConnectedToHull = true;

	/** Amount of space to leave between convex hulls and protected negative space */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (ClampMin = .01, UIMin = .1, Units = cm, EditCondition = "bProtectNegativeSpace"))
	float NegativeSpaceTolerance = 2.0;

	/** Spheres smaller than this are not included in the negative space */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (ClampMin = 0, Units = cm, EditCondition = "bProtectNegativeSpace"))
	float NegativeSpaceMinRadius = 10.0;
};


//~ TODO: Ideally this would be generated from the above FDataflowConvexDecompositionSettings struct
// Provide settings for running convex decomposition of geometry
USTRUCT(meta = (DataflowGeometryCollection))
struct FMakeDataflowConvexDecompositionSettingsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMakeDataflowConvexDecompositionSettingsNode, "MakeConvexDecompositionSettings", "GeometryCollection|Utilities", "")

public:
	// If greater than zero, the minimum geometry size (cube root of volume) to consider for convex decomposition
	UPROPERTY(EditAnywhere, Category = Filter, meta = (ClampMin = 0.0, DataflowInput))
	float MinSizeToDecompose = 0.f;
	
	// If the geo volume / hull volume ratio is greater than this, do not consider convex decomposition
	UPROPERTY(EditAnywhere, Category = Filter, meta = (ClampMin = 0.0, ClampMax = 1.0, DataflowInput))
	float MaxGeoToHullVolumeRatioToDecompose = 1.f;

	// Stop splitting when hulls have error less than this (expressed in cm; will be cubed for volumetric error).
	// Note: Decomposition will only be performed if: bProtectNegativeSpace is true, ErrorTolerance is > 0, or MaxHullsPerGeometry > 1
	UPROPERTY(EditAnywhere, Category = Decomposition, meta = (ClampMin = 0.0, Units = cm, DataflowInput, EditCondition = "!bProtectNegativeSpace"))
	float ErrorTolerance = 0.f;

	// If greater than zero, maximum number of convex hulls to use in each convex decomposition.
	// Note: Decomposition will only be performed if: bProtectNegativeSpace is true, ErrorTolerance is > 0, or MaxHullsPerGeometry > 1
	UPROPERTY(EditAnywhere, Category = Decomposition, meta = (ClampMin = -1, DataflowInput))
	int32 MaxHullsPerGeometry = -1;

	// Optionally specify a minimum thickness (in cm) for convex parts; parts below this thickness will always be merged away. Overrides NumOutputHulls and ErrorTolerance when needed.
	UPROPERTY(EditAnywhere, Category = Decomposition, meta = (ClampMin = 0.0, DataflowInput))
	float MinThicknessTolerance = 0.f;

	// Control the search effort spent per convex decomposition: larger values will require more computation but may find better convex decompositions
	UPROPERTY(EditAnywhere, Category = Decomposition, meta = (ClampMin = 0, DataflowInput, EditCondition = "!bProtectNegativeSpace"))
	int32 NumAdditionalSplits = 4;

	/** Whether to drive decomposition by finding a negative space that should not be covered by convex hulls. If enabled, ErrorTolerance and NumAdditionalSplits will not be used. */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput))
	bool bProtectNegativeSpace = false;

	/** When protecting negative space, only look for space that is connected out to the convex hull. This removes inaccessable internal negative space from consideration. */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, EditCondition = "bProtectNegativeSpace"))
	bool bOnlyConnectedToHull = true;

	/** Amount of space to leave between convex hulls and protected negative space */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = .01, UIMin = .1, Units = cm, EditCondition = "bProtectNegativeSpace"))
	float NegativeSpaceTolerance = 2.0;

	/** Spheres smaller than this are not included in the negative space */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = 0, Units = cm, EditCondition = "bProtectNegativeSpace"))
	float NegativeSpaceMinRadius = 10.0;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowConvexDecompositionSettings DecompositionSettings;

	FMakeDataflowConvexDecompositionSettingsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

USTRUCT(meta = (DataflowGeometryCollection))
struct FCreateLeafConvexHullsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCreateLeafConvexHullsDataflowNode, "CreateLeafConvexHulls", "GeometryCollection|Utilities", "")

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowSphereCovering SphereCovering;

	/** Optional transform selection to compute leaf hulls on -- if not provided, all leaf hulls will be computed. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection OptionalSelectionFilter;

	/** How convex hulls are generated -- computed from geometry, imported from external collision shapes, or an intersection of both options. */
	UPROPERTY(EditAnywhere, Category = Options)
	EGenerateConvexMethod GenerateMethod = EGenerateConvexMethod::ComputedFromGeometry;

	/** If GenerateMethod is Intersect, only actually intersect when the volume of the Computed Hull is less than this fraction of the volume of the External Hull(s). */
	UPROPERTY(EditAnywhere, Category = IntersectionFilters, meta = (ClampMin = 0.0, ClampMax = 1.0, EditCondition = "GenerateMethod == EGenerateConvexMethod::IntersectExternalWithComputed"))
	float IntersectIfComputedIsSmallerByFactor = 1.0f;

	/** If GenerateMethod is Intersect, only actually intersect if the volume of the External Hull(s) exceed this threshold. */
	UPROPERTY(EditAnywhere, Category = IntersectionFilters, meta = (ClampMin = 0.0, EditCondition = "GenerateMethod == EGenerateConvexMethod::IntersectExternalWithComputed"))
	float MinExternalVolumeToIntersect = 0.0f;

	/** Whether to compute the intersection before computing convex hulls. Typically should be enabled. */
	UPROPERTY(EditAnywhere, Category = Convex, meta = (EditCondition = "GenerateMethod == EGenerateConvexMethod::IntersectExternalWithComputed"))
	bool bComputeIntersectionsBeforeHull = true;

	/** Computed convex hulls are simplified to keep points spaced at least this far apart (except where needed to keep the hull from collapsing to zero volume). */
	UPROPERTY(EditAnywhere, Category = Convex, meta = (DataflowInput, ClampMin = 0.f, EditCondition = "GenerateMethod != EGenerateConvexMethod::ExternalCollision"))
	float SimplificationDistanceThreshold = 10.f;

	UPROPERTY(EditAnywhere, Category = Convex, meta = (DataflowInput))
	FDataflowConvexDecompositionSettings ConvexDecompositionSettings;

	/** Negative space that must be used. Applies in addition to any automatically-detected covering added via ConvexDecompositionSettings. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowSphereCovering RequireNegativeSpaceCovering;

	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ShowOnlyInnerProperties))
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	/** Randomize color per convex hull */
	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	bool bRandomizeColor = true;

	/** Random seed */
	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ClampMin = "0", EditCondition = "bRandomizeColor==true", EditConditionHides))
	int32 ColorRandomSeed = 0;

	UPROPERTY(EditAnywhere, Category = "Debug Draw - Sphere Covering", meta = (ShowOnlyInnerProperties))
	FDataflowNodeSphereCoveringDebugDrawSettings SphereCoveringDebugDrawRenderSettings;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override
	{
		return true;
	}
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif

public:
	FCreateLeafConvexHullsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

UENUM()
enum class EPrimitiveCollisionShapeDataflowEnum : uint8
{
	Sphere,
	Box,
	Capsule,
};

/**
 * Sets the ExternalCollisions attribute on each selected transform to a primitive shape
 * (sphere / box / capsule) auto-fitted to that bone's geometry bounds. Any existing
 * ExternalCollisions entry on the selected transforms is REPLACED.
 *
 * Shapes written here flow through the same runtime path used by static-mesh-imported
 * primitive collisions: the GeometryCollection asset must have bImportCollisionFromSource
 * enabled for the simulation to consume them.
 *
 * Empty/unconnected selection authors the shape on all rigid-leaf transforms.
 */
USTRUCT(meta = (DataflowGeometryCollection, Experimental))
struct FSetExternalCollisionsFromPrimitiveDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetExternalCollisionsFromPrimitiveDataflowNode, "SetExternalCollisionsFromPrimitive", "GeometryCollection|Utilities", "")

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Transforms to author the shape on. If unconnected or empty, all rigid-leaf transforms are used. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection OptionalSelectionFilter;

	/** Which primitive shape to author. Size is fitted automatically to each bone's geometry bounds. */
	UPROPERTY(EditAnywhere, Category = "Shape")
	EPrimitiveCollisionShapeDataflowEnum Shape = EPrimitiveCollisionShapeDataflowEnum::Box;

	/** Uniform multiplier applied to the auto-fitted shape extents. 1.0 = exact AABB fit; >1 pads out, <1 shrinks. */
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (DataflowInput, ClampMin = "0.01"))
	float Scale = 1.0f;

	/** For Box and Capsule: orient the shape along the bone geometry's principal axes (PCA over vertices) for a tighter fit. Off = axis-aligned (AABB) fit. No effect on Sphere (rotation-invariant). */
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (
		EditCondition = "Shape != EPrimitiveCollisionShapeDataflowEnum::Sphere", EditConditionHides))
	bool bAlignToPrincipalAxis = true;

	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ShowOnlyInnerProperties))
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

#if WITH_EDITOR
	virtual bool CanDebugDraw() const override
	{
		return true;
	}
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif

	void GetBoneSelection(UE::Dataflow::FContext& Context, const FManagedArrayCollection& InCollection, TArray<int32>& OutSelection) const;

public:
	FSetExternalCollisionsFromPrimitiveDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

USTRUCT(meta = (DataflowGeometryCollection))
struct FSimplifyConvexHullsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSimplifyConvexHullsDataflowNode, "SimplifyConvexHulls", "GeometryCollection|Utilities", "")

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput))
	FManagedArrayCollection Collection;

	/** Optional transform selection to compute leaf hulls on -- if not provided, all leaf hulls will be computed. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection OptionalSelectionFilter;

	UPROPERTY(EditAnywhere, Category = "Convex")
	EConvexHullSimplifyMethod SimplifyMethod = EConvexHullSimplifyMethod::MeshQSlim;

	/** Simplified hull should preserve angles larger than this (in degrees).  Used by the AngleTolerance simplification method. */
	UPROPERTY(EditAnywhere, Category = "Convex", meta = (DataflowInput, ClampMin = 0.f, EditCondition = "SimplifyMethod == EConvexHullSimplifyMethod::AngleTolerance"))
	float SimplificationAngleThreshold = 10.f;

	/** Simplified hull should stay within this distance of the initial convex hull. Used by the MeshQSlim simplification method. */
	UPROPERTY(EditAnywhere, Category = "Convex", meta = (DataflowInput, ClampMin = 0.f, EditCondition = "SimplifyMethod == EConvexHullSimplifyMethod::MeshQSlim"))
	float SimplificationDistanceThreshold = 10.f;

	/** The minimum number of faces to use for the convex hull. For MeshQSlim simplification, this is a triangle count, which may be further reduced on conversion back to a convex hull. */
	UPROPERTY(EditAnywhere, Category = "Convex", meta = (DisplayName = "Min Target Face Count", DataflowInput, ClampMin = 4))
	int32 MinTargetTriangleCount = 12;

	/** Whether to restrict the simplified hulls to only use vertices from the original hulls. */
	UPROPERTY(EditAnywhere, Category = "Convex")
	bool bUseExistingVertices = false;

	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ShowOnlyInnerProperties))
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	/** Randomize color per convex hull */
	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	bool bRandomizeColor = true;

	/** Random seed */
	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ClampMin = "0" , EditCondition = "bRandomizeColor==true", EditConditionHides))
	int32 ColorRandomSeed = 0;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override
	{
		return true;
	}
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif
public:
	FSimplifyConvexHullsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Generates convex hull representation for the bones for simulation
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCreateNonOverlappingConvexHullsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCreateNonOverlappingConvexHullsDataflowNode, "CreateNonOverlappingConvexHulls", "GeometryCollection|Utilities", "")

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput))
	FManagedArrayCollection Collection;

	/** Fraction (of geometry volume) by which a cluster's convex hull volume can exceed the actual geometry volume before instead using the hulls of the children.  0 means the convex volume cannot exceed the geometry volume; 1 means the convex volume is allowed to be 100% larger (2x) the geometry volume. */
	UPROPERTY(EditAnywhere, Category = Convex, meta = (DataflowInput, DisplayName = "Allow Larger Hull Fraction", ClampMin = 0.f))
	float CanExceedFraction = .5f;

	/** Computed convex hulls are simplified to keep points spaced at least this far apart (except where needed to keep the hull from collapsing to zero volume) */
	UPROPERTY(EditAnywhere, Category = Convex, meta = (DataflowInput, ClampMin = 0.f))
	float SimplificationDistanceThreshold = 10.f;

	/** Whether and in what cases to automatically cut away overlapping parts of the convex hulls, to avoid the simulation 'popping' to fix the overlaps */
	UPROPERTY(EditAnywhere, Category = AutomaticOverlapRemoval, meta = (DisplayName = "Remove Overlaps"))
	EConvexOverlapRemovalMethodEnum OverlapRemovalMethod = EConvexOverlapRemovalMethodEnum::Dataflow_EConvexOverlapRemovalMethod_All;

	/** Overlap removal will be computed as if convex hulls were this percentage smaller (in range 0-100) */
	UPROPERTY(EditAnywhere, Category = AutomaticOverlapRemoval, meta = (DataflowInput, ClampMin = 0.f, ClampMax = 99.9f))
	float OverlapRemovalShrinkPercent = 0.f;

	/** Fraction of the convex hulls for a cluster that we can remove before using the hulls of the children */
	UPROPERTY(EditAnywhere, Category = AutomaticOverlapRemoval, meta = (DataflowInput, DisplayName = "Max Removal Fraction", ClampMin = 0.01f, ClampMax = 1.f))
	float CanRemoveFraction = 0.3f;

	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ShowOnlyInnerProperties))
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	/** Randomize color per convex hull */
	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	bool bRandomizeColor = true;

	/** Random seed */
	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ClampMin = "0", EditCondition = "bRandomizeColor==true", EditConditionHides))
	int32 ColorRandomSeed = 0;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override
	{
		return true;
	}
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif
public:
	FCreateNonOverlappingConvexHullsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

//~ Dataflow-specific copy of the negative space sampling method enum in ConvexDecomposition3.h,
//~ so that it can be exposed as a UENUM
// Method to distribute sampling spheres
UENUM()
enum class ENegativeSpaceSampleMethodDataflowEnum : uint8
{
	// Place sample spheres in a uniform grid pattern
	Uniform,
	// Use voxel-based subtraction and offsetting methods to specifically target concavities
	VoxelSearch,
	// Use a variant of VoxelSearch that aims to limit negative space to the space that can be accessed by a ball of radius >= MinRadius
	NavigableVoxelSearch
};

/**
 *
 * Generates a negative space sphere covering for a given geometry collection or subset of bones in the collection
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCreateNegativeSpaceSphereCovering : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCreateNegativeSpaceSphereCovering, "CreateNegativeSpaceSphereCovering", "GeometryCollection|Utilities", "")

private:
	UPROPERTY(meta = (DataflowInput))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowOutput))
	FDataflowSphereCovering SphereCovering;

	/** Optional transform selection -- if not provided, covering for all leaf geometry will be computed. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection OptionalSelectionFilter;


	/** Method to use to find and sample negative space */
	UPROPERTY(EditAnywhere, Category = NegativeSpace)
	ENegativeSpaceSampleMethodDataflowEnum SampleMethod = ENegativeSpaceSampleMethodDataflowEnum::NavigableVoxelSearch;

	/** Whether to require that all candidate locations identified by Voxel Search are covered by negative space samples, up to the specified Min Sample Spacing. Only applies to Voxel Search. */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (EditCondition = "SampleMethod != ENegativeSpaceSampleMethodDataflowEnum::Uniform"))
	bool bRequireSearchSampleCoverage = true;

	/** When performing Voxel Search, only look for negative space that is connected out to the convex hull. This removes inaccessable internal negative space from consideration. Only applies to Voxel Search. */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (EditCondition = "SampleMethod != ENegativeSpaceSampleMethodDataflowEnum::Uniform"))
	bool bOnlyConnectedToHull = true;

	/** Approximate number of spheres to consider when covering negative space. Only applicable with the Uniform Sample Method or if Require Search Sample Coverage is disabled. */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = 1, EditCondition = "SampleMethod == ENegativeSpaceSampleMethodDataflowEnum::Uniform || !bRequireSearchSampleCoverage"))
	int32 TargetNumSamples = 50;

	/** Minimum desired spacing between spheres; if > 0, will attempt not to place sphere centers closer than this */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = 0, Units = cm))
	double MinSampleSpacing = 0.0;

	/** Amount of space to leave between input geometry and protected negative space */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = .01, UIMin = .1, Units = cm))
	double NegativeSpaceTolerance = 2.0;

	/** Spheres smaller than this are not included in the negative space */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = 0, Units = cm))
	double MinRadius = 10.0;

	UPROPERTY(EditAnywhere, Category = "Debug Draw - Sphere Covering", meta = (ShowOnlyInnerProperties))
	FDataflowNodeSphereCoveringDebugDrawSettings SphereCoveringDebugDrawRenderSettings;


	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override
	{
		return true;
	}
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif

public:
	FCreateNegativeSpaceSphereCovering(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Generates cluster convex hulls for leafs hulls
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGenerateClusterConvexHullsFromLeafHullsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGenerateClusterConvexHullsFromLeafHullsDataflowNode, "GenerateClusterConvexHullsFromLeafHulls", "GeometryCollection|Utilities", "")

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput))
	FManagedArrayCollection Collection;

	// A representation of the negative space protected by the 'protect negative space' option. If negative space is not protected, this will contain zero spheres.
	UPROPERTY(meta = (DataflowOutput))
	FDataflowSphereCovering SphereCovering;

	/** Maximum number of convex to generate for a specific cluster. Will be ignored if error tolerance is used instead */
	UPROPERTY(EditAnywhere, Category = "Convex", meta = (DataflowInput, EditCondition = "ErrorTolerance == 0"))
	int32 ConvexCount = 2;
	
	/** 
	* Error tolerance to use to decide to merge leaf convex together. 
	* This is in centimeters and represents the side of a cube, the volume of which will be used as threshold
	* to know if the volume of the generated convex is too large compared to the sum of the volume of the leaf convex
	*/
	UPROPERTY(EditAnywhere, Category = "Convex", meta = (DataflowInput, UIMin = "0", UIMax = "100.", Units = cm))
	double ErrorTolerance = 0.0;
	
	/** Whether to prefer available External (imported) collision shapes instead of the computed convex hulls on the Collection */
	UPROPERTY(EditAnywhere, Category = "Convex")
	bool bPreferExternalCollisionShapes = true;

	/** Method to determine if the convex hulls from two separate bones can potentially be merged */
	UPROPERTY(EditAnywhere, Category = "Convex", meta = (DisplayName = "Allow Merges Between Bones"))
	EAllowConvexMergeMethod AllowMerges = EAllowConvexMergeMethod::ByProximity;

	/** Filter to optionally only consider spatially close convex hulls for merges */
	UPROPERTY(EditAnywhere, Category = "Convex")
	EConvexHullProximityFilter MergeProximityFilter = EConvexHullProximityFilter::None;

	/** If applying a convex hull proximity filter, the distance threshold to use for determining that two convex hulls are close enough to merge */
	UPROPERTY(EditAnywhere, Category = "Convex", meta = (Units = cm, EditCondition = "MergeProximityFilter != EConvexHullProximityFilter::None"))
	float MergeProximityDistanceThreshold = .1f;

	/** Optional transform selection to compute cluster hulls on -- if not provided, all cluster hulls will be computed. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection OptionalSelectionFilter;

	/** Also allow the same hull merging process to run on leaf hulls (merging hulls on leaves in the selection) */
	UPROPERTY(EditAnywhere, Category = "Convex", meta = (DataflowInput))
	bool bAllowMergingLeafHulls = false;

	/** Negative space that must be used. Applies in addition to any automatically-detected covering added via bProtectNegativeSpace. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowSphereCovering RequireNegativeSpaceCovering;

	/** Whether to generate a sphere cover to define negative space that should not be covered by convex hulls */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput))
	bool bProtectNegativeSpace = false;

	/** Method to use to find and sample negative space */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (EditCondition = "bProtectNegativeSpace", EditConditionHides))
	ENegativeSpaceSampleMethodDataflowEnum SampleMethod = ENegativeSpaceSampleMethodDataflowEnum::Uniform;

	/** Whether to require that all candidate locations identified by Voxel Search are covered by negative space samples, up to the specified Min Sample Spacing. Only applies to Voxel Search. */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (EditCondition = "bProtectNegativeSpace && SampleMethod != ENegativeSpaceSampleMethodDataflowEnum::Uniform", EditConditionHides))
	bool bRequireSearchSampleCoverage = false;

	/** When performing Voxel Search, only look for negative space that is connected out to the convex hull. This removes inaccessable internal negative space from consideration. Only applies to Voxel Search. */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (EditCondition = "bProtectNegativeSpace && SampleMethod != ENegativeSpaceSampleMethodDataflowEnum::Uniform", EditConditionHides))
	bool bOnlyConnectedToHull = false;

	/** Approximate number of spheres to consider when covering negative space. Only applicable with the Uniform Sample Method or if Require Search Sample Coverage is disabled. */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = 1, EditCondition = "bProtectNegativeSpace", EditConditionHides))
	int32 TargetNumSamples = 50;

	/** Minimum desired spacing between spheres; if > 0, will attempt not to place sphere centers closer than this */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = 0, Units = cm, EditCondition = "bProtectNegativeSpace", EditConditionHides))
	double MinSampleSpacing = 1.0;

	/** Amount of space to leave between convex hulls and protected negative space */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = .01, UIMin = .1, Units = cm, EditCondition = "bProtectNegativeSpace", EditConditionHides))
	double NegativeSpaceTolerance = 2.0;

	/** Spheres smaller than this are not included in the negative space */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = 0, Units = cm, EditCondition = "bProtectNegativeSpace", EditConditionHides))
	double MinRadius = 10.0;

	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ShowOnlyInnerProperties))
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	/** Randomize color per convex hull */
	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	bool bRandomizeColor = true;

	/** Random seed */
	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ClampMin = "0", EditCondition = "bRandomizeColor==true", EditConditionHides))
	int32 ColorRandomSeed = 0;

	UPROPERTY(EditAnywhere, Category = "Debug Draw - Sphere Covering", meta = (ShowOnlyInnerProperties))
	FDataflowNodeSphereCoveringDebugDrawSettings SphereCoveringDebugDrawRenderSettings;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override
	{
		return true;
	}
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif

public:
	FGenerateClusterConvexHullsFromLeafHullsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());


};

/**
 *
 * Generates cluster convex hulls for children hulls
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGenerateClusterConvexHullsFromChildrenHullsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGenerateClusterConvexHullsFromChildrenHullsDataflowNode, "GenerateClusterConvexHullsFromChildrenHulls", "GeometryCollection|Utilities", "")

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput))
	FManagedArrayCollection Collection;
	
	// A representation of the negative space protected by the 'protect negative space' option. If negative space is not protected, this will contain zero spheres.
	UPROPERTY(meta = (DataflowOutput))
	FDataflowSphereCovering SphereCovering;

	/** Maximum number of convex to generate for a specific cluster. Will be ignored if error tolerance is used instead */
	UPROPERTY(EditAnywhere, Category = "Convex", meta = (DataflowInput, EditCondition = "ErrorTolerance == 0"))
	int32 ConvexCount = 2;

	/**
	* Error tolerance to use to decide to merge leaf convex together.
	* This is in centimeters and represents the side of a cube, the volume of which will be used as threshold
	* to know if the volume of the generated convex is too large compared to the sum of the volume of the leaf convex
	*/
	UPROPERTY(EditAnywhere, Category = "Convex", meta = (DataflowInput, UIMin = "0", UIMax = "100.", Units = cm))
	double ErrorTolerance = 0.0;
	
	/** Whether to prefer available External (imported) collision shapes instead of the computed convex hulls on the Collection */
	UPROPERTY(EditAnywhere, Category = "Convex")
	bool bPreferExternalCollisionShapes = true;

	/** Optional transform selection to compute cluster hulls on -- if not provided, all cluster hulls will be computed. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection OptionalSelectionFilter;

	/** Filter to optionally only consider spatially close convex hulls for merges */
	UPROPERTY(EditAnywhere, Category = "Convex")
	EConvexHullProximityFilter MergeProximityFilter = EConvexHullProximityFilter::None;

	/** If applying a convex hull proximity filter, the distance threshold to use for determining that two convex hulls are close enough to merge */
	UPROPERTY(EditAnywhere, Category = "Convex", meta = (Units = cm, EditCondition = "MergeProximityFilter != EConvexHullProximityFilter::None"))
	float MergeProximityDistanceThreshold = .1f;

	/** Also allow the same hull merging process to run on leaf hulls (merging hulls on leaves in the selection) */
	UPROPERTY(EditAnywhere, Category = "Convex", meta = (DataflowInput))
	bool bAllowMergingLeafHulls = false;

	/** Negative space that must be used. Applies in addition to any automatically-detected covering added via bProtectNegativeSpace. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowSphereCovering RequireNegativeSpaceCovering;

	/** Whether to generate a sphere cover to define negative space that should not be covered by convex hulls */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput))
	bool bProtectNegativeSpace = false;

	/** Method to use to find and sample negative space */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (EditCondition = "bProtectNegativeSpace", EditConditionHides))
	ENegativeSpaceSampleMethodDataflowEnum SampleMethod = ENegativeSpaceSampleMethodDataflowEnum::Uniform;

	/** Whether to require that all candidate locations identified by Voxel Search are covered by negative space samples, up to the specified Min Sample Spacing. Only applies to Voxel Search. */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (EditCondition = "bProtectNegativeSpace && SampleMethod != ENegativeSpaceSampleMethodDataflowEnum::Uniform", EditConditionHides))
	bool bRequireSearchSampleCoverage = false;

	/** When performing Voxel Search, only look for negative space that is connected out to the convex hull. This removes inaccessable internal negative space from consideration. Only applies to Voxel Search. */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (EditCondition = "bProtectNegativeSpace && SampleMethod != ENegativeSpaceSampleMethodDataflowEnum::Uniform", EditConditionHides))
	bool bOnlyConnectedToHull = false;

	/** Approximate number of spheres to consider when covering negative space. Only applicable with the Uniform Sample Method or if Require Search Sample Coverage is disabled. */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = 1, EditCondition = "bProtectNegativeSpace", EditConditionHides))
	int32 TargetNumSamples = 50;

	/** Minimum desired spacing between spheres; if > 0, will attempt not to place sphere centers closer than this */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = 0, Units = cm, EditCondition = "bProtectNegativeSpace", EditConditionHides))
	double MinSampleSpacing = 1.0;

	/** Amount of space to leave between convex hulls and protected negative space */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = .01, UIMin = .1, Units = cm, EditCondition = "bProtectNegativeSpace", EditConditionHides))
	double NegativeSpaceTolerance = 2.0;

	/** Spheres smaller than this are not included in the negative space */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = 0, Units = cm, EditCondition = "bProtectNegativeSpace", EditConditionHides))
	double MinRadius = 10.0;

	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ShowOnlyInnerProperties))
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	/** Randomize color per convex hull */
	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	bool bRandomizeColor = true;

	/** Random seed */
	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ClampMin = "0", EditCondition = "bRandomizeColor==true", EditConditionHides))
	int32 ColorRandomSeed = 0;

	UPROPERTY(EditAnywhere, Category = "Debug Draw - Sphere Covering", meta = (ShowOnlyInnerProperties))
	FDataflowNodeSphereCoveringDebugDrawSettings SphereCoveringDebugDrawRenderSettings;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override
	{
		return true;
	}
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif

public:
	FGenerateClusterConvexHullsFromChildrenHullsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());


};


/** Clear convex hulls from a collection */
USTRUCT(meta = (DataflowGeometryCollection))
struct FClearConvexHullsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FClearConvexHullsDataflowNode, "ClearConvexHulls", "GeometryCollection|Utilities", "")

public:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** [Optional] selection of transforms to clear convex on, if not set all the transform will be used */
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection TransformSelection;

	FClearConvexHullsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);

		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


/** Copy convex hulls from given transforms on a source collection to transforms on the target collection */
USTRUCT(meta = (DataflowGeometryCollection))
struct FCopyConvexHullsFromRootDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FCopyConvexHullsFromRootDataflowNode, "CopyConvexHullsFromRoot", "GeometryCollection|Utilities", "")

public:
	FCopyConvexHullsFromRootDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;
	
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FManagedArrayCollection FromCollection;

	// Whether to skip copying from roots with no convex hulls
	UPROPERTY(EditAnywhere, Category = "Options", meta = (DataflowInput))
	bool bSkipIfEmpty = true;

	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ShowOnlyInnerProperties))
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	/** Randomize color per convex hull */
	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	bool bRandomizeColor = true;

	/** Random seed */
	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ClampMin = "0"))
	int32 ColorRandomSeed = 0;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override
	{
		return true;
	}
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif

};


/** Merge convex hulls on transforms with multiple hulls */
USTRUCT(meta = (DataflowGeometryCollection))
struct FMergeConvexHullsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FMergeConvexHullsDataflowNode, "MergeConvexHulls", "GeometryCollection|Utilities", "")

private:
	UPROPERTY(meta = (DataflowInput, DataflowOutput))
	FManagedArrayCollection Collection;

	// A representation of the negative space protected by the 'protect negative space' option. If negative space is not protected, this will contain zero spheres.
	UPROPERTY(meta = (DataflowOutput))
	FDataflowSphereCovering SphereCovering;

	/** Maximum number of convex to generate per transform. Ignored if < 0. */
	UPROPERTY(EditAnywhere, Category = "Convex", meta = (DataflowInput))
	int32 MaxConvexCount = -1;

	/**
	* Error tolerance to use to decide to merge leaf convex together.
	* This is in centimeters and represents the side of a cube, the volume of which will be used as threshold
	* to know if the volume of the generated convex is too large compared to the sum of the volume of the leaf convex
	*/
	UPROPERTY(EditAnywhere, Category = "Convex", meta = (DataflowInput, ClampMin = "0", UIMax = "100.", Units = cm))
	double ErrorTolerance = 0.0;

	/** Optional transform selection to compute cluster hulls on -- if not provided, all cluster hulls will be computed. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowTransformSelection OptionalSelectionFilter;

	/** Filter to optionally only consider spatially close convex hulls for merges */
	UPROPERTY(EditAnywhere, Category = "Convex")
	EConvexHullProximityFilter MergeProximityFilter = EConvexHullProximityFilter::None;

	/** If applying a convex hull proximity filter, the distance threshold to use for determining that two convex hulls are close enough to merge */
	UPROPERTY(EditAnywhere, Category = "Convex", meta = (Units = cm, EditCondition = "MergeProximityFilter != EConvexHullProximityFilter::None"))
	float MergeProximityDistanceThreshold = .1f;

	/** Negative space that must be used. Applies in addition to any automatically-detected covering added via bProtectNegativeSpace. */
	UPROPERTY(meta = (DataflowInput))
	FDataflowSphereCovering RequireNegativeSpaceCovering;

	/** Whether to generate a sphere cover to define negative space that should not be covered by convex hulls */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput))
	bool bProtectNegativeSpace = false;

	/** Whether to compute separate negative space for each bone. Otherwise, a single negative space will be computed once and re-used for all bones. */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (EditCondition = "bProtectNegativeSpace", EditConditionHides))
	bool bComputeNegativeSpacePerBone = false;

	/** Method to use to find and sample negative space */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (EditCondition = "bProtectNegativeSpace", EditConditionHides))
	ENegativeSpaceSampleMethodDataflowEnum SampleMethod = ENegativeSpaceSampleMethodDataflowEnum::Uniform;

	/** Whether to require that all candidate locations identified by Voxel Search are covered by negative space samples, up to the specified Min Sample Spacing. Only applies to Voxel Search. */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (EditCondition = "bProtectNegativeSpace && SampleMethod != ENegativeSpaceSampleMethodDataflowEnum::Uniform", EditConditionHides))
	bool bRequireSearchSampleCoverage = false;

	/** When performing Voxel Search, only look for negative space that is connected out to the convex hull. This removes inaccessable internal negative space from consideration. Only applies to Voxel Search. */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (EditCondition = "bProtectNegativeSpace && SampleMethod != ENegativeSpaceSampleMethodDataflowEnum::Uniform", EditConditionHides))
	bool bOnlyConnectedToHull = false;

	/** Approximate number of spheres to consider when covering negative space. Only applicable with the Uniform Sample Method or if Require Search Sample Coverage is disabled. */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = 1, EditCondition = "bProtectNegativeSpace", EditConditionHides))
	int32 TargetNumSamples = 50;

	/** Minimum desired spacing between spheres; if > 0, will attempt not to place sphere centers closer than this */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = 0, Units = cm, EditCondition = "bProtectNegativeSpace", EditConditionHides))
	double MinSampleSpacing = 1.0;

	/** Amount of space to leave between convex hulls and protected negative space */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = .01, UIMin = .1, Units = cm, EditCondition = "bProtectNegativeSpace", EditConditionHides))
	double NegativeSpaceTolerance = 2.0;

	/** Spheres smaller than this are not included in the negative space */
	UPROPERTY(EditAnywhere, Category = NegativeSpace, meta = (DataflowInput, ClampMin = 0, Units = cm, EditCondition = "bProtectNegativeSpace", EditConditionHides))
	double MinRadius = 10.0;

	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ShowOnlyInnerProperties))
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	/** Randomize color per convex hull */
	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	bool bRandomizeColor = true;

	/** Random seed */
	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ClampMin = "0", EditCondition = "bRandomizeColor==true", EditConditionHides))
	int32 ColorRandomSeed = 0;

	UPROPERTY(EditAnywhere, Category = "Debug Draw - Sphere Covering", meta = (ShowOnlyInnerProperties))
	FDataflowNodeSphereCoveringDebugDrawSettings SphereCoveringDebugDrawRenderSettings;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override
	{
		return true;
	}
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif
public:
	FMergeConvexHullsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};


/**
 *
 * Update the Volume and Size attributes on the target Collection (and add them if they were not present)
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FUpdateVolumeAttributesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FUpdateVolumeAttributesDataflowNode, "UpdateVolumeAttributes", "GeometryCollection|Utilities", "collection")

public:
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Collection, DataflowIntrinsic))
	FManagedArrayCollection Collection;

	FUpdateVolumeAttributesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Compute information about the volume of a collection specific transform or root
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FComputeVolumeStatsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FComputeVolumeStatsDataflowNode, "ComputeVolumeStatsData", "GeometryCollection|Utilities", "collection empty space percent size convex geometry")

public:
	FComputeVolumeStatsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Collection to collect the stats from */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = Collection, DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Selection to consider , if set to -1 or not a valid transform in the collection, the root will be considered */
	UPROPERTY(meta = (DataflowInput))
	int32 TransformIndex = INDEX_NONE;

	/** Volume of the geometry in cm3 */
	UPROPERTY(meta = (DataflowOutput));
	float GeometryVolume = 0.f;

	/** Volume of the single wrapping convex around the geometry in cm3 */
	UPROPERTY(meta = (DataflowOutput));
	float ConvexVolume = 0.f;

	/** Ratio of the geometry volume in relation to the wrapping convex volume ( GeometryVolume / ConvexVolume ) */
	UPROPERTY(meta = (DataflowOutput));
	float VolumeRatio = 1.0f;

	/** Percentage of the geometry volume in relation to the wrapping convex volume ( GeometryVolume / ConvexVolume * 100 ) */
	UPROPERTY(meta = (DataflowOutput));
	float VolumePercentage = 100.f;

	/** Cube size of the empty space volume (cubic root of the volume = Edge size of a cube of similar volume) */
	UPROPERTY(meta = (DataflowOutput));
	float EmptySpaceCubeSize = 100.f;


	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 * Get the sum of volumes of the convex hulls on the selected nodes
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGetConvexHullVolumeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGetConvexHullVolumeDataflowNode, "GetConvexHullVolume", "GeometryCollection|Utilities", "")

private:
	UPROPERTY(meta = (DataflowInput))
	FManagedArrayCollection Collection;

	/** The transforms to consider */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	FDataflowTransformSelection TransformSelection;

	/** Sum of convex hull volumes */
	UPROPERTY(meta = (DataflowOutput));
	float Volume = 0.f;

	/** For any cluster transform that has no convex hulls, whether to fall back to the convex hulls of the cluster's children. Otherwise, the cluster will not add to the total volume sum. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bSumChildrenForClustersWithoutHulls = true;

	/** Whether to take the volume of the union of selected hulls, rather than the sum of each hull volume separately. This is more expensive but more accurate when hulls overlap. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bVolumeOfUnion = false;

	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ShowOnlyInnerProperties))
	FDataflowNodeDebugDrawSettings DebugDrawRenderSettings;

	/** Randomize color per convex hull */
	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	bool bRandomizeColor = true;

	/** Random seed */
	UPROPERTY(EditAnywhere, Category = "Debug Draw", meta = (ClampMin = "0", EditCondition = "bRandomizeColor==true", EditConditionHides))
	int32 ColorRandomSeed = 0;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override
	{
		return true;
	}
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif
public:
	FGetConvexHullVolumeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 * 
 * Editor Fracture Mode / Utilities / TinyGeo tool
 * Merge pieces of geometry onto their neighbors -- use it to, for example, clean up too small pieces of geometry.
 * 
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FFixTinyGeoDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FFixTinyGeoDataflowNode, "FixTinyGeo", "GeometryCollection|Fracture|Utilities", "")

public:
	/** Collection to use */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** The selected pieces to use */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/** Whether to merge small geometry, or small clusters */
	UPROPERTY(EditAnywhere, Category = Distribution)
	EFixTinyGeoMergeType MergeType = EFixTinyGeoMergeType::MergeGeometry;

	/** Only consider bones at the current Fracture Level */
	UPROPERTY(EditAnywhere, Category = MergeSettings, meta = (EditCondition = "!bFractureLevelIsAll && MergeType == EFixTinyGeoMergeType::MergeClusters", EditConditionHides))
	bool bOnFractureLevel = true;

	/** Only auto-consider clusters for merging. Note that leaf nodes can still be consider if manually selected. */
	UPROPERTY(EditAnywhere, Category = MergeSettings, meta = (EditCondition = "!bFractureLevelIsAll && MergeType == EFixTinyGeoMergeType::MergeClusters && bOnFractureLevel", EditConditionHides))
	bool bOnlyClusters = false;

	/** Only merge clusters to neighbors with the same parent in the hierarchy */
	UPROPERTY(EditAnywhere, Category = MergeSettings, meta = (DisplayName = "Only Merge Clusters w/ Same Parent", EditCondition = "MergeType == EFixTinyGeoMergeType::MergeClusters", EditConditionHides))
	bool bOnlySameParent = true;
	
	/** Only merge geometry to neighbors with the same parent in the hierarchy */
	UPROPERTY(EditAnywhere, Category = MergeSettings, meta = (DisplayName = "Only Merge Geometry w/ Same Parent", EditCondition = "MergeType == EFixTinyGeoMergeType::MergeGeometry", EditConditionHides))
	bool bGeometryOnlySameParent = false;

	/** Helper variable to let the EditConditions above check whether the Fracture Level is set to 'All' */
	UPROPERTY()
	bool bFractureLevelIsAll = false;

	UPROPERTY(EditAnywhere, Category = MergeSettings, meta = (DisplayName = "Merge To"))
	EFixTinyGeoNeighborSelectionMethod NeighborSelection = EFixTinyGeoNeighborSelectionMethod::LargestNeighbor;

	/** Only merge pieces that are connected in the proximity graph.If unchecked, connected pieces will still be favored, but if none are available the closest disconnected piece can be merged. */
	UPROPERTY(EditAnywhere, Category = MergeSettings, meta = (EditCondition = "MergeType == EFixTinyGeoMergeType::MergeClusters", EditConditionHides))
	bool bOnlyToConnected = true;

	/** Whether to use the Proximity (as computed by the Proximity node) to determine which bones are connected, and thus can be considered for merging. Otherwise will compute and use a reasonable default connectivity. */
	UPROPERTY(EditAnywhere, Category = MergeSettings, AdvancedDisplay)
	bool bUseCollectionProximityForConnections = false;

	/** Options for using the current bone selection */
	UPROPERTY(EditAnywhere, Category = MergeSettings, meta = (DisplayName = "Bone Selection"))
	EFixTinyGeoUseBoneSelection UseBoneSelection = EFixTinyGeoUseBoneSelection::NoEffect;

	UPROPERTY(EditAnywhere, Category = FilterSettings)
	EFixTinyGeoGeometrySelectionMethod SelectionMethod = EFixTinyGeoGeometrySelectionMethod::RelativeVolume;

	/** If size (cube root of volume) is less than this value, geometry should be merged into neighbors -- i.e. a value of 2 merges geometry smaller than a 2x2x2 cube */
	UPROPERTY(EditAnywhere, Category = FilterSettings, meta = (DisplayName = "MinSize", ClampMin = ".00001", UIMin = ".1", UIMax = "10", EditCondition = "SelectionMethod == EFixTinyGeoGeometrySelectionMethod::VolumeCubeRoot", EditConditionHides))
	float MinVolumeCubeRoot = 1.f;

	/** If cube root of volume relative to the overall shape's cube root of volume is less than this, the geometry should be merged into its neighbors.
	(Note: This is a bit different from the histogram viewer's "Relative Size," which instead shows values relative to the largest rigid bone.) */
	UPROPERTY(EditAnywhere, Category = FilterSettings, meta = (ClampMin = "0", UIMax = ".1", ClampMax = "1.0", EditCondition = "SelectionMethod == EFixTinyGeoGeometrySelectionMethod::RelativeVolume", EditConditionHides))
	float RelativeVolume = .01f;

	/**
	 * If enabled, add extra vertices (without triangles) to the geometry in regions where vertices are spaced too far apart (e.g. across large triangles)
	 * These extra vertices will be used as collision samples in particle-implicit collisions, and can help the physics system detect collisions more accurately
	 *
	 * Note this is *only* useful for simulations that use particle-implicit collisions
	 */
	UPROPERTY(EditAnywhere, Category = "Collision");
	bool AddSamplesForCollision = false;

	/**
	 * The number of centimeters to allow between vertices on the mesh surface: If there are gaps larger than this, add additional vertices (without triangles) to help support particle-implicit collisions
	 * Only used if Add Samples For Collision is enabled
	 */
	UPROPERTY(EditAnywhere, Category = "Collision", meta = (DataflowInput, DisplayName = "Point Spacing", UIMin = 0.f, EditCondition = "AddSamplesForCollision"));
	float CollisionSampleSpacing = 50.f;

	FFixTinyGeoDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterInputConnection(&CollisionSampleSpacing);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Editor Fracture Mode / Utilities / Split Islands tool
 * Split selected geometry into islands based on mesh connectivity and distances.
 *
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FSplitIslandsDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSplitIslandsDataflowNode, "SplitIslands", "GeometryCollection|Fracture|Utilities", "Disconnect Connected Components")

private:
	/** Collection to use */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** The selected pieces to use */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/** Distance threshold for connecting spatially-close vertices before computing islands */
	UPROPERTY(EditAnywhere, Category = "Split Islands", meta = (Units = "cm", ClampMin = "0.0"))
	float CloseVertexDistance = 0.001f;

	/** If > 0, bridge separate islands whose surfaces are within this vertex-to-triangle distance. 0 = disabled. */
	UPROPERTY(EditAnywhere, Category = "Split Islands", meta = (Units = "cm", ClampMin = "0.0"))
	float VertexToSurfaceBridgeDistance = 0.f;
	
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	
public:
	FSplitIslandsDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};

/**
 *
 * Editor Fracture Mode / Utilities / Normals tool
 * Recompute normals and tangents.
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FRecomputeNormalsInGeometryCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FRecomputeNormalsInGeometryCollectionDataflowNode, "RecomputeNormalsInGeometryCollection", "GeometryCollection|Fracture|Utilities", "")

public:
	/** Collection to use */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** The selected pieces to use */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/** Whether to only recompute tangents, and leave normals as they were */
	UPROPERTY(EditAnywhere, Category = RecomputeSettings)
	bool bOnlyTangents = false;

	/** If true, update where edges are 'sharp' by comparing adjacent triangle face normals vs the Sharp Edge Angle Threshold. */
	UPROPERTY(EditAnywhere, Category = RecomputeSettings, meta = (EditCondition = "!bOnlyTangents"))
	bool bRecomputeSharpEdges = false;

	/** Threshold on angle of change in face normals across an edge, above which we create a sharp edge if bRecomputeSharpEdges is true */
	UPROPERTY(EditAnywhere, Category = RecomputeSettings, meta = (UIMin = "0.0", UIMax = "180.0", ClampMin = "0.0", ClampMax = "180.0", EditCondition = "bRecomputeSharpEdges && !bOnlyTangents"))
	float SharpEdgeAngleThreshold = 60.0f;

	/** Whether to only change internal surface normals / tangents */
	UPROPERTY(EditAnywhere, Category = RecomputeSettings, AdvancedDisplay)
	bool bOnlyInternalSurfaces = true;

	FRecomputeNormalsInGeometryCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Editor Fracture Mode / Utilities / Resample tool
 * Resample to add collision particles in large flat regions that otherwise might have poor collision response.
 * Only useful to help improve Particle - Implicit collisions.
 * 
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FResampleGeometryCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FResampleGeometryCollectionDataflowNode, "ResampleGeometryCollection", "GeometryCollection|Fracture|Utilities", "")

public:
	/** Collection to use */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** The selected pieces to use */
	UPROPERTY(meta = (DataflowInput, DisplayName = "TransformSelection"))
	FDataflowTransformSelection TransformSelection;

	/**
	 * If enabled, add extra vertices (without triangles) to the geometry in regions where vertices are spaced too far apart (e.g. across large triangles)
	 * These extra vertices will be used as collision samples in particle-implicit collisions, and can help the physics system detect collisions more accurately
	 *
	 * Note this is *only* useful for simulations that use particle-implicit collisions
	 */
	UPROPERTY(EditAnywhere, Category = "Collision");
	bool AddSamplesForCollision = false;

	/**
	 * The number of centimeters to allow between vertices on the mesh surface: If there are gaps larger than this, add additional vertices (without triangles) to help support particle-implicit collisions
	 * Only used if Add Samples For Collision is enabled
	 */
	UPROPERTY(EditAnywhere, Category = "Collision", meta = (DataflowInput, DisplayName = "Point Spacing", UIMin = 0.f, EditCondition = "AddSamplesForCollision"));
	float CollisionSampleSpacing = 50.f;

	FResampleGeometryCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&TransformSelection);
		RegisterInputConnection(&CollisionSampleSpacing);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/**
 *
 * Editor Fracture Mode / Utilities / Validate tool
 * Ensures that geometrycollection is valid and clean.
 * 
 */
USTRUCT(meta = (DataflowGeometryCollection))
struct FValidateGeometryCollectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FValidateGeometryCollectionDataflowNode, "ValidateGeometryCollection", "GeometryCollection|Fracture|Utilities", "")

public:
	/** Collection to use */
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Find and remove any unused geometry data */
	UPROPERTY(EditAnywhere, Category = CleanUnused)
	bool bRemoveUnreferencedGeometry = true;

	/** Whether to collapse any clusters with only a single child */
	UPROPERTY(EditAnywhere, Category = Clustering)
	bool bRemoveClustersOfOne = false;

	/** Remove dangling clusters -- Note this can invalidate caches */
	UPROPERTY(EditAnywhere, Category = Clustering)
	bool bRemoveDanglingClusters = false;

	FValidateGeometryCollectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void GeometryCollectionUtilityNodes();
}

