// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowDebugDraw.h"
#if WITH_EDITOR
#include "Dataflow/DataflowRenderingViewMode.h"
#endif
#include "Dataflow/DataflowEngineUtil.h"

namespace UE::Dataflow
{
	namespace Utils
	{
#if WITH_EDITOR
		GEOMETRYCOLLECTIONNODES_API void DebugDrawProximity(IDataflowDebugDrawInterface& DataflowRenderingInterface,
			const FManagedArrayCollection& Collection,
			const FLinearColor Color,
			const float LineWidthMultiplier,
			const float CenterSize,
			const FLinearColor CenterColor,
			const bool bRandomizeColor,
			const int32 ColorRandomSeed);

		GEOMETRYCOLLECTIONNODES_API FLinearColor GetColorByLevel(const FManagedArrayCollection& InCollection, int32 InLevel);
#endif

		const uint32 Error_InvalidChars = 1;
		const uint32 Error_InvalidFormatInSegment = 2;

		/* e.g. "0, 2, 5-10, 12-15" */
		GEOMETRYCOLLECTIONNODES_API bool ParseIndicesStr(const FString& InFramesString, TArray<int32>& OutIndices, uint32& OutErrorCode);
	}
}


