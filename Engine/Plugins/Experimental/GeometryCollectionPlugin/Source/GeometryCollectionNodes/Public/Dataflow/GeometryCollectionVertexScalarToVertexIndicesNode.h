// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowCollectionAttributeKeyNodes.h"
#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "GeometryCollectionVertexScalarToVertexIndicesNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/** Convert an vertex float array to a list of indices */
USTRUCT(meta = (DataflowGeometryCollection))
struct FGeometryCollectionVertexScalarToVertexIndicesNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FGeometryCollectionVertexScalarToVertexIndicesNode, "VertexScalarToVertexIndices", "GeometryCollection", "Collection Vertex Weight Map to Indices")

public:

	UPROPERTY(Meta = (DataflowInput))
	FManagedArrayCollection Collection;

	/** The name of the vertex attribute and group to generate indices from. */
	UPROPERTY(EditAnywhere, Category = "Dataflow", meta = (DataflowInput, DisplayName = "AttributeKey"))
	FCollectionAttributeKey AttributeKey = FCollectionAttributeKey("", "Vertices");

	/** The value threshold for what is included in the vertex list. */
	UPROPERTY(EditAnywhere, Category = "Selection Filter", Meta = (ClampMin = "0", ClampMax = "1"))
	float SelectionThreshold = 0.f;

	/** Output list of indices */
	UPROPERTY(Meta = (DataflowOutput, DisplayName = "VertexIndices"))
	TArray<int32> VertexIndices = {};

	FGeometryCollectionVertexScalarToVertexIndicesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
