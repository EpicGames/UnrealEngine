// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "GeometryCollectionVerticesNodes.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class USkeletalMesh;

USTRUCT(meta = (DataflowGeometryCollection))
struct FTransformCollectionAttributeDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FTransformCollectionAttributeDataflowNode, "TransformCollectionAttribute", "GeometryCollection", "")

public:
	typedef FManagedArrayCollection DataType;


	UPROPERTY(meta = (DataflowInput, DataflowOutput, DisplayName = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(meta = (DataflowInput, DisplayName = "Transform"))
	FTransform TransformIn = FTransform::Identity;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FTransform LocalTransform = FTransform::Identity;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FString GroupName = FString("Vertices");

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FString AttributeName = FString("Vertex");


	FTransformCollectionAttributeDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&TransformIn);
		RegisterInputConnection(&Collection);
		RegisterOutputConnection(&Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

namespace UE::Dataflow
{
	void GeometryCollectionVerticesNodes();
}

