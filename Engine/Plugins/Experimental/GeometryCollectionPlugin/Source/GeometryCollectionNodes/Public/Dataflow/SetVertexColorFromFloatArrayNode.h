// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "SetVertexColorFromFloatArrayNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class FGeometryCollection;

/** Set the vertex color on the collection based on the normalized float array. */
USTRUCT()
struct FSetVertexColorFromFloatArrayDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetVertexColorFromFloatArrayDataflowNode, "SetVertexColorFromFloatArray", "Collection|Utilities", "")

public:
	/** Collection Passthrough*/
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Float array to use as a scalar for the color */
	UPROPERTY(meta = (DataflowInput, DataflowIntrinsic))
	TArray<float> FloatArray;

	/** Enable normalization of input array */
	UPROPERTY(EditAnywhere, Category = "Color", meta = (DisplayName = "Normalize Input"))
	bool bNormalizeInput = true;
	
	/** Base color for the normalized float array */
	UPROPERTY(EditAnywhere, Category = "Color", meta = (DisplayName = "Color"))
	FLinearColor Color = FLinearColor(FColor::White);


	FSetVertexColorFromFloatArrayDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&FloatArray);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};
