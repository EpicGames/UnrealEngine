// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "SetVertexColorFromVertexIndicesNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/** Set the vertex color of the collection based on the selection set. */
USTRUCT()
struct FSetVertexColorFromVertexIndicesDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetVertexColorFromVertexIndicesDataflowNode, "SetVertexColorFromVertexIndices", "Collection|Utilities", "")

public:
	/** Collection Passthrough*/
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Vertex indices set */
	UPROPERTY(EditAnywhere, Category = "Vertices", meta = (DataflowInput, DisplayName = "VertexIndices"))
	TArray<int32> VertexIndicesIn;

	/** Selected vertex color */
	UPROPERTY(EditAnywhere, Category = "Color")
	FLinearColor SelectedColor = FLinearColor(FColor::Yellow);

	FSetVertexColorFromVertexIndicesDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&VertexIndicesIn);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};
