// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Dataflow/DataflowEngine.h"
#include "Dataflow/DataflowSelection.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "SetVertexColorFromVertexSelectionNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/** Set the vertex color of the collection based on the selection set. */
USTRUCT()
struct FSetVertexColorFromVertexSelectionDataflowNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FSetVertexColorFromVertexSelectionDataflowNode, "SetVertexColorFromVertexSelection", "Collection|Utilities", "")

public:
	/** Collection Passthrough*/
	UPROPERTY(meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection", DataflowIntrinsic))
	FManagedArrayCollection Collection;

	/** Vertex selection set */
	UPROPERTY(meta = (DataflowInput, DisplayName = "VertexSelection", DataflowIntrinsic))
	FDataflowVertexSelection VertexSelection;

	/** Selected vertex color */
	UPROPERTY(EditAnywhere, Category = "Color")
	FLinearColor SelectedColor = FLinearColor(FColor::Yellow);

	FSetVertexColorFromVertexSelectionDataflowNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid())
		: FDataflowNode(InParam, InGuid)
	{
		RegisterInputConnection(&Collection);
		RegisterInputConnection(&VertexSelection);
		RegisterOutputConnection(&Collection, &Collection);
	}

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

};
