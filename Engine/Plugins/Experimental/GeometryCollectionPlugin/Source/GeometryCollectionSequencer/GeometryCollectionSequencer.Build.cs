// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class GeometryCollectionSequencer : ModuleRules
    {
        public GeometryCollectionSequencer(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicIncludePathModuleNames.AddRange(
                new string[] {
                "Sequencer",
                }
            );

            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "AssetTools",
                    "Core",
                    "CoreUObject",
                    "Engine",
					"LevelSequence",
                    "MovieScene",
                    "MovieSceneTools",
                    "MovieSceneTracks",
                    "RHI",
                    "SequencerCore",
                    "Sequencer",
                    "Slate",
                    "SlateCore",
                    "TimeManagement",
					"EditorFramework",
                    "UnrealEd",
                    "GeometryCollectionTracks",
					"Chaos",
                    "GeometryCollectionEngine",
                }
            );
        }
    }
}
