// Copyright Epic Games, Inc. All Rights Reserved.
#include "GeometryCollectionSequencerModule.h"
#include "ISequencerModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "GeometryCollectionSequencer"



IMPLEMENT_MODULE(FGeometryCollectionSequencerModule, GeometryCollectionSequencer);

#undef LOCTEXT_NAMESPACE
