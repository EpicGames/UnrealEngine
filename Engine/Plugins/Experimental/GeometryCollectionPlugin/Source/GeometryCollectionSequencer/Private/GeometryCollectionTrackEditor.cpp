// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollectionTrackEditor.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "Modules/ModuleManager.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBox.h"
#include "SequencerSectionPainter.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "MovieSceneGeometryCollectionTrack.h"
#include "MovieSceneGeometryCollectionSection.h"
#include "ContentBrowserModule.h"
#include "MVVM/Views/ViewUtilities.h"
#include "ISectionLayoutBuilder.h"
#include "Styling/AppStyle.h"
#include "MovieSceneTimeHelpers.h"
#include "Fonts/FontMeasure.h"
#include "SequencerTimeSliderController.h"
#include "Misc/MessageDialog.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "Styling/SlateIconFinder.h"
#include "LevelSequence.h"
#include "TimeToPixel.h"

namespace GeometryCollectionEditorConstants
{
	// @todo Sequencer Allow this to be customizable
	const uint32 AnimationTrackHeight = 28;
}

#define LOCTEXT_NAMESPACE "FGeometryCollectionTrackEditor"

static UGeometryCollectionComponent* AcquireGeometryCollectionFromObjectGuid(const FGuid& Guid, TSharedPtr<ISequencer> SequencerPtr)
{
	UObject* BoundObject = SequencerPtr.IsValid() ? SequencerPtr->FindSpawnedObjectOrTemplate(Guid) : nullptr;

	if (AActor* Actor = Cast<AActor>(BoundObject))
	{
		for (UActorComponent* Component : Actor->GetComponents())
		{
			if (UGeometryCollectionComponent* GeometryMeshComp = Cast<UGeometryCollectionComponent>(Component))
			{
				return GeometryMeshComp;
			}
		}
	}
	else if(UGeometryCollectionComponent* GeometryMeshComp = Cast<UGeometryCollectionComponent>(BoundObject))
	{
		return GeometryMeshComp;
	}

	return nullptr;
}


FGeometryCollectionTrackSection::FGeometryCollectionTrackSection( UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer)
	: Section(*CastChecked<UMovieSceneGeometryCollectionSection>(&InSection))
	, Sequencer(InSequencer)
	, InitialStartOffsetDuringResize(0)
	, InitialStartTimeDuringResize(0)
{ }


UMovieSceneSection* FGeometryCollectionTrackSection::GetSectionObject()
{ 
	return &Section;
}


FText FGeometryCollectionTrackSection::GetSectionTitle() const
{
	if (Section.Params.GeometryCollectionCache.ResolveObject() != nullptr )
	{
		UGeometryCollectionCache* GeometryCollectionCache = Cast<UGeometryCollectionCache>(Section.Params.GeometryCollectionCache.ResolveObject());
		if (GeometryCollectionCache)
		{	
			return FText::FromString(GeometryCollectionCache->GetName());
		}
	}
	return LOCTEXT("NoGeometryCollectionSection", "No GeometryCollection");
}


float FGeometryCollectionTrackSection::GetSectionHeight() const
{
	return (float)GeometryCollectionEditorConstants::AnimationTrackHeight;
}


int32 FGeometryCollectionTrackSection::OnPaintSection( FSequencerSectionPainter& Painter ) const
{
	using namespace UE::Sequencer;

	const ESlateDrawEffect DrawEffects = Painter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	
	const FTimeToPixel& TimeToPixelConverter = Painter.GetTimeConverter();

	int32 LayerId = Painter.PaintSectionBackground();

	static const FSlateBrush* GenericDivider = FAppStyle::GetBrush("Sequencer.GenericDivider");

	if (!Section.HasStartFrame() || !Section.HasEndFrame())
	{
		return LayerId;
	}

	FFrameRate TickResolution = TimeToPixelConverter.GetTickResolution();

	// Add lines where the animation starts and ends/loops
	float AnimPlayRate = FMath::IsNearlyZero(Section.Params.PlayRate) ? 1.0f : Section.Params.PlayRate;
	float SeqLength = Section.Params.GetSequenceLength() - TickResolution.AsSeconds(Section.Params.StartFrameOffset + Section.Params.EndFrameOffset) / AnimPlayRate;

	if (!FMath::IsNearlyZero(SeqLength, KINDA_SMALL_NUMBER) && SeqLength > 0)
	{
		float MaxOffset  = Section.GetRange().Size<FFrameTime>() / TickResolution;
		float OffsetTime = SeqLength;
		float StartTime  = Section.GetInclusiveStartFrame() / TickResolution;

		while (OffsetTime < MaxOffset)
		{
			float OffsetPixel = TimeToPixelConverter.SecondsToPixel(StartTime + OffsetTime) - TimeToPixelConverter.SecondsToPixel(StartTime);

			FSlateDrawElement::MakeBox(
				Painter.DrawElements,
				LayerId,
				Painter.SectionGeometry.MakeChild(
					FVector2D(2.f, Painter.SectionGeometry.Size.Y-2.f),
					FSlateLayoutTransform(FVector2D(OffsetPixel, 1.f))
				).ToPaintGeometry(),
				GenericDivider,
				DrawEffects
			);

			OffsetTime += SeqLength;
		}
	}

	TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
	if (Painter.bIsSelected && SequencerPtr.IsValid())
	{
		FFrameTime CurrentTime = SequencerPtr->GetLocalTime().Time;
		if (Section.GetRange().Contains(CurrentTime.FrameNumber) && Section.Params.GeometryCollectionCache.ResolveObject() != nullptr)
		{
			const float Time = TimeToPixelConverter.FrameToPixel(CurrentTime); 

			UGeometryCollectionComponent* GeometryCollection = Cast<UGeometryCollectionComponent>(Section.Params.GeometryCollectionCache.ResolveObject());

			// Draw the current time next to the scrub handle
			const float AnimTime = Section.MapTimeToAnimation(CurrentTime, TickResolution);			
			int32 FrameTime = 0; // GeometryCollection->GetFrameAtTime(AnimTime);
			FString FrameString = FString::FromInt(FrameTime);

			const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);
			const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			FVector2D TextSize = FontMeasureService->Measure(FrameString, SmallLayoutFont);

			// Flip the text position if getting near the end of the view range
			static const float TextOffsetPx = 10.f;
			bool  bDrawLeft = (Painter.SectionGeometry.Size.X - Time) < (TextSize.X + 22.f) - TextOffsetPx;
			float TextPosition = bDrawLeft ? Time - TextSize.X - TextOffsetPx : Time + TextOffsetPx;
			//handle mirrored labels
			const float MajorTickHeight = 9.0f; 
			FVector2D TextOffset(TextPosition, Painter.SectionGeometry.Size.Y - (MajorTickHeight + TextSize.Y));

			const FLinearColor DrawColor = FAppStyle::GetSlateColor("SelectionColor").GetColor(FWidgetStyle());
			const FVector2D BoxPadding = FVector2D(4.0f, 2.0f);
			// draw time string
	
			FSlateDrawElement::MakeBox(
				Painter.DrawElements,
				LayerId + 5,
				Painter.SectionGeometry.ToPaintGeometry(TextSize + 2.0f * BoxPadding, FSlateLayoutTransform(TextOffset - BoxPadding)),
				FAppStyle::GetBrush("WhiteBrush"),
				ESlateDrawEffect::None,
				FLinearColor::Black.CopyWithNewOpacity(0.5f)
			);

			FSlateDrawElement::MakeText(
				Painter.DrawElements,
				LayerId + 6,
				Painter.SectionGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(TextOffset)),
				FrameString,
				SmallLayoutFont,
				DrawEffects,
				DrawColor
			);

		}
	}
	
	return LayerId;
}

void FGeometryCollectionTrackSection::BeginResizeSection()
{
	InitialStartOffsetDuringResize = Section.Params.StartFrameOffset;
	InitialStartTimeDuringResize   = Section.HasStartFrame() ? Section.GetInclusiveStartFrame() : 0;
}

void FGeometryCollectionTrackSection::ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime)
{
	// Adjust the start offset when resizing from the beginning
	if (ResizeMode == SSRM_LeadingEdge)
	{
		FFrameRate FrameRate = Section.GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber StartOffset = FrameRate.AsFrameNumber((ResizeTime - InitialStartTimeDuringResize) / FrameRate * Section.Params.PlayRate);

		StartOffset += InitialStartOffsetDuringResize;

		// Ensure start offset is not less than 0 and adjust ResizeTime
		if (StartOffset < 0)
		{
			ResizeTime = ResizeTime - StartOffset;

			StartOffset = FFrameNumber(0);
		}

		Section.Params.StartFrameOffset = StartOffset;
	}

	ISequencerSection::ResizeSection(ResizeMode, ResizeTime);
}

void FGeometryCollectionTrackSection::BeginSlipSection()
{
	BeginResizeSection();
}

void FGeometryCollectionTrackSection::SlipSection(FFrameNumber SlipTime)
{
	FFrameRate FrameRate = Section.GetTypedOuter<UMovieScene>()->GetTickResolution();
	FFrameNumber StartOffset = FrameRate.AsFrameNumber((SlipTime - InitialStartTimeDuringResize) / FrameRate * Section.Params.PlayRate);

	StartOffset += InitialStartOffsetDuringResize;

	// Ensure start offset is not less than 0 and adjust ResizeTime
	if (StartOffset < 0)
	{
		SlipTime = SlipTime - StartOffset;

		StartOffset = FFrameNumber(0);
	}

	Section.Params.StartFrameOffset = StartOffset;

	ISequencerSection::SlipSection(SlipTime);
}

void FGeometryCollectionTrackSection::BeginDilateSection()
{
	Section.PreviousPlayRate = Section.Params.PlayRate; //make sure to cache the play rate
}
void FGeometryCollectionTrackSection::DilateSection(const TRange<FFrameNumber>& NewRange, float DilationFactor)
{
	Section.Params.PlayRate = Section.PreviousPlayRate / DilationFactor;
	Section.SetRange(NewRange);
}

FGeometryCollectionTrackEditor::FGeometryCollectionTrackEditor( TSharedRef<ISequencer> InSequencer )
	: FMovieSceneTrackEditor( InSequencer ) 
{ }


TSharedRef<ISequencerTrackEditor> FGeometryCollectionTrackEditor::CreateTrackEditor( TSharedRef<ISequencer> InSequencer )
{
	return MakeShareable( new FGeometryCollectionTrackEditor( InSequencer ) );
}

bool FGeometryCollectionTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneGeometryCollectionTrack::StaticClass()) : ETrackSupport::Default;

	if (TrackSupported == ETrackSupport::NotSupported)
	{
		return false;
	}

	return (InSequence && InSequence->IsA(ULevelSequence::StaticClass())) || TrackSupported == ETrackSupport::Supported;
}

bool FGeometryCollectionTrackEditor::SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const
{
	return Type == UMovieSceneGeometryCollectionTrack::StaticClass();
}


TSharedRef<ISequencerSection> FGeometryCollectionTrackEditor::MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding )
{
	check( SupportsType( SectionObject.GetOuter()->GetClass() ) );
	
	return MakeShareable( new FGeometryCollectionTrackSection(SectionObject, GetSequencer()) );
}

FText FGeometryCollectionTrackEditor::GetDisplayName() const
{
	return LOCTEXT("GeometryCollectionTrackEditor_DisplayName", "Geometry Collection");
}

void FGeometryCollectionTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	if (ObjectClass->IsChildOf(UGeometryCollectionComponent::StaticClass()))
	{
		for (const FGuid& ObjectBinding : ObjectBindings)
		{
			UGeometryCollectionComponent* GeomMeshComp = AcquireGeometryCollectionFromObjectGuid(ObjectBinding, GetSequencer());

			if (GeomMeshComp)
			{
				UMovieSceneTrack* Track = nullptr;

				MenuBuilder.AddMenuEntry(
					NSLOCTEXT("Sequencer", "AddGeometryCollection", "Geometry Collection"),
					NSLOCTEXT("Sequencer", "AddGeometryCollectionTooltip", "Adds a Geometry Collection track."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(this, &FGeometryCollectionTrackEditor::BuildGeometryCollectionTrack, ObjectBinding, GeomMeshComp, Track)
					)
				);
			}
		}
	}
}

void FGeometryCollectionTrackEditor::BuildGeometryCollectionTrack(FGuid ObjectBinding, UGeometryCollectionComponent *GeometryCollectionComponent, UMovieSceneTrack* Track)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	if (SequencerPtr.IsValid() && ObjectBinding.IsValid())
	{
		UObject* Object = SequencerPtr->FindSpawnedObjectOrTemplate(ObjectBinding);
		if (Object)
		{
			const FScopedTransaction Transaction(LOCTEXT("AddGeometryCollection_Transaction", "Add Geometry Collection"));
			AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FGeometryCollectionTrackEditor::AddKeyInternal, Object, GeometryCollectionComponent, Track));
		}
	}
}

FKeyPropertyResult FGeometryCollectionTrackEditor::AddKeyInternal( FFrameNumber KeyTime, UObject* Object, UGeometryCollectionComponent* GeometryCollectionComponent, UMovieSceneTrack* Track)
{
	FKeyPropertyResult KeyPropertyResult;

	FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject( Object );
	FGuid ObjectHandle = HandleResult.Handle;
	KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;
	if (ObjectHandle.IsValid())
	{
		if (!Track)
		{
			Track = AddTrack(GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene(), ObjectHandle, UMovieSceneGeometryCollectionTrack::StaticClass(), NAME_None);
			KeyPropertyResult.bTrackCreated = true;
		}

		if (ensure(Track))
		{
			Track->Modify();

			UMovieSceneSection* NewSection = Cast<UMovieSceneGeometryCollectionTrack>(Track)->AddNewAnimation( KeyTime, GeometryCollectionComponent );
			KeyPropertyResult.bTrackModified = true;
			KeyPropertyResult.SectionsCreated.Add(NewSection);

			GetSequencer()->EmptySelection();
			GetSequencer()->SelectSection(NewSection);
			GetSequencer()->ThrobSectionSelection();
		}
	}

	return KeyPropertyResult;
}


TSharedPtr<SWidget> FGeometryCollectionTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	UGeometryCollectionComponent* GeomMeshComp = AcquireGeometryCollectionFromObjectGuid(ObjectBinding, GetSequencer());

	if (GeomMeshComp)
	{
		TWeakPtr<ISequencer> WeakSequencer = GetSequencer();

		auto SubMenuCallback = [this, ObjectBinding, Track, GeomMeshComp]() -> TSharedRef<SWidget>
		{
			FMenuBuilder MenuBuilder(true, nullptr);

			BuildGeometryCollectionTrack(ObjectBinding, GeomMeshComp, Track);
			
			return MenuBuilder.MakeWidget();
		};

		return UE::Sequencer::MakeAddButton(LOCTEXT("GeomCollectionText", "Geometry Collection"), FOnGetContent::CreateLambda(SubMenuCallback), Params.ViewModel);
	}
	else
	{
		return TSharedPtr<SWidget>();
	}
	
}

const FSlateBrush* FGeometryCollectionTrackEditor::GetIconBrush() const
{
	return FSlateIconFinder::FindIconForClass(UGeometryCollectionComponent::StaticClass()).GetIcon();
}


#undef LOCTEXT_NAMESPACE
