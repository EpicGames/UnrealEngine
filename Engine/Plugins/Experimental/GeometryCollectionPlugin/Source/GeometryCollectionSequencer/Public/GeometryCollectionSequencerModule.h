// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ISequencerModule.h"
#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "GeometryCollectionTrackEditor.h"

/**
 * The public interface to this module
 */
class FGeometryCollectionSequencerModule : public IModuleInterface
{
public:

	virtual void StartupModule() override
	{

		ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
		TrackEditorBindingHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FGeometryCollectionTrackEditor::CreateTrackEditor));

	}

	virtual void ShutdownModule() override
	{

		ISequencerModule* SequencerModulePtr = FModuleManager::Get().GetModulePtr<ISequencerModule>("Sequencer");
		if (SequencerModulePtr)
		{
			SequencerModulePtr->UnRegisterTrackEditor(TrackEditorBindingHandle);
		}

	}

	FDelegateHandle TrackEditorBindingHandle;
};

