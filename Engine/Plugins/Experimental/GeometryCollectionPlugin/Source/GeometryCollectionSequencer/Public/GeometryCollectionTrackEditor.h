// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Widgets/SWidget.h"
#include "ISequencer.h"
#include "MovieSceneTrack.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "MovieSceneTrackEditor.h"

struct FAssetData;
class FMenuBuilder;
class FSequencerSectionPainter;
class UMovieSceneGeometryCollectionSection;
class UMovieSceneSequence;
class UGeometryCollectionComponent;

/**
 * Tools for Geometry Collection tracks
 */
class FGeometryCollectionTrackEditor : public FMovieSceneTrackEditor
{
public:
	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 */
	FGeometryCollectionTrackEditor( TSharedRef<ISequencer> InSequencer );

	/** Virtual destructor. */
	virtual ~FGeometryCollectionTrackEditor() = default;

	/**
	 * Creates an instance of this class.  Called by a sequencer 
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor( TSharedRef<ISequencer> OwningSequencer );

public:

	// ISequencerTrackEditor interface
	virtual FText GetDisplayName() const override;
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface( UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding ) override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual bool SupportsType( TSubclassOf<UMovieSceneTrack> Type ) const override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	virtual const FSlateBrush* GetIconBrush() const override;

private:

	void BuildGeometryCollectionTrack(FGuid ObjectBinding, UGeometryCollectionComponent *GeometryCollectionComponent, UMovieSceneTrack* Track);

	/** Delegate for AnimatablePropertyChanged in AddKey */
	FKeyPropertyResult AddKeyInternal(FFrameNumber KeyTime, UObject* Object, UGeometryCollectionComponent* GeometryCollectionComponent, UMovieSceneTrack* Track);
};


/** Class for animation sections */
class FGeometryCollectionTrackSection
	: public ISequencerSection
	, public TSharedFromThis<FGeometryCollectionTrackSection>
{
public:

	/** Constructor. */
	FGeometryCollectionTrackSection( UMovieSceneSection& InSection, TWeakPtr<ISequencer> InSequencer);

	/** Virtual destructor. */
	virtual ~FGeometryCollectionTrackSection() = default;

public:

	// ISequencerSection interface

	virtual UMovieSceneSection* GetSectionObject() override;
	virtual FText GetSectionTitle() const override;
	virtual float GetSectionHeight() const override;
	virtual int32 OnPaintSection( FSequencerSectionPainter& Painter ) const override;
	virtual void BeginResizeSection() override;
	virtual void ResizeSection(ESequencerSectionResizeMode ResizeMode, FFrameNumber ResizeTime) override;
	virtual void BeginSlipSection() override;
	virtual void SlipSection(FFrameNumber SlipTime) override;
	virtual void BeginDilateSection() override;
	virtual void DilateSection(const TRange<FFrameNumber>& NewRange, float DilationFactor) override;



private:

	/** The section we are visualizing */
	UMovieSceneGeometryCollectionSection& Section;

	/** Used to draw animation frame, need selection state and local time*/
	TWeakPtr<ISequencer> Sequencer;

	/** Cached start offset value valid only during resize */
	FFrameNumber InitialStartOffsetDuringResize;
	
	/** Cached start time valid only during resize */
	FFrameNumber InitialStartTimeDuringResize;
};
