// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class GeometryCollectionTracks : ModuleRules
    {
        public GeometryCollectionTracks(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                    "CoreUObject",
                    "Engine",
                    "MovieScene",
                    "MovieSceneTracks",
					"Chaos",
                    "GeometryCollectionEngine",
                }
            );

            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "AnimGraphRuntime",
                    "TimeManagement",
                }
            );


            PublicIncludePathModuleNames.Add("TargetPlatform");

            if (Target.bBuildEditor)
            {
                PublicIncludePathModuleNames.Add("GeometryCollectionSequencer");
                DynamicallyLoadedModuleNames.Add("GeometryCollectionSequencer");
                
                PrivateDependencyModuleNames.AddRange(
	                new string[]
	                {
		                "SlateCore",
		                "Slate",
	                }
                );
            }
        }
    }
}
