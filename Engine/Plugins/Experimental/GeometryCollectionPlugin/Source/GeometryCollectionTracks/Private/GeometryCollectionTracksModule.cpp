// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollectionTracksModule.h"
#if WITH_EDITOR
#include "GeometryCollectionSequencerModule.h"
#include "GeometryCollection/GeometryCollectionComponentPlugin.h"
#endif // WITH_EDITOR

IMPLEMENT_MODULE(FGeometryCollectionTracksModule, GeometryCollectionTracks)

void FGeometryCollectionTracksModule::StartupModule()
{
#if WITH_EDITOR
	FGeometryCollectionSequencerModule& Module = FModuleManager::LoadModuleChecked<FGeometryCollectionSequencerModule>(TEXT("GeometryCollectionSequencer"));
	
	//HACK: The geometry collection engine module will not be added to live coding without this.
	//We should probably move the engine module into this plugin or something, but for now this makes it so we can change the GC engine module and use live coding
	IGeometryCollectionComponentPlugin::Get();
#endif
}

void FGeometryCollectionTracksModule::ShutdownModule()
{
}
