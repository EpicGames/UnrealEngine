// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneGeometryCollectionSection.h"
#include "MovieSceneGeometryCollectionTemplate.h"
#include "Logging/MessageLog.h"
#include "MovieScene.h"
#include "UObject/SequencerObjectVersion.h"
#include "MovieSceneTimeHelpers.h"
#include "Misc/FrameRate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneGeometryCollectionSection)

#define LOCTEXT_NAMESPACE "MovieSceneGeometryCollectionSection"

namespace
{
	float GeometryCollectionDeprecatedMagicNumber = TNumericLimits<float>::Lowest();
}

FMovieSceneGeometryCollectionParams::FMovieSceneGeometryCollectionParams()
{
	PlayRate = 1.f;
}

UMovieSceneGeometryCollectionSection::UMovieSceneGeometryCollectionSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	BlendType = EMovieSceneBlendType::Absolute;
	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::RestoreState);

#if WITH_EDITOR

	PreviousPlayRate = Params.PlayRate;

#endif
}

TOptional<FFrameTime> UMovieSceneGeometryCollectionSection::GetOffsetTime() const
{
	return TOptional<FFrameTime>(Params.StartFrameOffset);
}

void UMovieSceneGeometryCollectionSection::MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate)
{
	if (Params.StartFrameOffset.Value > 0)
	{
		FFrameNumber NewStartFrameOffset = ConvertFrameTime(FFrameTime(Params.StartFrameOffset), SourceRate, DestinationRate).FloorToFrame();
		Params.StartFrameOffset = NewStartFrameOffset;
	}

	if (Params.EndFrameOffset.Value > 0)
	{
		FFrameNumber NewEndFrameOffset = ConvertFrameTime(FFrameTime(Params.EndFrameOffset), SourceRate, DestinationRate).FloorToFrame();
		Params.EndFrameOffset = NewEndFrameOffset;
	}
}

UObject* UMovieSceneGeometryCollectionSection::GetSourceObject() const
{
	return Params.GeometryCollectionCache.ResolveObject();
}

FFrameNumber GetStartOffsetAtTrimTime(FQualifiedFrameTime TrimTime, const FMovieSceneGeometryCollectionParams& Params, FFrameNumber StartFrame, FFrameRate FrameRate)
{
	float AnimPlayRate = FMath::IsNearlyZero(Params.PlayRate) ? 1.0f : Params.PlayRate;
	float AnimPosition = (TrimTime.Time - StartFrame) / TrimTime.Rate * AnimPlayRate;
	float SeqLength = Params.GetSequenceLength() - FrameRate.AsSeconds(Params.StartFrameOffset + Params.EndFrameOffset) / AnimPlayRate;

	FFrameNumber NewOffset = FrameRate.AsFrameNumber(FMath::Fmod(AnimPosition, SeqLength));
	NewOffset += Params.StartFrameOffset;

	return NewOffset;
}


TOptional<TRange<FFrameNumber> > UMovieSceneGeometryCollectionSection::GetAutoSizeRange() const
{
	FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();

	FFrameTime AnimationLength = Params.GetSequenceLength() * FrameRate;

	return TRange<FFrameNumber>(GetInclusiveStartFrame(), GetInclusiveStartFrame() + AnimationLength.FrameNumber);
}


void UMovieSceneGeometryCollectionSection::TrimSection(FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys)
{
	SetFlags(RF_Transactional);

	if (TryModify())
	{
		if (bTrimLeft)
		{
			FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();

			Params.StartFrameOffset = HasStartFrame() ? GetStartOffsetAtTrimTime(TrimTime, Params, GetInclusiveStartFrame(), FrameRate) : 0;
		}

		Super::TrimSection(TrimTime, bTrimLeft, bDeleteKeys);
	}
}

UMovieSceneSection* UMovieSceneGeometryCollectionSection::SplitSection(FQualifiedFrameTime SplitTime, bool bDeleteKeys)
{
	const FFrameNumber InitialStartFrameOffset = Params.StartFrameOffset;

	FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();

	const FFrameNumber NewOffset = HasStartFrame() ? GetStartOffsetAtTrimTime(SplitTime, Params, GetInclusiveStartFrame(), FrameRate) : 0;

	UMovieSceneSection* NewSection = Super::SplitSection(SplitTime, bDeleteKeys);
	if (NewSection != nullptr)
	{
		UMovieSceneGeometryCollectionSection* NewGeometrySection = Cast<UMovieSceneGeometryCollectionSection>(NewSection);
		NewGeometrySection->Params.StartFrameOffset = NewOffset;
	}

	// Restore original offset modified by splitting
	Params.StartFrameOffset = InitialStartFrameOffset;

	return NewSection;
}


void UMovieSceneGeometryCollectionSection::GetSnapTimes(TArray<FFrameNumber>& OutSnapTimes, bool bGetSectionBorders) const
{
	Super::GetSnapTimes(OutSnapTimes, bGetSectionBorders);

	const FFrameRate   FrameRate  = GetTypedOuter<UMovieScene>()->GetTickResolution();
	const FFrameNumber StartFrame = GetInclusiveStartFrame();
	const FFrameNumber EndFrame   = GetExclusiveEndFrame() - 1; // -1 because we don't need to add the end frame twice

	const float AnimPlayRate     = FMath::IsNearlyZero(Params.PlayRate) ? 1.0f : Params.PlayRate;
	const float SeqLengthSeconds = Params.GetSequenceLength() - FrameRate.AsSeconds(Params.StartFrameOffset + Params.EndFrameOffset) / AnimPlayRate;

	FFrameTime SequenceFrameLength = SeqLengthSeconds * FrameRate;
	if (SequenceFrameLength.FrameNumber > 1)
	{
		// Snap to the repeat times
		FFrameTime CurrentTime = StartFrame;
		while (CurrentTime < EndFrame)
		{
			OutSnapTimes.Add(CurrentTime.FrameNumber);
			CurrentTime += SequenceFrameLength;
		}
	}
}

float UMovieSceneGeometryCollectionSection::MapTimeToAnimation(FFrameTime InPosition, FFrameRate InTickResolution) const
{
	FMovieSceneGeometryCollectionSectionTemplateParameters TemplateParams(Params, GetInclusiveStartFrame(), GetExclusiveEndFrame());
	return TemplateParams.MapTimeToAnimation(InPosition, InTickResolution);
}


#if WITH_EDITOR
void UMovieSceneGeometryCollectionSection::PreEditChange(FProperty* PropertyAboutToChange)
{
	// Store the current play rate so that we can compute the amount to compensate the section end time when the play rate changes
	PreviousPlayRate = Params.PlayRate;

	Super::PreEditChange(PropertyAboutToChange);
}

void UMovieSceneGeometryCollectionSection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Adjust the duration automatically if the play rate changes
	if (PropertyChangedEvent.Property != nullptr &&
		PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FMovieSceneGeometryCollectionParams, PlayRate))
	{
		float NewPlayRate = Params.PlayRate;

		if (!FMath::IsNearlyZero(NewPlayRate))
		{
			float CurrentDuration = UE::MovieScene::DiscreteSize(GetRange());
			float NewDuration = CurrentDuration * (PreviousPlayRate / NewPlayRate);
			SetEndFrame( GetInclusiveStartFrame() + FMath::FloorToInt(NewDuration) );

			PreviousPlayRate = NewPlayRate;
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

#undef LOCTEXT_NAMESPACE 

