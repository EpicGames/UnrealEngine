// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneGeometryCollectionTemplate.h"
#include "Compilation/MovieSceneCompilerRules.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "Evaluation/MovieSceneEvaluation.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"
#include "IMovieScenePlayer.h"
#include "UObject/ObjectKey.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneGeometryCollectionTemplate)


DECLARE_CYCLE_STAT(TEXT("Geometry Collection Evaluate"), MovieSceneEval_GeometryCollection_Evaluate, STATGROUP_MovieSceneEval);
DECLARE_CYCLE_STAT(TEXT("Geometry Collection Token Execute"), MovieSceneEval_GeometryCollection_TokenExecute, STATGROUP_MovieSceneEval);

/** Used to set Manual Tick back to previous when outside section */
struct FPreAnimatedGeometryCollectionTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const
	{
		struct FToken : IMovieScenePreAnimatedToken
		{
			FToken(UGeometryCollectionComponent* InComponent)
			{
				// Cache any information about the UGeometryCollectionComponent you need to when we start evaluating, ie:
				// What is it's Cache's Cache Mode, or what is the Object Type. These get overriden when actual evaluation happens.
				OriginalCache = InComponent->CacheParameters.TargetCache;
				OriginalObjectType = InComponent->ObjectType;
				OriginalCachePlayback = InComponent->CachePlayback;
				OriginalCacheMode = InComponent->CacheParameters.CacheMode;
			}

			virtual void RestoreState(UObject& Object, const UE::MovieScene::FRestoreStateParams& Params)
			{
				// This is called after Sequencer tears down, or stops evaluating the section (depending on user settings).
				// Use this to restore any changes you made to the object.
				UGeometryCollectionComponent* Component = CastChecked<UGeometryCollectionComponent>(&Object);
				Component->CacheParameters.TargetCache = OriginalCache;
				Component->ObjectType = OriginalObjectType;
				Component->CacheParameters.CacheMode = OriginalCacheMode;
				Component->DesiredCacheTime = -1.f; // This resets the cache so it's no longer playing.
				Component->CachePlayback = true; // This has to be true for the Desired Cache Time to be read and set us backwards.

			}

			EObjectStateTypeEnum OriginalObjectType;
			UGeometryCollectionCache* OriginalCache;
			bool OriginalCachePlayback;
			EGeometryCollectionCacheType OriginalCacheMode;
		};

		return FToken(CastChecked<UGeometryCollectionComponent>(&Object));
	}
	static FMovieSceneAnimTypeID GetAnimTypeID()
	{
		return TMovieSceneAnimTypeID<FPreAnimatedGeometryCollectionTokenProducer>();
	}
};


/** A movie scene execution token that executes a geometry collection */
struct FGeometryCollectionExecutionToken
	: IMovieSceneExecutionToken
	
{
	FGeometryCollectionExecutionToken(const FMovieSceneGeometryCollectionSectionTemplateParameters &InParams):
		Params(InParams)
	{}

	/** Execute this token, operating on all objects referenced by 'Operand' */
	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_GeometryCollection_TokenExecute)
		for (TWeakObjectPtr<> WeakObject : Player.FindBoundObjects(Operand))
		{
			UGeometryCollectionComponent* GeometryCollectionComponent = Cast<UGeometryCollectionComponent>(WeakObject.Get());
			if (GeometryCollectionComponent)
			{
				if (UGeometryCollectionCache* GeometryCollectionCache = Cast<UGeometryCollectionCache>(Params.GeometryCollectionCache.ResolveObject()))
				{
					// Validate that the Cache is compatible with the Collection before attempting to play back.
					const bool bIsValidCache = GeometryCollectionCache->CompatibleWithForPlayback(GeometryCollectionComponent->GetRestCollection());
					if (!bIsValidCache)
					{
						UE_LOGF(LogMovieScene, Warning, "Unsupported Geometry Collection Cache (%ls) for Component (%ls), ignoring playback!", *GeometryCollectionCache->GetFullName(), *GeometryCollectionComponent->GetFullName());
						continue;
					}

					Player.SavePreAnimatedState(*GeometryCollectionComponent, FPreAnimatedGeometryCollectionTokenProducer::GetAnimTypeID(), FPreAnimatedGeometryCollectionTokenProducer());

					// Now that we've stored the PreAnimated State we change whatever settings are needed for Sequencer to actually control this. This is done every frame in the event
					// that Gameplay Code tries to fight Sequencer and resets a setting.
					GeometryCollectionComponent->ObjectType = EObjectStateTypeEnum::Chaos_Object_Kinematic;
					GeometryCollectionComponent->CachePlayback = true;
					GeometryCollectionComponent->CacheParameters.CacheMode = EGeometryCollectionCacheType::None;

					GeometryCollectionComponent->CacheParameters.TargetCache = GeometryCollectionCache;

					// Finding out what time (relative to the start of the section)
					const float TimeInSeconds = Params.MapTimeToAnimation(Context.GetTime(), Context.GetFrameRate());

					GeometryCollectionComponent->DesiredCacheTime = TimeInSeconds;
				}
			}
		}
	}

	FMovieSceneGeometryCollectionSectionTemplateParameters Params;
};

FMovieSceneGeometryCollectionSectionTemplate::FMovieSceneGeometryCollectionSectionTemplate(const UMovieSceneGeometryCollectionSection& InSection)
	: Params(InSection.Params, InSection.GetInclusiveStartFrame(), InSection.GetExclusiveEndFrame())
{
}

//We use a token here so we can set the manual tick state back to what it was previously when outside this section.
//This is similar to how Skeletal Animation evaluation also works.
void FMovieSceneGeometryCollectionSectionTemplate::Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_GeometryCollection_Evaluate)
	ExecutionTokens.Add(FGeometryCollectionExecutionToken(Params));
}

float FMovieSceneGeometryCollectionSectionTemplateParameters::MapTimeToAnimation(FFrameTime InPosition, FFrameRate InTickResolution) const
{
	InPosition = FMath::Clamp(InPosition, FFrameTime(SectionStartTime), FFrameTime(SectionEndTime-1));

	const float SectionPlayRate = PlayRate;
	const float AnimPlayRate = FMath::IsNearlyZero(SectionPlayRate) ? 1.0f : SectionPlayRate;

	const float SeqLength = GetSequenceLength() - InTickResolution.AsSeconds(StartFrameOffset + EndFrameOffset);

	float AnimPosition = FFrameTime::FromDecimal((InPosition - SectionStartTime).AsDecimal() * AnimPlayRate) / InTickResolution;
	if (SeqLength > 0.f)
	{
		// We don't support looping right now as it confuses the scrub with the current cache evaluation, just clamp so it holds on the
		// last frame.
		AnimPosition = FMath::Clamp(AnimPosition, AnimPosition, SeqLength);
	}
	AnimPosition += InTickResolution.AsSeconds(StartFrameOffset);
	// if (bReverse)
	// {
	// 	AnimPosition = (SeqLength - (AnimPosition - InFrameRate.AsSeconds(StartFrameOffset))) + InFrameRate.AsSeconds(StartFrameOffset);
	// }

	return AnimPosition;
}

