// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneGeometryCollectionTrack.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "MovieSceneGeometryCollectionSection.h"
#include "Compilation/MovieSceneCompilerRules.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "MovieSceneGeometryCollectionTemplate.h"
#include "Compilation/IMovieSceneTemplateGenerator.h"
#include "MovieSceneGeometryCollectionTemplate.h"
#include "MovieScene.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneGeometryCollectionTrack)

#define LOCTEXT_NAMESPACE "MovieSceneGeometryCollectionTrack"


/* UMovieSceneGeometryCollectionTrack structors
 *****************************************************************************/

UMovieSceneGeometryCollectionTrack::UMovieSceneGeometryCollectionTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(49, 103, 213, 255);
#endif

	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);

	EvalOptions.bCanEvaluateNearestSection = true;
	EvalOptions.bEvaluateInPreroll = true;
}


/* UMovieSceneGeometryCollectionTrack interface
 *****************************************************************************/

UMovieSceneSection* UMovieSceneGeometryCollectionTrack::AddNewAnimation(FFrameNumber KeyTime, UGeometryCollectionComponent* GeometryCollectionComponent)
{
	UMovieSceneGeometryCollectionSection* NewSection = Cast<UMovieSceneGeometryCollectionSection>(CreateNewSection());
	{
		// Look to see if there's already a cached recording for this component and use that length. Otherwise we'll just default to five seconds.
		FFrameTime Duration;
		if (GeometryCollectionComponent->CacheParameters.TargetCache)
		{
			const FRecordedTransformTrack* Data = GeometryCollectionComponent->CacheParameters.TargetCache->GetData();
			if (Data->Records.Num() > 0)
			{
				Duration = Data->Records.Last().Timestamp * GetTypedOuter<UMovieScene>()->GetTickResolution();
			}
		}
		
		if(Duration == FFrameTime())
		{
			// They haven't recorded this to a cached asset before so we don't actually know how long it'll be. Default to 5s.
			Duration = 5.0f * GetTypedOuter<UMovieScene>()->GetTickResolution();
		}
		NewSection->InitialPlacementOnRow(AnimationSections, KeyTime, Duration.FrameNumber.Value, INDEX_NONE);
		NewSection->Params.GeometryCollectionCache = GeometryCollectionComponent->CacheParameters.TargetCache;
	}

	AddSection(*NewSection);

	return NewSection;
}


TArray<UMovieSceneSection*> UMovieSceneGeometryCollectionTrack::GetAnimSectionsAtTime(FFrameNumber Time)
{
	TArray<UMovieSceneSection*> Sections;
	for (auto Section : AnimationSections)
	{
		if (Section->IsTimeWithinSection(Time))
		{
			Sections.Add(Section);
		}
	}

	return Sections;
}


/* UMovieSceneTrack interface
 *****************************************************************************/


const TArray<UMovieSceneSection*>& UMovieSceneGeometryCollectionTrack::GetAllSections() const
{
	return AnimationSections;
}


bool UMovieSceneGeometryCollectionTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneGeometryCollectionSection::StaticClass();
}

UMovieSceneSection* UMovieSceneGeometryCollectionTrack::CreateNewSection()
{
	return NewObject<UMovieSceneGeometryCollectionSection>(this, NAME_None, RF_Transactional);
}


void UMovieSceneGeometryCollectionTrack::RemoveAllAnimationData()
{
	AnimationSections.Empty();
}


bool UMovieSceneGeometryCollectionTrack::HasSection(const UMovieSceneSection& Section) const
{
	return AnimationSections.Contains(&Section);
}


void UMovieSceneGeometryCollectionTrack::AddSection(UMovieSceneSection& Section)
{
	AnimationSections.Add(&Section);
}


void UMovieSceneGeometryCollectionTrack::RemoveSection(UMovieSceneSection& Section)
{
	AnimationSections.Remove(&Section);
}

void UMovieSceneGeometryCollectionTrack::RemoveSectionAt(int32 SectionIndex)
{
	AnimationSections.RemoveAt(SectionIndex);
}


bool UMovieSceneGeometryCollectionTrack::IsEmpty() const
{
	return AnimationSections.Num() == 0;
}

FMovieSceneEvalTemplatePtr UMovieSceneGeometryCollectionTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	return FMovieSceneGeometryCollectionSectionTemplate(*CastChecked<const UMovieSceneGeometryCollectionSection>(&InSection));
}

#if WITH_EDITORONLY_DATA

FText UMovieSceneGeometryCollectionTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("TrackName", "Geometry Collection");
}

#endif


#undef LOCTEXT_NAMESPACE

