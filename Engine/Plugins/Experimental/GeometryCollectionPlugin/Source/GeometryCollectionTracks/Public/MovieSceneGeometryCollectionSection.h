// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneSection.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "UObject/SoftObjectPath.h"
#include "GeometryCollection/GeometryCollectionCache.h"
#include "MovieSceneGeometryCollectionSection.generated.h"

USTRUCT()
struct FMovieSceneGeometryCollectionParams
{
	GENERATED_BODY()

	FMovieSceneGeometryCollectionParams();
	virtual ~FMovieSceneGeometryCollectionParams() = default;

	/** Gets the animation duration, modified by play rate */
	float GetDuration() const
	{ 
		if (!FMath::IsNearlyZero(PlayRate))
		{
			return GetSequenceLength() / PlayRate;
		}

		return 0.f;
	}

	/** Gets the animation sequence length, not modified by play rate */
	float GetSequenceLength() const
	{
		if (FMath::IsNearlyZero(PlayRate) || GeometryCollectionCache.ResolveObject() == nullptr)
		{
			return 0.f;
		}

		const FRecordedTransformTrack* CacheData = Cast<UGeometryCollectionCache>(GeometryCollectionCache.ResolveObject())->GetData();
		if (CacheData->Records.Num() > 0)
		{
			return CacheData->Records.Last().Timestamp;
		}

		return 0.0f;
	}

	/** The animation this section plays */
	UPROPERTY(EditAnywhere, Category="GeometryCollection", meta=(AllowedClasses = "/Script/GeometryCollectionEngine.GeometryCollectionCache"))
	FSoftObjectPath GeometryCollectionCache;

	/** The offset into the beginning of the animation clip */
	UPROPERTY(EditAnywhere, Category="GeometryCollection")
	FFrameNumber StartFrameOffset;
	
	/** The offset into the end of the animation clip */
	UPROPERTY(EditAnywhere, Category="GeometryCollection")
	FFrameNumber EndFrameOffset;
	
	/** The playback rate of the animation clip */
	UPROPERTY(EditAnywhere, Category="GeometryCollection")
	float PlayRate;
};

/**
 * Movie scene section that control geometry cache playback
 */
UCLASS( MinimalAPI )
class UMovieSceneGeometryCollectionSection
	: public UMovieSceneSection
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Animation", meta=(ShowOnlyInnerProperties))
	FMovieSceneGeometryCollectionParams Params;

	/** Get Frame Time as Animation Time*/
    virtual float MapTimeToAnimation(FFrameTime InPosition, FFrameRate InTickResolution) const;

protected:
	//~ UMovieSceneSection interface
	virtual TOptional<TRange<FFrameNumber> > GetAutoSizeRange() const override;
	virtual void TrimSection(FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys) override;
	virtual UMovieSceneSection* SplitSection(FQualifiedFrameTime SplitTime, bool bDeleteKeys) override;
	virtual void GetSnapTimes(TArray<FFrameNumber>& OutSnapTimes, bool bGetSectionBorders) const override;
	virtual TOptional<FFrameTime> GetOffsetTime() const override;
	virtual void MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate) override;
	virtual UObject* GetSourceObject() const override;

private:

	//~ UObject interface

#if WITH_EDITOR

	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
public:
	float PreviousPlayRate;

#endif


};
