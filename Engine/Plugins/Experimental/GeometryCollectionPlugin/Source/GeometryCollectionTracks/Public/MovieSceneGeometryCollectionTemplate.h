// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "MovieSceneGeometryCollectionSection.h"
#include "MovieSceneGeometryCollectionTemplate.generated.h"

USTRUCT()
struct FMovieSceneGeometryCollectionSectionTemplateParameters : public FMovieSceneGeometryCollectionParams
{
	GENERATED_BODY()

	FMovieSceneGeometryCollectionSectionTemplateParameters() {}
	FMovieSceneGeometryCollectionSectionTemplateParameters(const FMovieSceneGeometryCollectionParams& BaseParams, FFrameNumber InSectionStartTime, FFrameNumber InSectionEndTime)
		: FMovieSceneGeometryCollectionParams(BaseParams)
		, SectionStartTime(InSectionStartTime)
		, SectionEndTime(InSectionEndTime)
	{}

	float MapTimeToAnimation(FFrameTime InPosition, FFrameRate InTickResolution) const;

	UPROPERTY()
	FFrameNumber SectionStartTime;

	UPROPERTY()
	FFrameNumber SectionEndTime;
};

USTRUCT()
struct FMovieSceneGeometryCollectionSectionTemplate : public FMovieSceneEvalTemplate
{
	GENERATED_BODY()
	
	FMovieSceneGeometryCollectionSectionTemplate() {}
	FMovieSceneGeometryCollectionSectionTemplate(const UMovieSceneGeometryCollectionSection& Section);

	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;

	UPROPERTY()
	FMovieSceneGeometryCollectionSectionTemplateParameters Params;
};
